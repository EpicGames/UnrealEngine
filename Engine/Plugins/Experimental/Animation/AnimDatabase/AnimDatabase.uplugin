{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "AnimDatabase",
	"Description": "",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "AnimDatabase",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "AnimDatabaseEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "AnimationWarping",
			"Enabled": true
		},
		{
			"Name": "LearningCore",
			"Enabled": true
		},
		{
			"Name": "DrawDebugLibrary",
			"Enabled": true
		}
	]
}