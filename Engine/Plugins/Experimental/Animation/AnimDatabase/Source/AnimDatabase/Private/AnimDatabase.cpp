// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimDatabase.h"

#include "AnimDatabaseMath.h"
#include "AnimDatabaseFrameRanges.h"
#include "AnimDatabaseFrameAttribute.h"
#include "AnimDatabasePose.h"
#include "AnimDatabaseIndex.h"

#include "LearningFrameAttribute.h"
#include "LearningFrameRangeSet.h"
#include "LearningFrameSet.h"

#include "Animation/Skeleton.h"
#include "Animation/AnimSequence.h"
#include "Animation/AttributesRuntime.h"
#include "Animation/MirrorDataTable.h"
#include "AnimationRuntime.h"
#include "BonePose.h"
#include "Engine/SkeletalMesh.h"

#include "UObject/LinkerLoad.h"
#include "UObject/Package.h"
#include "Async/ParallelFor.h"
#if WITH_EDITOR
#include "ScopedTransaction.h"
#endif

#include "Dom/JsonObject.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"
#include "Misc/ScopedSlowTask.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonWriter.h"
#include "Serialization/JsonSerializer.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetRegistry/AssetData.h"
#include "Engine/Blueprint.h"
#include "DrawDebugLibrary.h"

#include "RigVMCore/RigVMRegistry.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimDatabase)

#define LOCTEXT_NAMESPACE "AnimDatabase"

void FAnimDatabaseModule::StartupModule()
{
	static UScriptStruct* const AllowedStructTypes[] =
	{
		FAnimDatabaseFrames::StaticStruct(),
		FAnimDatabaseFrameRanges::StaticStruct(),
		FAnimDatabaseFrameAttribute::StaticStruct(),
	};

	static TPair<UClass*, FRigVMRegistry::ERegisterObjectOperation> const AllowedObjectTypes[] =
	{
		{ UAnimDatabase::StaticClass(), FRigVMRegistry::ERegisterObjectOperation::Class },
		{ UAnimDatabaseIndex::StaticClass(), FRigVMRegistry::ERegisterObjectOperation::Class },
	};

	FRigVMRegistry& RigVMRegistry = FRigVMRegistry::Get();
	RigVMRegistry.RegisterStructTypes(AllowedStructTypes);
	RigVMRegistry.RegisterObjectTypes(AllowedObjectTypes);
}
	
IMPLEMENT_MODULE(FAnimDatabaseModule, AnimDatabase)

DEFINE_LOG_CATEGORY(LogAnimDatabase);

namespace UE::AnimDatabase::Private
{
	void LocationOrderFromStrings(TArray<int32>& LocationOrder, const FString& A, const FString& B, const FString& C)
	{
		if      (A == TEXT("Xposition") && B == TEXT("Yposition") && C == TEXT("Zposition")) { LocationOrder = { 0, 1, 2 }; return; }
		else if (A == TEXT("Xposition") && B == TEXT("Zposition") && C == TEXT("Yposition")) { LocationOrder = { 0, 2, 1 }; return; }
		else if (A == TEXT("Yposition") && B == TEXT("Xposition") && C == TEXT("Zposition")) { LocationOrder = { 1, 0, 2 }; return; }
		else if (A == TEXT("Yposition") && B == TEXT("Zposition") && C == TEXT("Xposition")) { LocationOrder = { 1, 2, 0 }; return; }
		else if (A == TEXT("Zposition") && B == TEXT("Xposition") && C == TEXT("Yposition")) { LocationOrder = { 2, 0, 1 }; return; }
		else if (A == TEXT("Zposition") && B == TEXT("Yposition") && C == TEXT("Xposition")) { LocationOrder = { 2, 1, 0 }; return; }
		else { LocationOrder = { 0, 1, 2 }; return; }
	}

	EAnimDatabaseRotationOrder RotationOrderFromStrings(const FString& A, const FString& B, const FString& C)
	{
		if      (A == TEXT("Xrotation") && B == TEXT("Yrotation") && C == TEXT("Zrotation")) { return EAnimDatabaseRotationOrder::XYZ; }
		else if (A == TEXT("Xrotation") && B == TEXT("Zrotation") && C == TEXT("Yrotation")) { return EAnimDatabaseRotationOrder::XZY; }
		else if (A == TEXT("Yrotation") && B == TEXT("Xrotation") && C == TEXT("Zrotation")) { return EAnimDatabaseRotationOrder::YXZ; }
		else if (A == TEXT("Yrotation") && B == TEXT("Zrotation") && C == TEXT("Xrotation")) { return EAnimDatabaseRotationOrder::YZX; }
		else if (A == TEXT("Zrotation") && B == TEXT("Xrotation") && C == TEXT("Yrotation")) { return EAnimDatabaseRotationOrder::ZXY; }
		else if (A == TEXT("Zrotation") && B == TEXT("Yrotation") && C == TEXT("Xrotation")) { return EAnimDatabaseRotationOrder::ZYX; }
		else { return EAnimDatabaseRotationOrder::XYZ; }
	}

	FVector3f RotationOrderAxis(const EAnimDatabaseRotationOrder RotationOrder, const uint8 OrderIndex)
	{
		check(OrderIndex >= 0 && OrderIndex < 3);

		switch (RotationOrder)
		{
		case EAnimDatabaseRotationOrder::XYZ:
		switch (OrderIndex) { case 0: return FVector3f::XAxisVector; case 1: return FVector3f::YAxisVector; case 2: return FVector3f::ZAxisVector; default: checkNoEntry(); return FVector3f::ZeroVector; }
		case EAnimDatabaseRotationOrder::XZY:
		switch (OrderIndex) { case 0: return FVector3f::XAxisVector; case 1: return FVector3f::ZAxisVector; case 2: return FVector3f::YAxisVector; default: checkNoEntry(); return FVector3f::ZeroVector; }
		case EAnimDatabaseRotationOrder::YXZ:
		switch (OrderIndex) { case 0: return FVector3f::YAxisVector; case 1: return FVector3f::XAxisVector; case 2: return FVector3f::ZAxisVector; default: checkNoEntry(); return FVector3f::ZeroVector; }
		case EAnimDatabaseRotationOrder::YZX:
		switch (OrderIndex) { case 0: return FVector3f::YAxisVector; case 1: return FVector3f::ZAxisVector; case 2: return FVector3f::XAxisVector; default: checkNoEntry(); return FVector3f::ZeroVector; }
		case EAnimDatabaseRotationOrder::ZXY:
		switch (OrderIndex) { case 0: return FVector3f::ZAxisVector; case 1: return FVector3f::XAxisVector; case 2: return FVector3f::YAxisVector; default: checkNoEntry(); return FVector3f::ZeroVector; }
		case EAnimDatabaseRotationOrder::ZYX:
		switch (OrderIndex) { case 0: return FVector3f::ZAxisVector; case 1: return FVector3f::YAxisVector; case 2: return FVector3f::XAxisVector; default: checkNoEntry(); return FVector3f::ZeroVector; }
		default: checkNoEntry(); return FVector3f::ZeroVector;
		}
	}

	const TCHAR* RotationOrderAxisName(const EAnimDatabaseRotationOrder RotationOrder, const uint8 OrderIndex)
	{
		switch (RotationOrder)
		{
		case EAnimDatabaseRotationOrder::XYZ:
		switch (OrderIndex) { case 0: return TEXT("X"); case 1: return TEXT("Y"); case 2: return TEXT("Z"); default: checkNoEntry(); return TEXT(""); }
		case EAnimDatabaseRotationOrder::XZY:
		switch (OrderIndex) { case 0: return TEXT("X"); case 1: return TEXT("Z"); case 2: return TEXT("Y"); default: checkNoEntry(); return TEXT(""); }
		case EAnimDatabaseRotationOrder::YXZ:
		switch (OrderIndex) { case 0: return TEXT("Y"); case 1: return TEXT("X"); case 2: return TEXT("Z"); default: checkNoEntry(); return TEXT(""); }
		case EAnimDatabaseRotationOrder::YZX:
		switch (OrderIndex) { case 0: return TEXT("Y"); case 1: return TEXT("Z"); case 2: return TEXT("X"); default: checkNoEntry(); return TEXT(""); }
		case EAnimDatabaseRotationOrder::ZXY:
		switch (OrderIndex) { case 0: return TEXT("Z"); case 1: return TEXT("X"); case 2: return TEXT("Y"); default: checkNoEntry(); return TEXT(""); }
		case EAnimDatabaseRotationOrder::ZYX:
		switch (OrderIndex) { case 0: return TEXT("Z"); case 1: return TEXT("Y"); case 2: return TEXT("X"); default: checkNoEntry(); return TEXT(""); }
		default: checkNoEntry(); return TEXT("");
		}
	}

	FQuat4f QuatFromEuler(const FVector3f E, const EAnimDatabaseRotationOrder RotationOrder)
	{
		const FQuat4f Q0 = FQuat4f::MakeFromRotationVector(E.X * RotationOrderAxis(RotationOrder, 0));
		const FQuat4f Q1 = FQuat4f::MakeFromRotationVector(E.Y * RotationOrderAxis(RotationOrder, 1));
		const FQuat4f Q2 = FQuat4f::MakeFromRotationVector(E.Z * RotationOrderAxis(RotationOrder, 2));
		return Q0 * Q1 * Q2;
	}

	FVector3f QuatToEuler(const FQuat4f Q, const EAnimDatabaseRotationOrder RotationOrder)
	{
		switch (RotationOrder)
		{
		case EAnimDatabaseRotationOrder::XYZ:
		{
			const float X = FMath::Atan2(2 * (Q.W * Q.X - Q.Y * Q.Z), 1 - 2 * (Q.X * Q.X + Q.Y * Q.Y));
			const float Y = FMath::Asin(FMath::Clamp(2 * (Q.X * Q.Z + Q.W * Q.Y), -1.0f, 1.0f));
			const float Z = FMath::Atan2(2 * (Q.W * Q.Z - Q.X * Q.Y), 1 - 2 * (Q.Y * Q.Y + Q.Z * Q.Z));
			return FVector3f(X, Y, Z);
		}
		case EAnimDatabaseRotationOrder::XZY:
		{
			const float X = FMath::Atan2(2 * (Q.W * Q.X + Q.Y * Q.Z), 1 - 2 * (Q.X * Q.X + Q.Z * Q.Z));
			const float Y = FMath::Atan2(2 * (Q.W * Q.Y + Q.X * Q.Z), 1 - 2 * (Q.Y * Q.Y + Q.Z * Q.Z));
			const float Z = FMath::Asin(FMath::Clamp(2 * (Q.W * Q.Z - Q.X * Q.Y), -1.0f, 1.0f));
			return FVector3f(X, Z, Y);
		}
		case EAnimDatabaseRotationOrder::YXZ:
		{
			const float X = FMath::Asin(FMath::Clamp(2 * (Q.W * Q.X - Q.Y * Q.Z), -1.0f, 1.0f));
			const float Y = FMath::Atan2(2 * (Q.X * Q.Z + Q.W * Q.Y), 1 - 2 * (Q.X * Q.X + Q.Y * Q.Y));
			const float Z = FMath::Atan2(2 * (Q.X * Q.Y + Q.W * Q.Z), 1 - 2 * (Q.X * Q.X + Q.Z * Q.Z));
			return FVector3f(Y, X, Z);
		}
		case EAnimDatabaseRotationOrder::YZX:
		{
			const float X = FMath::Atan2(2 * (Q.W * Q.X - Q.Y * Q.Z), 1 - 2 * (Q.X * Q.X + Q.Z * Q.Z));
			const float Y = FMath::Atan2(2 * (Q.W * Q.Y - Q.X * Q.Z), 1 - 2 * (Q.Y * Q.Y + Q.Z * Q.Z));
			const float Z = FMath::Asin(FMath::Clamp(2 * (Q.X * Q.Y + Q.W * Q.Z), -1.0f, 1.0f));
			return FVector3f(Y, Z, X);
		}
		case EAnimDatabaseRotationOrder::ZXY:
		{
			const float X = FMath::Asin(FMath::Clamp(2 * (Q.W * Q.X + Q.Y * Q.Z), -1.0f, 1.0f));
			const float Y = FMath::Atan2(2 * (Q.W * Q.Y - Q.X * Q.Z), 1 - 2 * (Q.X * Q.X + Q.Y * Q.Y));
			const float Z = FMath::Atan2(2 * (Q.W * Q.Z - Q.X * Q.Y), 1 - 2 * (Q.X * Q.X + Q.Z * Q.Z));
			return FVector3f(Z, X, Y);
		}
		case EAnimDatabaseRotationOrder::ZYX:
		{
			const float X = FMath::Atan2(2 * (Q.W * Q.X + Q.Y * Q.Z), 1 - 2 * (Q.X * Q.X + Q.Y * Q.Y));
			const float Y = FMath::Asin(FMath::Clamp(2 * (Q.W * Q.Y - Q.X * Q.Z), -1.0f, 1.0f));
			const float Z = FMath::Atan2(2 * (Q.W * Q.Z + Q.X * Q.Y), 1 - 2 * (Q.Y * Q.Y + Q.Z * Q.Z));
			return FVector3f(Z, Y, X);
		}
		default: checkNoEntry(); return FVector3f::ZeroVector;
		}
	}

	static inline FTransform ExtractRoot(const UAnimSequence* AnimSequence, const float Time, const FTransform& ReferenceTransform)
	{
		if (AnimSequence->bEnableRootMotion)
		{
			FAnimExtractContext Context;
			Context.bExtractRootMotion = false;
#if WITH_EDITOR
			Context.bExtractWithRootMotionProvider = false;
			Context.bIgnoreRootLock = false;
#endif
			Context.bLooping = false;
			Context.CurrentTime = Time;
			Context.DeltaTimeRecord = FDeltaTimeRecord(0.0f);

			return ReferenceTransform.Inverse() * AnimSequence->ExtractRootTrackTransform(Context, nullptr);
		}
		else
		{
			return FTransform::Identity;
		}
	}

	static inline FTransform MirrorRootTransform(const FTransform& Transform, const EAxis::Type MirrorAxis, const FQuat& ComponentSpaceRefRotation)
	{
		const FVector T = FAnimationRuntime::MirrorVector(Transform.GetTranslation(), MirrorAxis);

		FQuat Q = Transform.GetRotation();
		Q = FAnimationRuntime::MirrorQuat(Q, MirrorAxis);
		Q = Q * FAnimationRuntime::MirrorQuat(ComponentSpaceRefRotation, MirrorAxis).Inverse() * ComponentSpaceRefRotation;

		return FTransform(Q, T, Transform.GetScale3D());
	}

	static inline void ExtractRoot(
		const TLearningArrayView<1, FVector> OutLocations,
		const TLearningArrayView<1, FQuat4f> OutRotations,
		const TLearningArrayView<1, FVector3f> OutScales,
		const UAnimSequence* AnimSequence,
		const int32 FrameStart,
		const UMirrorDataTable* MirrorDataTable,
		const bool bMirrored)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(AnimDatabase::Private::ExtractRoot);

		const int32 FrameNum = OutLocations.Num();

		const float DeltaTime = 1.0f / FMath::Max(AnimSequence->GetSamplingFrameRate().AsDecimal(), UE_SMALL_NUMBER);
		const FTransform ReferenceTransform = AnimSequence->GetSkeleton() ? 
			AnimSequence->GetSkeleton()->GetReferenceSkeleton().GetRefBonePose()[0] :
			FTransform::Identity;

		if (bMirrored)
		{
			const EAxis::Type MirrorAxis = MirrorDataTable->MirrorAxis;
			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				const FTransform Transform = MirrorRootTransform(ExtractRoot(AnimSequence, (FrameStart + FrameIdx) * DeltaTime, ReferenceTransform), MirrorAxis, ReferenceTransform.GetRotation());
				OutLocations[FrameIdx] = Transform.GetLocation();
				OutRotations[FrameIdx] = (FQuat4f)Transform.GetRotation();
				OutScales[FrameIdx] = (FVector3f)Transform.GetScale3D();
			}
		}
		else
		{
			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				const FTransform Transform = ExtractRoot(AnimSequence, (FrameStart + FrameIdx) * DeltaTime, ReferenceTransform);
				OutLocations[FrameIdx] = Transform.GetLocation();
				OutRotations[FrameIdx] = (FQuat4f)Transform.GetRotation();
				OutScales[FrameIdx] = (FVector3f)Transform.GetScale3D();
			}
		}
	}

	static inline void ExtractRootLocations(
		const TLearningArrayView<1, FVector> OutLocations,
		const UAnimSequence* AnimSequence,
		const int32 FrameStart,
		const UMirrorDataTable* MirrorDataTable,
		const bool bMirrored)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(AnimDatabase::Private::ExtractRootLocations);

		const int32 FrameNum = OutLocations.Num();

		const float DeltaTime = 1.0f / FMath::Max(AnimSequence->GetSamplingFrameRate().AsDecimal(), UE_SMALL_NUMBER);
		const FTransform ReferenceTransform = AnimSequence->GetSkeleton() ?
			AnimSequence->GetSkeleton()->GetReferenceSkeleton().GetRefBonePose()[0] :
			FTransform::Identity;

		if (bMirrored)
		{
			const EAxis::Type MirrorAxis = MirrorDataTable->MirrorAxis;
			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				OutLocations[FrameIdx] = MirrorRootTransform(ExtractRoot(AnimSequence, (FrameStart + FrameIdx) * DeltaTime, ReferenceTransform), MirrorAxis, ReferenceTransform.GetRotation()).GetLocation();
			}
		}
		else
		{
			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				OutLocations[FrameIdx] = ExtractRoot(AnimSequence, (FrameStart + FrameIdx) * DeltaTime, ReferenceTransform).GetLocation();
			}
		}
	}

	static inline void ExtractRootRotations(
		const TLearningArrayView<1, FQuat4f> OutRotations,
		const UAnimSequence* AnimSequence,
		const int32 FrameStart,
		const UMirrorDataTable* MirrorDataTable,
		const bool bMirrored)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(AnimDatabase::Private::ExtractRootRotations);

		const int32 FrameNum = OutRotations.Num();

		const float DeltaTime = 1.0f / FMath::Max(AnimSequence->GetSamplingFrameRate().AsDecimal(), UE_SMALL_NUMBER);
		const FTransform ReferenceTransform = AnimSequence->GetSkeleton() ?
			AnimSequence->GetSkeleton()->GetReferenceSkeleton().GetRefBonePose()[0] :
			FTransform::Identity;

		if (bMirrored)
		{
			const EAxis::Type MirrorAxis = MirrorDataTable->MirrorAxis;
			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				OutRotations[FrameIdx] = (FQuat4f)MirrorRootTransform(ExtractRoot(AnimSequence, (FrameStart + FrameIdx) * DeltaTime, ReferenceTransform), MirrorAxis, ReferenceTransform.GetRotation()).GetRotation();
			}
		}
		else
		{
			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				OutRotations[FrameIdx] = (FQuat4f)ExtractRoot(AnimSequence, (FrameStart + FrameIdx) * DeltaTime, ReferenceTransform).GetRotation();
			}
		}
	}

	static inline void ExtractRootDirections(
		const TLearningArrayView<1, FVector3f> OutDirections,
		const UAnimSequence* AnimSequence,
		const int32 FrameStart,
		const UMirrorDataTable* MirrorDataTable,
		const bool bMirrored,
		const FVector3f ForwardVector)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(AnimDatabase::Private::ExtractRootDirections);

		const int32 FrameNum = OutDirections.Num();

		const float DeltaTime = 1.0f / FMath::Max(AnimSequence->GetSamplingFrameRate().AsDecimal(), UE_SMALL_NUMBER);
		const FTransform ReferenceTransform = AnimSequence->GetSkeleton() ?
			AnimSequence->GetSkeleton()->GetReferenceSkeleton().GetRefBonePose()[0] :
			FTransform::Identity;

		if (bMirrored)
		{
			const EAxis::Type MirrorAxis = MirrorDataTable->MirrorAxis;
			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				OutDirections[FrameIdx] = ((FVector3f)MirrorRootTransform(ExtractRoot(AnimSequence, (FrameStart + FrameIdx) * DeltaTime, ReferenceTransform), MirrorAxis, ReferenceTransform.GetRotation()).TransformVectorNoScale((FVector)ForwardVector)).GetSafeNormal();
			}
		}
		else
		{
			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				OutDirections[FrameIdx] = ((FVector3f)ExtractRoot(AnimSequence, (FrameStart + FrameIdx) * DeltaTime, ReferenceTransform).TransformVectorNoScale((FVector)ForwardVector)).GetSafeNormal();
			}
		}
	}

	static inline void ExtractBoneTransforms(
		const TLearningArrayView<2, FVector3f> OutBoneLocations,
		const TLearningArrayView<2, FQuat4f> OutBoneRotations,
		const TLearningArrayView<2, FVector3f> OutBoneScales,
		const UAnimSequence* AnimSequence,
		const int32 FrameStart,
		const UMirrorDataTable* MirrorDataTable,
		const bool bMirrored,
		const UE::Learning::FIndexSet BoneIndices)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(AnimDatabase::Private::ExtractBoneTransforms);

		check(OutBoneLocations.Num<1>() == BoneIndices.Num());
		check(OutBoneRotations.Num<1>() == BoneIndices.Num());
		check(OutBoneScales.Num<1>() == BoneIndices.Num());
		check(UE::AnimDatabase::Math::BoneIndicesAreSortedAndUnique(BoneIndices));
		check(AnimSequence && AnimSequence->GetSkeleton());

		const int32 FrameNum = OutBoneLocations.Num<0>();
		const int32 BoneNum = OutBoneLocations.Num<1>();
		const float SequenceFrameTime = 1.0f / FMath::Max(AnimSequence->GetSamplingFrameRate().AsDecimal(), UE_SMALL_NUMBER);

		TArray<FBoneIndexType, TInlineAllocator<256>> ContainerBoneIndices;
		ContainerBoneIndices.Empty(bMirrored ? BoneNum * 4 + 1 : BoneNum + 1);
		ContainerBoneIndices.AddUnique(0); // Seems important to always include root bone...

		// To mirror we need to include both the parent bone index, the mirror bone index, plus the parent of the mirror bone index
		if (bMirrored)
		{
			TCustomBoneIndexArray<FSkeletonPoseBoneIndex, FSkeletonPoseBoneIndex> MirrorBoneIndexes;
			MirrorDataTable->FillMirrorBoneIndexes(AnimSequence->GetSkeleton(), MirrorBoneIndexes);

			for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
			{
				// Add bone and ascendants
				ContainerBoneIndices.AddUnique(BoneIndices[BoneIdx]);

				int32 ParentBoneIndex = AnimSequence->GetSkeleton()->GetReferenceSkeleton().GetParentIndex(BoneIndices[BoneIdx]);
				while (ParentBoneIndex != INDEX_NONE)
				{
					ContainerBoneIndices.AddUnique(ParentBoneIndex);
					ParentBoneIndex = AnimSequence->GetSkeleton()->GetReferenceSkeleton().GetParentIndex(ParentBoneIndex);
				}

				// Add mirror bone and ascendants
				const int32 MirrorBoneIndex = MirrorBoneIndexes[BoneIndices[BoneIdx]].GetInt();
				if (MirrorBoneIndex != INDEX_NONE)
				{
					ContainerBoneIndices.AddUnique(MirrorBoneIndex);

					int32 MirrorParentBoneIndex = AnimSequence->GetSkeleton()->GetReferenceSkeleton().GetParentIndex(MirrorBoneIndex);
					while (MirrorParentBoneIndex != INDEX_NONE)
					{
						ContainerBoneIndices.AddUnique(MirrorParentBoneIndex);
						MirrorParentBoneIndex = AnimSequence->GetSkeleton()->GetReferenceSkeleton().GetParentIndex(MirrorParentBoneIndex);
					}
				}
			}
		}
		else
		{
			for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
			{
				ContainerBoneIndices.AddUnique(BoneIndices[BoneIdx]);
			}
		}

		// Make sure bone indices are sorted

		ContainerBoneIndices.Sort();

		// Since we have a different set of bones in the container than what we asked for (and in a different order) 
		// we need to find the mapping from container to desired bone indices.

		TArray<int32, TInlineAllocator<256>> DesiredToContainerBoneIndices;
		DesiredToContainerBoneIndices.SetNumUninitialized(BoneNum);

		for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
		{
			DesiredToContainerBoneIndices[BoneIdx] = ContainerBoneIndices.Find(BoneIndices[BoneIdx]);
			check(DesiredToContainerBoneIndices[BoneIdx] != INDEX_NONE);
		}

		// Make Sampler Object

		FMemMark Mark(FMemStack::Get());

		FBoneContainer BoneContainer;
		BoneContainer.InitializeTo(ContainerBoneIndices, UE::Anim::FCurveFilterSettings(UE::Anim::ECurveFilterMode::DisallowAll), *AnimSequence->GetSkeleton());
		BoneContainer.SetUseRAWData(false);

		FCompactPose Pose;
		FBlendedCurve Curves;
		UE::Anim::FStackAttributeContainer Attributes;
		FAnimationPoseData PoseData(Pose, Curves, Attributes);

		Pose.SetBoneContainer(&BoneContainer);
		Curves.InitFrom(BoneContainer);

		TCustomBoneIndexArray<FCompactPoseBoneIndex, FCompactPoseBoneIndex> CompactPoseMirrorBones;
		TCustomBoneIndexArray<FQuat, FCompactPoseBoneIndex> ComponentSpaceRefRotations;

		if (bMirrored)
		{
			MirrorDataTable->FillCompactPoseAndComponentRefRotations(
				BoneContainer,
				CompactPoseMirrorBones,
				ComponentSpaceRefRotations);
		}

		for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
		{
			FDeltaTimeRecord DeltaTimeRecord;
			DeltaTimeRecord.Set(((FrameStart + FrameIdx) * SequenceFrameTime) - SequenceFrameTime, SequenceFrameTime);

			FAnimExtractContext ExtractionCtx((double)((FrameStart + FrameIdx) * SequenceFrameTime), false, DeltaTimeRecord, false);
			ExtractionCtx.bExtractRootMotion = false;
#if WITH_EDITOR
			ExtractionCtx.bExtractWithRootMotionProvider = false;
#endif
			ExtractionCtx.bLooping = false;

			AnimSequence->GetAnimationPose(PoseData, ExtractionCtx);

			if (bMirrored)
			{
				FAnimationRuntime::MirrorPose(Pose, MirrorDataTable->MirrorAxis, CompactPoseMirrorBones, ComponentSpaceRefRotations);
			}

			for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
			{
				OutBoneLocations[FrameIdx][BoneIdx] = (FVector3f)Pose.GetBones()[DesiredToContainerBoneIndices[BoneIdx]].GetLocation();
				OutBoneRotations[FrameIdx][BoneIdx] = (FQuat4f)Pose.GetBones()[DesiredToContainerBoneIndices[BoneIdx]].GetRotation();
				OutBoneScales[FrameIdx][BoneIdx] = (FVector3f)Pose.GetBones()[DesiredToContainerBoneIndices[BoneIdx]].GetScale3D();
			}
		}
	}


	static inline void ExtractCurveActiveData(
		const TLearningArrayView<2, bool> OutCurveActive,
		const UAnimSequence* AnimSequence,
		const int32 FrameStart,
		const UMirrorDataTable* MirrorDataTable,
		const bool bMirrored,
		const TArrayView<const FName> CurveNames)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(AnimDatabase::Private::ExtractCurveActiveData);

		check(OutCurveActive.Num<1>() == CurveNames.Num());

		const int32 FrameNum = OutCurveActive.Num<0>();
		const float SequenceFrameTime = 1.0f / FMath::Max(AnimSequence->GetSamplingFrameRate().AsDecimal(), UE_SMALL_NUMBER);

		TArray<FName> CurveNameArray;
		CurveNameArray = CurveNames;

		FMemMark Mark(FMemStack::Get());

		FBoneContainer BoneContainer;
		BoneContainer.InitializeTo({ 0 }, UE::Anim::FCurveFilterSettings(UE::Anim::ECurveFilterMode::AllowOnlyFiltered, &CurveNameArray), *AnimSequence->GetSkeleton());
		BoneContainer.SetUseRAWData(false);

		FCompactPose Pose;
		FBlendedCurve Curves;
		UE::Anim::FStackAttributeContainer Attributes;
		FAnimationPoseData PoseData(Pose, Curves, Attributes);

		Pose.SetBoneContainer(&BoneContainer);
		Curves.InitFrom(BoneContainer);

		for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
		{
			FDeltaTimeRecord DeltaTimeRecord;
			DeltaTimeRecord.Set(((FrameStart + FrameIdx) * SequenceFrameTime) - SequenceFrameTime, SequenceFrameTime);

			FAnimExtractContext ExtractionCtx((double)((FrameStart + FrameIdx) * SequenceFrameTime), false, DeltaTimeRecord, false);
			ExtractionCtx.bExtractRootMotion = false;
#if WITH_EDITOR
			ExtractionCtx.bExtractWithRootMotionProvider = false;
#endif
			ExtractionCtx.bLooping = false;

			AnimSequence->GetAnimationPose(PoseData, ExtractionCtx);

			if (bMirrored)
			{
				FAnimationRuntime::MirrorCurves(PoseData.GetCurve(), *MirrorDataTable);
			}

			UE::Learning::Array::Zero(OutCurveActive[FrameIdx]);

			PoseData.GetCurve().ForEachElement([&OutCurveActive, &CurveNames, FrameIdx](const UE::Anim::FCurveElement& Element) {
				const int32 CurveIdx = CurveNames.Find(Element.Name);
				if (CurveIdx != INDEX_NONE)
				{
					OutCurveActive[FrameIdx][CurveIdx] = true;
				}
				});
		}
	}

	static inline void ExtractCurveData(
		const TLearningArrayView<2, float> OutCurveValues,
		const TLearningArrayView<2, bool> OutCurveActive,
		const UAnimSequence* AnimSequence,
		const int32 FrameStart,
		const UMirrorDataTable* MirrorDataTable,
		const bool bMirrored,
		const TArrayView<const FName> CurveNames)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(AnimDatabase::Private::ExtractCurveData);

		check(OutCurveValues.Num<1>() == CurveNames.Num());
		check(OutCurveActive.Num<0>() == OutCurveValues.Num<0>());
		check(OutCurveActive.Num<1>() == OutCurveValues.Num<1>());

		const int32 FrameNum = OutCurveValues.Num<0>();
		const float SequenceFrameTime = 1.0f / FMath::Max(AnimSequence->GetSamplingFrameRate().AsDecimal(), UE_SMALL_NUMBER);

		TArray<FName> CurveNameArray;
		CurveNameArray = CurveNames;

		FMemMark Mark(FMemStack::Get());

		FBoneContainer BoneContainer;
		BoneContainer.InitializeTo({ 0 }, UE::Anim::FCurveFilterSettings(UE::Anim::ECurveFilterMode::None, nullptr), *AnimSequence->GetSkeleton());
		BoneContainer.SetUseRAWData(false);

		FCompactPose Pose;
		FBlendedCurve Curves;
		UE::Anim::FStackAttributeContainer Attributes;
		FAnimationPoseData PoseData(Pose, Curves, Attributes);

		Pose.SetBoneContainer(&BoneContainer);

		UE::Anim::FCurveFilter CurveFilter;
		CurveFilter.SetFilterMode(UE::Anim::ECurveFilterMode::AllowOnlyFiltered);
		for (const FName CurveName : CurveNames)
		{
			CurveFilter.Add(CurveName, UE::Anim::ECurveFilterFlags::Filtered);
		}

		Curves.SetFilter(&CurveFilter);

		for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
		{
			// Extract Pose Data

			FDeltaTimeRecord DeltaTimeRecord;
			DeltaTimeRecord.Set(((FrameStart + FrameIdx) * SequenceFrameTime) - SequenceFrameTime, SequenceFrameTime);

			FAnimExtractContext ExtractionCtx((double)((FrameStart + FrameIdx) * SequenceFrameTime), false, DeltaTimeRecord, false);
			ExtractionCtx.bExtractRootMotion = false;
#if WITH_EDITOR
			ExtractionCtx.bExtractWithRootMotionProvider = false;
#endif
			ExtractionCtx.bLooping = false;

			AnimSequence->GetAnimationPose(PoseData, ExtractionCtx);

			if (bMirrored)
			{
				FAnimationRuntime::MirrorCurves(PoseData.GetCurve(), *MirrorDataTable);
			}

			UE::Learning::Array::Zero(OutCurveValues[FrameIdx]);
			UE::Learning::Array::Zero(OutCurveActive[FrameIdx]);

			PoseData.GetCurve().ForEachElement([&OutCurveValues, &OutCurveActive, &CurveNames, FrameIdx](const UE::Anim::FCurveElement& Element) {
				const int32 CurveIdx = CurveNames.Find(Element.Name);
				if (CurveIdx != INDEX_NONE)
				{
					OutCurveValues[FrameIdx][CurveIdx] = Element.Value;
					OutCurveActive[FrameIdx][CurveIdx] = true;
				}
				});
		}
	}

	static inline void ComputeExtractionRanges(
		int32& OutSequenceStartFrame,
		int32& OutSequenceStopFrame,
		const float DatabaseToSequence,
		const int32 DatabaseStartFrame,
		const int32 DatabaseStopFrame,
		const int32 SequenceFrameNum,
		const int32 WindowSize = 2)
	{
		OutSequenceStartFrame = FMath::Clamp(FMath::FloorToInt(DatabaseStartFrame * DatabaseToSequence) - WindowSize, 0, SequenceFrameNum - 1);
		OutSequenceStopFrame = FMath::Clamp(FMath::CeilToInt(DatabaseStopFrame * DatabaseToSequence) + WindowSize, 1, SequenceFrameNum);
		check(SequenceFrameNum > 0 && OutSequenceStopFrame > OutSequenceStartFrame);
	}

	static inline void ComputeExtractionRangesForFrameTime(
		int32& OutSequenceStartFrame,
		int32& OutSequenceStopFrame,
		const float DatabaseToSequence,
		const float DatabaseFrameTime,
		const int32 SequenceFrameNum,
		const int32 WindowSize = 2)
	{
		OutSequenceStartFrame = FMath::Clamp(FMath::FloorToInt(DatabaseFrameTime * DatabaseToSequence) - WindowSize, 0, SequenceFrameNum - 1);
		OutSequenceStopFrame = FMath::Clamp(FMath::CeilToInt(DatabaseFrameTime * DatabaseToSequence) + WindowSize, 1, SequenceFrameNum);
		check(SequenceFrameNum > 0 && OutSequenceStopFrame > OutSequenceStartFrame);
	}

	static inline void SampleSequenceTransforms(
		const TLearningArrayView<1, FTransform> OutDatabaseTransforms,
		const TLearningArrayView<1, const FVector> InSequenceLocations,
		const TLearningArrayView<1, const FQuat4f> InSequenceRotations,
		const TLearningArrayView<1, const FVector3f> InSequenceScales,
		const int32 SequenceStartFrame,
		const int32 SequenceStopFrame,
		const float DatabaseToSequence,
		const int32 DatabaseStartFrame,
		const int32 DatabaseStopFrame,
		const float SequenceDeltaTime,
		const EAnimDatabaseSampler Sampler)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(AnimDatabase::Private::SampleSequenceTransforms);

		check(OutDatabaseTransforms.Num() == (DatabaseStopFrame - DatabaseStartFrame));
		check(InSequenceLocations.Num() == (SequenceStopFrame - SequenceStartFrame));
		check(InSequenceRotations.Num() == (SequenceStopFrame - SequenceStartFrame));
		check(InSequenceScales.Num() == (SequenceStopFrame - SequenceStartFrame));

		if (Sampler == EAnimDatabaseSampler::Nearest)
		{
			for (int32 FrameIdx = 0; FrameIdx < DatabaseStopFrame - DatabaseStartFrame; FrameIdx++)
			{
				const int64 Nearest = Math::ComputeNearestSampleFrame(DatabaseToSequence * (DatabaseStartFrame + FrameIdx) - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

				FVector Location = FVector::ZeroVector; FQuat4f Rotation = FQuat4f::Identity; FVector3f Scale = FVector3f::OneVector;
				Math::TransformSampleNearest(Location, Rotation, Scale, InSequenceLocations, InSequenceRotations, InSequenceScales, Nearest);

				OutDatabaseTransforms[FrameIdx] = FTransform(
					((FQuat)Rotation).GetNormalized(),
					Location,
					(FVector)Scale);
			}
		}
		else if (Sampler == EAnimDatabaseSampler::Linear)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE;
			float Alpha = 0.0f;

			for (int32 FrameIdx = 0; FrameIdx < DatabaseStopFrame - DatabaseStartFrame; FrameIdx++)
			{
				Math::ComputeLinearSampleFramesAndAlpha(
					I0, I1, Alpha,
					DatabaseToSequence * (DatabaseStartFrame + FrameIdx) - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

				FVector Location = FVector::ZeroVector; FQuat4f Rotation = FQuat4f::Identity; FVector3f Scale = FVector3f::OneVector;
				Math::TransformSampleLinear(Location, Rotation, Scale, InSequenceLocations, InSequenceRotations, InSequenceScales, I0, I1, Alpha);

				OutDatabaseTransforms[FrameIdx] = FTransform(
					((FQuat)Rotation).GetNormalized(),
					Location,
					(FVector)Scale);
			}
		}
		else if (Sampler == EAnimDatabaseSampler::Cubic)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE, I2 = INDEX_NONE, I3 = INDEX_NONE;
			float Alpha = 0.0f;

			for (int32 FrameIdx = 0; FrameIdx < DatabaseStopFrame - DatabaseStartFrame; FrameIdx++)
			{
				Math::ComputeCubicSampleFramesAndAlpha(
					I0, I1, I2, I3, Alpha,
					DatabaseToSequence * (DatabaseStartFrame + FrameIdx) - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

				FVector Location = FVector::ZeroVector; FQuat4f Rotation = FQuat4f::Identity; FVector3f Scale = FVector3f::OneVector;
				Math::TransformSampleCubic(Location, Rotation, Scale, InSequenceLocations, InSequenceRotations, InSequenceScales, I0, I1, I2, I3, Alpha);

				OutDatabaseTransforms[FrameIdx] = FTransform(
					((FQuat)Rotation).GetNormalized(),
					Location,
					(FVector)Scale);
			}
		}
		else if (Sampler == EAnimDatabaseSampler::CubicMono)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE, I2 = INDEX_NONE, I3 = INDEX_NONE;
			float Alpha = 0.0f;

			for (int32 FrameIdx = 0; FrameIdx < DatabaseStopFrame - DatabaseStartFrame; FrameIdx++)
			{
				Math::ComputeCubicSampleFramesAndAlpha(
					I0, I1, I2, I3, Alpha,
					DatabaseToSequence * (DatabaseStartFrame + FrameIdx) - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

				FVector Location = FVector::ZeroVector; FQuat4f Rotation = FQuat4f::Identity; FVector3f Scale = FVector3f::OneVector;
				Math::TransformSampleCubicMono(Location, Rotation, Scale, InSequenceLocations, InSequenceRotations, InSequenceScales, I0, I1, I2, I3, Alpha);

				OutDatabaseTransforms[FrameIdx] = FTransform(
					((FQuat)Rotation).GetNormalized(),
					Location,
					(FVector)Scale);
			}
		}
		else
		{
			checkNoEntry();
		}
	}

	static inline void SampleSequenceTransformsVelocities(
		const TLearningArrayView<1, FVector> OutDatabaseLocations,
		const TLearningArrayView<1, FQuat4f> OutDatabaseRotations,
		const TLearningArrayView<1, FVector3f> OutDatabaseScales,
		const TLearningArrayView<1, FVector3f> OutDatabaseLinearVelocities,
		const TLearningArrayView<1, FVector3f> OutDatabaseAngularVelocities,
		const TLearningArrayView<1, FVector3f> OutDatabaseScalarVelocities,
		const TLearningArrayView<1, const FVector> InSequenceLocations,
		const TLearningArrayView<1, const FQuat4f> InSequenceRotations,
		const TLearningArrayView<1, const FVector3f> InSequenceScales,
		const int32 SequenceStartFrame,
		const int32 SequenceStopFrame,
		const float DatabaseToSequence,
		const int32 DatabaseStartFrame,
		const int32 DatabaseStopFrame,
		const float SequenceDeltaTime,
		const EAnimDatabaseSampler Sampler)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(AnimDatabase::Private::SampleSequenceTransformsVelocities);

		check(OutDatabaseLocations.Num() == (DatabaseStopFrame - DatabaseStartFrame));
		check(OutDatabaseRotations.Num() == (DatabaseStopFrame - DatabaseStartFrame));
		check(OutDatabaseScales.Num() == (DatabaseStopFrame - DatabaseStartFrame));
		check(OutDatabaseLinearVelocities.Num() == (DatabaseStopFrame - DatabaseStartFrame));
		check(OutDatabaseAngularVelocities.Num() == (DatabaseStopFrame - DatabaseStartFrame));
		check(OutDatabaseScalarVelocities.Num() == (DatabaseStopFrame - DatabaseStartFrame));
		check(InSequenceLocations.Num() == (SequenceStopFrame - SequenceStartFrame));
		check(InSequenceRotations.Num() == (SequenceStopFrame - SequenceStartFrame));
		check(InSequenceScales.Num() == (SequenceStopFrame - SequenceStartFrame));

		if (Sampler == EAnimDatabaseSampler::Nearest)
		{
			for (int32 FrameIdx = 0; FrameIdx < DatabaseStopFrame - DatabaseStartFrame; FrameIdx++)
			{
				const int64 Nearest = Math::ComputeNearestSampleFrame(DatabaseToSequence * (DatabaseStartFrame + FrameIdx) - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

				Math::TransformSampleNearest(
					OutDatabaseLocations[FrameIdx],
					OutDatabaseRotations[FrameIdx],
					OutDatabaseScales[FrameIdx],
					OutDatabaseLinearVelocities[FrameIdx],
					OutDatabaseAngularVelocities[FrameIdx],
					OutDatabaseScalarVelocities[FrameIdx],
					InSequenceLocations, InSequenceRotations, InSequenceScales, Nearest);
			}
		}
		else if (Sampler == EAnimDatabaseSampler::Linear)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE;
			float Alpha = 0.0f;

			for (int32 FrameIdx = 0; FrameIdx < DatabaseStopFrame - DatabaseStartFrame; FrameIdx++)
			{
				Math::ComputeLinearSampleFramesAndAlpha(
					I0, I1, Alpha,
					DatabaseToSequence * (DatabaseStartFrame + FrameIdx) - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

				Math::TransformSampleLinear(
					OutDatabaseLocations[FrameIdx],
					OutDatabaseRotations[FrameIdx],
					OutDatabaseScales[FrameIdx],
					OutDatabaseLinearVelocities[FrameIdx],
					OutDatabaseAngularVelocities[FrameIdx],
					OutDatabaseScalarVelocities[FrameIdx],
					InSequenceLocations, InSequenceRotations, InSequenceScales, I0, I1, Alpha, SequenceDeltaTime);
			}
		}
		else if (Sampler == EAnimDatabaseSampler::Cubic)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE, I2 = INDEX_NONE, I3 = INDEX_NONE;
			float Alpha = 0.0f;

			for (int32 FrameIdx = 0; FrameIdx < DatabaseStopFrame - DatabaseStartFrame; FrameIdx++)
			{
				Math::ComputeCubicSampleFramesAndAlpha(
					I0, I1, I2, I3, Alpha,
					DatabaseToSequence * (DatabaseStartFrame + FrameIdx) - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

				Math::TransformSampleCubic(
					OutDatabaseLocations[FrameIdx],
					OutDatabaseRotations[FrameIdx],
					OutDatabaseScales[FrameIdx],
					OutDatabaseLinearVelocities[FrameIdx],
					OutDatabaseAngularVelocities[FrameIdx],
					OutDatabaseScalarVelocities[FrameIdx],
					InSequenceLocations, InSequenceRotations, InSequenceScales, I0, I1, I2, I3, Alpha, SequenceDeltaTime);
			}
		}
		else if (Sampler == EAnimDatabaseSampler::CubicMono)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE, I2 = INDEX_NONE, I3 = INDEX_NONE;
			float Alpha = 0.0f;

			for (int32 FrameIdx = 0; FrameIdx < DatabaseStopFrame - DatabaseStartFrame; FrameIdx++)
			{
				Math::ComputeCubicSampleFramesAndAlpha(
					I0, I1, I2, I3, Alpha,
					DatabaseToSequence * (DatabaseStartFrame + FrameIdx) - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

				Math::TransformSampleCubicMono(
					OutDatabaseLocations[FrameIdx],
					OutDatabaseRotations[FrameIdx],
					OutDatabaseScales[FrameIdx],
					OutDatabaseLinearVelocities[FrameIdx],
					OutDatabaseAngularVelocities[FrameIdx],
					OutDatabaseScalarVelocities[FrameIdx],
					InSequenceLocations, InSequenceRotations, InSequenceScales, I0, I1, I2, I3, Alpha, SequenceDeltaTime);
			}
		}
		else
		{
			checkNoEntry();
		}
	}

	static inline void SampleSequenceTransformsVelocities(
		const TLearningArrayView<2, FVector3f> OutDatabaseLocations,
		const TLearningArrayView<2, FQuat4f> OutDatabaseRotations,
		const TLearningArrayView<2, FVector3f> OutDatabaseScales,
		const TLearningArrayView<2, FVector3f> OutDatabaseLinearVelocities,
		const TLearningArrayView<2, FVector3f> OutDatabaseAngularVelocities,
		const TLearningArrayView<2, FVector3f> OutDatabaseScalarVelocities,
		const TLearningArrayView<2, const FVector3f> InSequenceLocations,
		const TLearningArrayView<2, const FQuat4f> InSequenceRotations,
		const TLearningArrayView<2, const FVector3f> InSequenceScales,
		const int32 SequenceStartFrame,
		const int32 SequenceStopFrame,
		const float DatabaseToSequence,
		const int32 DatabaseStartFrame,
		const int32 DatabaseStopFrame,
		const float SequenceDeltaTime,
		const EAnimDatabaseSampler Sampler)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(AnimDatabase::Private::SampleSequenceTransformsVelocities);

		check(OutDatabaseLocations.Num<0>() == (DatabaseStopFrame - DatabaseStartFrame));
		check(OutDatabaseRotations.Num<0>() == (DatabaseStopFrame - DatabaseStartFrame));
		check(OutDatabaseScales.Num<0>() == (DatabaseStopFrame - DatabaseStartFrame));
		check(OutDatabaseLinearVelocities.Num<0>() == (DatabaseStopFrame - DatabaseStartFrame));
		check(OutDatabaseAngularVelocities.Num<0>() == (DatabaseStopFrame - DatabaseStartFrame));
		check(OutDatabaseScalarVelocities.Num<0>() == (DatabaseStopFrame - DatabaseStartFrame));
		check(InSequenceLocations.Num<0>() == (SequenceStopFrame - SequenceStartFrame));
		check(InSequenceRotations.Num<0>() == (SequenceStopFrame - SequenceStartFrame));
		check(InSequenceScales.Num<0>() == (SequenceStopFrame - SequenceStartFrame));

		if (Sampler == EAnimDatabaseSampler::Nearest)
		{
			for (int32 FrameIdx = 0; FrameIdx < DatabaseStopFrame - DatabaseStartFrame; FrameIdx++)
			{
				const int64 Nearest = Math::ComputeNearestSampleFrame(DatabaseToSequence * (DatabaseStartFrame + FrameIdx) - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

				Math::TransformSampleNearest(
					OutDatabaseLocations[FrameIdx],
					OutDatabaseRotations[FrameIdx],
					OutDatabaseScales[FrameIdx],
					OutDatabaseLinearVelocities[FrameIdx],
					OutDatabaseAngularVelocities[FrameIdx],
					OutDatabaseScalarVelocities[FrameIdx],
					InSequenceLocations, InSequenceRotations, InSequenceScales, Nearest);
			}
		}
		else if (Sampler == EAnimDatabaseSampler::Linear)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE;
			float Alpha = 0.0f;

			for (int32 FrameIdx = 0; FrameIdx < DatabaseStopFrame - DatabaseStartFrame; FrameIdx++)
			{
				Math::ComputeLinearSampleFramesAndAlpha(
					I0, I1, Alpha,
					DatabaseToSequence * (DatabaseStartFrame + FrameIdx) - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

				Math::TransformSampleLinear(
					OutDatabaseLocations[FrameIdx],
					OutDatabaseRotations[FrameIdx],
					OutDatabaseScales[FrameIdx],
					OutDatabaseLinearVelocities[FrameIdx],
					OutDatabaseAngularVelocities[FrameIdx],
					OutDatabaseScalarVelocities[FrameIdx],
					InSequenceLocations, InSequenceRotations, InSequenceScales, I0, I1, Alpha, SequenceDeltaTime);
			}
		}
		else if (Sampler == EAnimDatabaseSampler::Cubic)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE, I2 = INDEX_NONE, I3 = INDEX_NONE;
			float Alpha = 0.0f;

			for (int32 FrameIdx = 0; FrameIdx < DatabaseStopFrame - DatabaseStartFrame; FrameIdx++)
			{
				Math::ComputeCubicSampleFramesAndAlpha(
					I0, I1, I2, I3, Alpha,
					DatabaseToSequence * (DatabaseStartFrame + FrameIdx) - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

				Math::TransformSampleCubic(
					OutDatabaseLocations[FrameIdx],
					OutDatabaseRotations[FrameIdx],
					OutDatabaseScales[FrameIdx],
					OutDatabaseLinearVelocities[FrameIdx],
					OutDatabaseAngularVelocities[FrameIdx],
					OutDatabaseScalarVelocities[FrameIdx],
					InSequenceLocations, InSequenceRotations, InSequenceScales, I0, I1, I2, I3, Alpha, SequenceDeltaTime);
			}
		}
		else if (Sampler == EAnimDatabaseSampler::CubicMono)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE, I2 = INDEX_NONE, I3 = INDEX_NONE;
			float Alpha = 0.0f;

			for (int32 FrameIdx = 0; FrameIdx < DatabaseStopFrame - DatabaseStartFrame; FrameIdx++)
			{
				Math::ComputeCubicSampleFramesAndAlpha(
					I0, I1, I2, I3, Alpha,
					DatabaseToSequence * (DatabaseStartFrame + FrameIdx) - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

				Math::TransformSampleCubicMono(
					OutDatabaseLocations[FrameIdx],
					OutDatabaseRotations[FrameIdx],
					OutDatabaseScales[FrameIdx],
					OutDatabaseLinearVelocities[FrameIdx],
					OutDatabaseAngularVelocities[FrameIdx],
					OutDatabaseScalarVelocities[FrameIdx],
					InSequenceLocations, InSequenceRotations, InSequenceScales, I0, I1, I2, I3, Alpha, SequenceDeltaTime);
			}
		}
		else
		{
			checkNoEntry();
		}
	}

	static inline void SampleSequenceLocations(
		const TLearningArrayView<1, FVector> OutDatabaseLocations,
		const TLearningArrayView<1, const FVector> InSequenceLocations,
		const int32 SequenceStartFrame,
		const int32 SequenceStopFrame,
		const float DatabaseToSequence,
		const int32 DatabaseStartFrame,
		const int32 DatabaseStopFrame,
		const float SequenceDeltaTime,
		const EAnimDatabaseSampler Sampler)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(AnimDatabase::Private::SampleSequenceLocations);

		check(OutDatabaseLocations.Num() == (DatabaseStopFrame - DatabaseStartFrame));
		check(InSequenceLocations.Num() == (SequenceStopFrame - SequenceStartFrame));

		if (Sampler == EAnimDatabaseSampler::Nearest)
		{
			for (int32 FrameIdx = 0; FrameIdx < DatabaseStopFrame - DatabaseStartFrame; FrameIdx++)
			{
				const int64 Nearest = Math::ComputeNearestSampleFrame(DatabaseToSequence * (DatabaseStartFrame + FrameIdx) - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

				Math::LocationSampleNearest(OutDatabaseLocations[FrameIdx], InSequenceLocations, Nearest);
			}
		}
		else if (Sampler == EAnimDatabaseSampler::Linear)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE;
			float Alpha = 0.0f;

			for (int32 FrameIdx = 0; FrameIdx < DatabaseStopFrame - DatabaseStartFrame; FrameIdx++)
			{
				Math::ComputeLinearSampleFramesAndAlpha(
					I0, I1, Alpha,
					DatabaseToSequence * (DatabaseStartFrame + FrameIdx) - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

				Math::LocationSampleLinear(OutDatabaseLocations[FrameIdx], InSequenceLocations, I0, I1, Alpha);
			}
		}
		else if (Sampler == EAnimDatabaseSampler::Cubic)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE, I2 = INDEX_NONE, I3 = INDEX_NONE;
			float Alpha = 0.0f;

			for (int32 FrameIdx = 0; FrameIdx < DatabaseStopFrame - DatabaseStartFrame; FrameIdx++)
			{
				Math::ComputeCubicSampleFramesAndAlpha(
					I0, I1, I2, I3, Alpha,
					DatabaseToSequence * (DatabaseStartFrame + FrameIdx) - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

				Math::LocationSampleCubic(OutDatabaseLocations[FrameIdx], InSequenceLocations, I0, I1, I2, I3, Alpha);
			}
		}
		else if (Sampler == EAnimDatabaseSampler::CubicMono)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE, I2 = INDEX_NONE, I3 = INDEX_NONE;
			float Alpha = 0.0f;

			for (int32 FrameIdx = 0; FrameIdx < DatabaseStopFrame - DatabaseStartFrame; FrameIdx++)
			{
				Math::ComputeCubicSampleFramesAndAlpha(
					I0, I1, I2, I3, Alpha,
					DatabaseToSequence * (DatabaseStartFrame + FrameIdx) - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

				Math::LocationSampleCubicMono(OutDatabaseLocations[FrameIdx], InSequenceLocations, I0, I1, I2, I3, Alpha);
			}
		}
		else
		{
			checkNoEntry();
		}
	}

	static inline void SampleSequenceRotations(
		const TLearningArrayView<1, FQuat4f> OutDatabaseRotations,
		const TLearningArrayView<1, const FQuat4f> InSequenceRotations,
		const int32 SequenceStartFrame,
		const int32 SequenceStopFrame,
		const float DatabaseToSequence,
		const int32 DatabaseStartFrame,
		const int32 DatabaseStopFrame,
		const float SequenceDeltaTime,
		const EAnimDatabaseSampler Sampler)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(AnimDatabase::Private::SampleSequenceRotations);

		check(OutDatabaseRotations.Num() == (DatabaseStopFrame - DatabaseStartFrame));
		check(InSequenceRotations.Num() == (SequenceStopFrame - SequenceStartFrame));

		if (Sampler == EAnimDatabaseSampler::Nearest)
		{
			for (int32 FrameIdx = 0; FrameIdx < DatabaseStopFrame - DatabaseStartFrame; FrameIdx++)
			{
				const int64 Nearest = Math::ComputeNearestSampleFrame(DatabaseToSequence * (DatabaseStartFrame + FrameIdx) - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

				Math::RotationSampleNearest(OutDatabaseRotations[FrameIdx], InSequenceRotations, Nearest);
			}
		}
		else if (Sampler == EAnimDatabaseSampler::Linear)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE;
			float Alpha = 0.0f;

			for (int32 FrameIdx = 0; FrameIdx < DatabaseStopFrame - DatabaseStartFrame; FrameIdx++)
			{
				Math::ComputeLinearSampleFramesAndAlpha(
					I0, I1, Alpha,
					DatabaseToSequence * (DatabaseStartFrame + FrameIdx) - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

				Math::RotationSampleLinear(OutDatabaseRotations[FrameIdx], InSequenceRotations, I0, I1, Alpha);
			}
		}
		else if (Sampler == EAnimDatabaseSampler::Cubic)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE, I2 = INDEX_NONE, I3 = INDEX_NONE;
			float Alpha = 0.0f;

			for (int32 FrameIdx = 0; FrameIdx < DatabaseStopFrame - DatabaseStartFrame; FrameIdx++)
			{
				Math::ComputeCubicSampleFramesAndAlpha(
					I0, I1, I2, I3, Alpha,
					DatabaseToSequence * (DatabaseStartFrame + FrameIdx) - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

				Math::RotationSampleCubic(OutDatabaseRotations[FrameIdx], InSequenceRotations, I0, I1, I2, I3, Alpha);
			}
		}
		else if (Sampler == EAnimDatabaseSampler::CubicMono)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE, I2 = INDEX_NONE, I3 = INDEX_NONE;
			float Alpha = 0.0f;

			for (int32 FrameIdx = 0; FrameIdx < DatabaseStopFrame - DatabaseStartFrame; FrameIdx++)
			{
				Math::ComputeCubicSampleFramesAndAlpha(
					I0, I1, I2, I3, Alpha,
					DatabaseToSequence * (DatabaseStartFrame + FrameIdx) - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

				Math::RotationSampleCubicMono(OutDatabaseRotations[FrameIdx], InSequenceRotations, I0, I1, I2, I3, Alpha);
			}
		}
		else
		{
			checkNoEntry();
		}
	}

	static inline void SampleSequenceDirectionsFromRotations(
		const TLearningArrayView<1, FVector3f> OutDatabaseDirections,
		const TLearningArrayView<1, const FQuat4f> InSequenceRotations,
		const int32 SequenceStartFrame,
		const int32 SequenceStopFrame,
		const float DatabaseToSequence,
		const int32 DatabaseStartFrame,
		const int32 DatabaseStopFrame,
		const float SequenceDeltaTime,
		const FVector3f ForwardVector,
		const EAnimDatabaseSampler Sampler)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(AnimDatabase::Private::SampleSequenceDirectionsFromRotations);

		check(OutDatabaseDirections.Num() == (DatabaseStopFrame - DatabaseStartFrame));
		check(InSequenceRotations.Num() == (SequenceStopFrame - SequenceStartFrame));

		if (Sampler == EAnimDatabaseSampler::Nearest)
		{
			for (int32 FrameIdx = 0; FrameIdx < DatabaseStopFrame - DatabaseStartFrame; FrameIdx++)
			{
				const int64 Nearest = Math::ComputeNearestSampleFrame(DatabaseToSequence * (DatabaseStartFrame + FrameIdx) - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

				FQuat4f Rotation = FQuat4f::Identity;
				Math::RotationSampleNearest(Rotation, InSequenceRotations, Nearest);
				OutDatabaseDirections[FrameIdx] = Rotation.RotateVector(ForwardVector);
			}
		}
		else if (Sampler == EAnimDatabaseSampler::Linear)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE;
			float Alpha = 0.0f;

			for (int32 FrameIdx = 0; FrameIdx < DatabaseStopFrame - DatabaseStartFrame; FrameIdx++)
			{
				Math::ComputeLinearSampleFramesAndAlpha(
					I0, I1, Alpha,
					DatabaseToSequence * (DatabaseStartFrame + FrameIdx) - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

				FQuat4f Rotation = FQuat4f::Identity;
				Math::RotationSampleLinear(Rotation, InSequenceRotations, I0, I1, Alpha);
				OutDatabaseDirections[FrameIdx] = Rotation.RotateVector(ForwardVector);
			}
		}
		else if (Sampler == EAnimDatabaseSampler::Cubic)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE, I2 = INDEX_NONE, I3 = INDEX_NONE;
			float Alpha = 0.0f;

			for (int32 FrameIdx = 0; FrameIdx < DatabaseStopFrame - DatabaseStartFrame; FrameIdx++)
			{
				Math::ComputeCubicSampleFramesAndAlpha(
					I0, I1, I2, I3, Alpha,
					DatabaseToSequence * (DatabaseStartFrame + FrameIdx) - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

				FQuat4f Rotation = FQuat4f::Identity;
				Math::RotationSampleCubic(Rotation, InSequenceRotations, I0, I1, I2, I3, Alpha);
				OutDatabaseDirections[FrameIdx] = Rotation.RotateVector(ForwardVector);
			}
		}
		else if (Sampler == EAnimDatabaseSampler::CubicMono)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE, I2 = INDEX_NONE, I3 = INDEX_NONE;
			float Alpha = 0.0f;

			for (int32 FrameIdx = 0; FrameIdx < DatabaseStopFrame - DatabaseStartFrame; FrameIdx++)
			{
				Math::ComputeCubicSampleFramesAndAlpha(
					I0, I1, I2, I3, Alpha,
					DatabaseToSequence * (DatabaseStartFrame + FrameIdx) - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

				FQuat4f Rotation = FQuat4f::Identity;
				Math::RotationSampleCubicMono(Rotation, InSequenceRotations, I0, I1, I2, I3, Alpha);
				OutDatabaseDirections[FrameIdx] = Rotation.RotateVector(ForwardVector);
			}
		}
		else
		{
			checkNoEntry();
		}
	}

	static inline void SampleSequenceLinearVelocities(
		const TLearningArrayView<1, FVector3f> OutDatabaseLinearVelocities,
		const TLearningArrayView<1, const FVector> InSequenceLocations,
		const int32 SequenceStartFrame,
		const int32 SequenceStopFrame,
		const float DatabaseToSequence,
		const int32 DatabaseStartFrame,
		const int32 DatabaseStopFrame,
		const float SequenceDeltaTime,
		const EAnimDatabaseSampler Sampler)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(AnimDatabase::Private::SampleSequenceLinearVelocities);

		check(OutDatabaseLinearVelocities.Num() == (DatabaseStopFrame - DatabaseStartFrame));
		check(InSequenceLocations.Num() == (SequenceStopFrame - SequenceStartFrame));

		if (Sampler == EAnimDatabaseSampler::Nearest)
		{
			for (int32 FrameIdx = 0; FrameIdx < DatabaseStopFrame - DatabaseStartFrame; FrameIdx++)
			{
				const int64 Nearest = Math::ComputeNearestSampleFrame(DatabaseToSequence * (DatabaseStartFrame + FrameIdx) - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

				FVector OutLocation = FVector::ZeroVector;
				Math::LocationSampleNearest(OutLocation, OutDatabaseLinearVelocities[FrameIdx], InSequenceLocations, Nearest);
			}
		}
		else if (Sampler == EAnimDatabaseSampler::Linear)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE;
			float Alpha = 0.0f;

			for (int32 FrameIdx = 0; FrameIdx < DatabaseStopFrame - DatabaseStartFrame; FrameIdx++)
			{
				Math::ComputeLinearSampleFramesAndAlpha(
					I0, I1, Alpha,
					DatabaseToSequence * (DatabaseStartFrame + FrameIdx) - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

				FVector OutLocation = FVector::ZeroVector;
				Math::LocationSampleLinear(OutLocation, OutDatabaseLinearVelocities[FrameIdx], InSequenceLocations, I0, I1, Alpha, SequenceDeltaTime);
			}
		}
		else if (Sampler == EAnimDatabaseSampler::Cubic)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE, I2 = INDEX_NONE, I3 = INDEX_NONE;
			float Alpha = 0.0f;

			for (int32 FrameIdx = 0; FrameIdx < DatabaseStopFrame - DatabaseStartFrame; FrameIdx++)
			{
				Math::ComputeCubicSampleFramesAndAlpha(
					I0, I1, I2, I3, Alpha,
					DatabaseToSequence * (DatabaseStartFrame + FrameIdx) - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

				FVector OutLocation = FVector::ZeroVector;
				Math::LocationSampleCubic(OutLocation, OutDatabaseLinearVelocities[FrameIdx], InSequenceLocations, I0, I1, I2, I3, Alpha, SequenceDeltaTime);
			}
		}
		else if (Sampler == EAnimDatabaseSampler::CubicMono)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE, I2 = INDEX_NONE, I3 = INDEX_NONE;
			float Alpha = 0.0f;

			for (int32 FrameIdx = 0; FrameIdx < DatabaseStopFrame - DatabaseStartFrame; FrameIdx++)
			{
				Math::ComputeCubicSampleFramesAndAlpha(
					I0, I1, I2, I3, Alpha,
					DatabaseToSequence * (DatabaseStartFrame + FrameIdx) - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

				FVector OutLocation = FVector::ZeroVector;
				Math::LocationSampleCubicMono(OutLocation, OutDatabaseLinearVelocities[FrameIdx], InSequenceLocations, I0, I1, I2, I3, Alpha, SequenceDeltaTime);
			}
		}
		else
		{
			checkNoEntry();
		}
	}

	static inline void SampleSequenceAngularVelocities(
		const TLearningArrayView<1, FVector3f> OutDatabaseAngularVelocities,
		const TLearningArrayView<1, const FQuat4f> InSequenceRotations,
		const int32 SequenceStartFrame,
		const int32 SequenceStopFrame,
		const float DatabaseToSequence,
		const int32 DatabaseStartFrame,
		const int32 DatabaseStopFrame,
		const float SequenceDeltaTime,
		const EAnimDatabaseSampler Sampler)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(AnimDatabase::Private::SampleSequenceAngularVelocities);

		check(OutDatabaseAngularVelocities.Num() == (DatabaseStopFrame - DatabaseStartFrame));
		check(InSequenceRotations.Num() == (SequenceStopFrame - SequenceStartFrame));

		if (Sampler == EAnimDatabaseSampler::Nearest)
		{
			for (int32 FrameIdx = 0; FrameIdx < DatabaseStopFrame - DatabaseStartFrame; FrameIdx++)
			{
				const int64 Nearest = Math::ComputeNearestSampleFrame(DatabaseToSequence * (DatabaseStartFrame + FrameIdx) - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

				FQuat4f OutRotation = FQuat4f::Identity;
				Math::RotationSampleNearest(OutRotation, OutDatabaseAngularVelocities[FrameIdx], InSequenceRotations, Nearest);
			}
		}
		else if (Sampler == EAnimDatabaseSampler::Linear)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE;
			float Alpha = 0.0f;

			for (int32 FrameIdx = 0; FrameIdx < DatabaseStopFrame - DatabaseStartFrame; FrameIdx++)
			{
				Math::ComputeLinearSampleFramesAndAlpha(
					I0, I1, Alpha,
					DatabaseToSequence * (DatabaseStartFrame + FrameIdx) - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

				FQuat4f OutRotation = FQuat4f::Identity;
				Math::RotationSampleLinear(OutRotation, OutDatabaseAngularVelocities[FrameIdx], InSequenceRotations, I0, I1, Alpha, SequenceDeltaTime);
			}
		}
		else if (Sampler == EAnimDatabaseSampler::Cubic)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE, I2 = INDEX_NONE, I3 = INDEX_NONE;
			float Alpha = 0.0f;

			for (int32 FrameIdx = 0; FrameIdx < DatabaseStopFrame - DatabaseStartFrame; FrameIdx++)
			{
				Math::ComputeCubicSampleFramesAndAlpha(
					I0, I1, I2, I3, Alpha,
					DatabaseToSequence * (DatabaseStartFrame + FrameIdx) - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

				FQuat4f OutRotation = FQuat4f::Identity;
				Math::RotationSampleCubic(OutRotation, OutDatabaseAngularVelocities[FrameIdx], InSequenceRotations, I0, I1, I2, I3, Alpha, SequenceDeltaTime);
			}
		}
		else if (Sampler == EAnimDatabaseSampler::CubicMono)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE, I2 = INDEX_NONE, I3 = INDEX_NONE;
			float Alpha = 0.0f;

			for (int32 FrameIdx = 0; FrameIdx < DatabaseStopFrame - DatabaseStartFrame; FrameIdx++)
			{
				Math::ComputeCubicSampleFramesAndAlpha(
					I0, I1, I2, I3, Alpha,
					DatabaseToSequence * (DatabaseStartFrame + FrameIdx) - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

				FQuat4f OutRotation = FQuat4f::Identity;
				Math::RotationSampleCubicMono(OutRotation, OutDatabaseAngularVelocities[FrameIdx], InSequenceRotations, I0, I1, I2, I3, Alpha, SequenceDeltaTime);
			}
		}
		else
		{
			checkNoEntry();
		}
	}

	static inline void SampleSequenceCurveActive(
		const TLearningArrayView<2, bool> OutDatabaseCurveActive,
		const TLearningArrayView<2, const bool> InSequenceCurveActive,
		const int32 SequenceStartFrame,
		const int32 SequenceStopFrame,
		const float DatabaseToSequence,
		const int32 DatabaseStartFrame,
		const int32 DatabaseStopFrame,
		const float SequenceDeltaTime)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(AnimDatabase::Private::SampleSequenceCurveActive);

		check(OutDatabaseCurveActive.Num<1>() == InSequenceCurveActive.Num<1>());
		check(OutDatabaseCurveActive.Num<0>() == (DatabaseStopFrame - DatabaseStartFrame));
		check(InSequenceCurveActive.Num<0>() == (SequenceStopFrame - SequenceStartFrame));

		const int32 CurveNum = OutDatabaseCurveActive.Num<1>();

		for (int32 FrameIdx = 0; FrameIdx < DatabaseStopFrame - DatabaseStartFrame; FrameIdx++)
		{
			const int64 Nearest = Math::ComputeNearestSampleFrame(
				DatabaseToSequence * (DatabaseStartFrame + FrameIdx) - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

			UE::Learning::Array::Copy(OutDatabaseCurveActive[FrameIdx], InSequenceCurveActive[Nearest]);
		}
	}

	static inline void SampleSequenceCurveValue(
		const TLearningArrayView<2, float> OutDatabaseCurveValue,
		const TLearningArrayView<2, float> OutDatabaseCurveVelocity,
		const TLearningArrayView<2, const float> InSequenceCurveValue,
		const int32 SequenceStartFrame,
		const int32 SequenceStopFrame,
		const float DatabaseToSequence,
		const int32 DatabaseStartFrame,
		const int32 DatabaseStopFrame,
		const float SequenceDeltaTime,
		const EAnimDatabaseSampler Sampler)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(AnimDatabase::Private::SampleSequenceCurveValue);

		check(OutDatabaseCurveValue.Num<1>() == InSequenceCurveValue.Num<1>());
		check(OutDatabaseCurveVelocity.Num<1>() == InSequenceCurveValue.Num<1>());
		check(OutDatabaseCurveValue.Num<0>() == (DatabaseStopFrame - DatabaseStartFrame));
		check(OutDatabaseCurveVelocity.Num<0>() == (DatabaseStopFrame - DatabaseStartFrame));
		check(InSequenceCurveValue.Num<0>() == (SequenceStopFrame - SequenceStartFrame));

		const int32 CurveNum = OutDatabaseCurveValue.Num<1>();
		
		if (Sampler == EAnimDatabaseSampler::Nearest)
		{
			for (int32 FrameIdx = 0; FrameIdx < DatabaseStopFrame - DatabaseStartFrame; FrameIdx++)
			{
				const int64 Nearest = Math::ComputeNearestSampleFrame(DatabaseToSequence * (DatabaseStartFrame + FrameIdx) - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

				Math::ArraySampleNearest(OutDatabaseCurveValue[FrameIdx], OutDatabaseCurveVelocity[FrameIdx], InSequenceCurveValue, Nearest);
			}
		}
		else if (Sampler == EAnimDatabaseSampler::Linear)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE;
			float Alpha = 0.0f;

			for (int32 FrameIdx = 0; FrameIdx < DatabaseStopFrame - DatabaseStartFrame; FrameIdx++)
			{
				Math::ComputeLinearSampleFramesAndAlpha(
					I0, I1, Alpha,
					DatabaseToSequence * (DatabaseStartFrame + FrameIdx) - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

				Math::ArraySampleLinear(OutDatabaseCurveValue[FrameIdx], OutDatabaseCurveVelocity[FrameIdx], InSequenceCurveValue, I0, I1, Alpha, SequenceDeltaTime);
			}
		}
		else if (Sampler == EAnimDatabaseSampler::Cubic)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE, I2 = INDEX_NONE, I3 = INDEX_NONE;
			float Alpha = 0.0f;

			for (int32 FrameIdx = 0; FrameIdx < DatabaseStopFrame - DatabaseStartFrame; FrameIdx++)
			{
				Math::ComputeCubicSampleFramesAndAlpha(
					I0, I1, I2, I3, Alpha,
					DatabaseToSequence * (DatabaseStartFrame + FrameIdx) - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

				Math::ArraySampleCubic(OutDatabaseCurveValue[FrameIdx], OutDatabaseCurveVelocity[FrameIdx], InSequenceCurveValue, I0, I1, I2, I3, Alpha, SequenceDeltaTime);
			}
		}
		else if (Sampler == EAnimDatabaseSampler::CubicMono)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE, I2 = INDEX_NONE, I3 = INDEX_NONE;
			float Alpha = 0.0f;

			for (int32 FrameIdx = 0; FrameIdx < DatabaseStopFrame - DatabaseStartFrame; FrameIdx++)
			{
				Math::ComputeCubicSampleFramesAndAlpha(
					I0, I1, I2, I3, Alpha,
					DatabaseToSequence * (DatabaseStartFrame + FrameIdx) - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

				Math::ArraySampleCubicMono(OutDatabaseCurveValue[FrameIdx], OutDatabaseCurveVelocity[FrameIdx], InSequenceCurveValue, I0, I1, I2, I3, Alpha, SequenceDeltaTime);
			}
		}
		else
		{
			checkNoEntry();
		}
	}

	static inline FTransform SampleSequenceTransform(
		const TLearningArrayView<1, const FVector> InSequenceLocations,
		const TLearningArrayView<1, const FQuat4f> InSequenceRotations,
		const TLearningArrayView<1, const FVector3f> InSequenceScales,
		const int32 SequenceStartFrame,
		const int32 SequenceStopFrame,
		const float DatabaseToSequence,
		const float DatabaseFrameTime,
		const float SequenceDeltaTime,
		const EAnimDatabaseSampler Sampler)
	{
		check(InSequenceLocations.Num() == (SequenceStopFrame - SequenceStartFrame));
		check(InSequenceRotations.Num() == (SequenceStopFrame - SequenceStartFrame));
		check(InSequenceScales.Num() == (SequenceStopFrame - SequenceStartFrame));

		if (Sampler == EAnimDatabaseSampler::Nearest)
		{
			const int64 Nearest = Math::ComputeNearestSampleFrame(DatabaseToSequence * DatabaseFrameTime - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

			FVector Location = FVector::ZeroVector; FQuat4f Rotation = FQuat4f::Identity; FVector3f Scale = FVector3f::OneVector;
			Math::TransformSampleNearest(Location, Rotation, Scale, InSequenceLocations, InSequenceRotations, InSequenceScales, Nearest);

			return FTransform(
				((FQuat)Rotation).GetNormalized(),
				Location,
				(FVector)Scale);
		}
		else if (Sampler == EAnimDatabaseSampler::Linear)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE;
			float Alpha = 0.0f;

			Math::ComputeLinearSampleFramesAndAlpha(
				I0, I1, Alpha,
				DatabaseToSequence * DatabaseFrameTime - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

			FVector Location = FVector::ZeroVector; FQuat4f Rotation = FQuat4f::Identity; FVector3f Scale = FVector3f::OneVector;
			Math::TransformSampleLinear(Location, Rotation, Scale, InSequenceLocations, InSequenceRotations, InSequenceScales, I0, I1, Alpha);

			return FTransform(
				((FQuat)Rotation).GetNormalized(),
				Location,
				(FVector)Scale);
		}
		else if (Sampler == EAnimDatabaseSampler::Cubic)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE, I2 = INDEX_NONE, I3 = INDEX_NONE;
			float Alpha = 0.0f;

			Math::ComputeCubicSampleFramesAndAlpha(
				I0, I1, I2, I3, Alpha,
				DatabaseToSequence * DatabaseFrameTime - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

			FVector Location = FVector::ZeroVector; FQuat4f Rotation = FQuat4f::Identity; FVector3f Scale = FVector3f::OneVector;
			Math::TransformSampleCubic(Location, Rotation, Scale, InSequenceLocations, InSequenceRotations, InSequenceScales, I0, I1, I2, I3, Alpha);

			return FTransform(
				((FQuat)Rotation).GetNormalized(),
				Location,
				(FVector)Scale);
		}
		else if (Sampler == EAnimDatabaseSampler::CubicMono)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE, I2 = INDEX_NONE, I3 = INDEX_NONE;
			float Alpha = 0.0f;

			Math::ComputeCubicSampleFramesAndAlpha(
				I0, I1, I2, I3, Alpha,
				DatabaseToSequence * DatabaseFrameTime - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

			FVector Location = FVector::ZeroVector; FQuat4f Rotation = FQuat4f::Identity; FVector3f Scale = FVector3f::OneVector;
			Math::TransformSampleCubicMono(Location, Rotation, Scale, InSequenceLocations, InSequenceRotations, InSequenceScales, I0, I1, I2, I3, Alpha);

			return FTransform(
				((FQuat)Rotation).GetNormalized(),
				Location,
				(FVector)Scale);
		}
		else
		{
			checkNoEntry();
			return FTransform();
		}
	}

	static inline FVector SampleSequenceLocation(
		const TLearningArrayView<1, const FVector> InSequenceLocations,
		const int32 SequenceStartFrame,
		const int32 SequenceStopFrame,
		const float DatabaseToSequence,
		const float DatabaseFrameTime,
		const float SequenceDeltaTime,
		const EAnimDatabaseSampler Sampler)
	{
		check(InSequenceLocations.Num() == (SequenceStopFrame - SequenceStartFrame));

		if (Sampler == EAnimDatabaseSampler::Nearest)
		{
			const int64 Nearest = Math::ComputeNearestSampleFrame(DatabaseToSequence * DatabaseFrameTime - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

			FVector Location = FVector::ZeroVector;
			Math::LocationSampleNearest(Location, InSequenceLocations, Nearest);

			return Location;
		}
		else if (Sampler == EAnimDatabaseSampler::Linear)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE;
			float Alpha = 0.0f;

			Math::ComputeLinearSampleFramesAndAlpha(
				I0, I1, Alpha,
				DatabaseToSequence * DatabaseFrameTime - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

			FVector Location = FVector::ZeroVector;
			Math::LocationSampleLinear(Location, InSequenceLocations, I0, I1, Alpha);

			return Location;
		}
		else if (Sampler == EAnimDatabaseSampler::Cubic)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE, I2 = INDEX_NONE, I3 = INDEX_NONE;
			float Alpha = 0.0f;

			Math::ComputeCubicSampleFramesAndAlpha(
				I0, I1, I2, I3, Alpha,
				DatabaseToSequence * DatabaseFrameTime - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

			FVector Location = FVector::ZeroVector;
			Math::LocationSampleCubic(Location, InSequenceLocations, I0, I1, I2, I3, Alpha);

			return Location;
		}
		else if (Sampler == EAnimDatabaseSampler::CubicMono)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE, I2 = INDEX_NONE, I3 = INDEX_NONE;
			float Alpha = 0.0f;

			Math::ComputeCubicSampleFramesAndAlpha(
				I0, I1, I2, I3, Alpha,
				DatabaseToSequence * DatabaseFrameTime - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

			FVector Location = FVector::ZeroVector;
			Math::LocationSampleCubicMono(Location, InSequenceLocations, I0, I1, I2, I3, Alpha);

			return Location;
		}
		else
		{
			checkNoEntry();
			return FVector::ZeroVector;
		}
	}

	static inline FQuat4f SampleSequenceRotation(
		const TLearningArrayView<1, const FQuat4f> InSequenceRotations,
		const int32 SequenceStartFrame,
		const int32 SequenceStopFrame,
		const float DatabaseToSequence,
		const float DatabaseFrameTime,
		const float SequenceDeltaTime,
		const EAnimDatabaseSampler Sampler)
	{
		check(InSequenceRotations.Num() == (SequenceStopFrame - SequenceStartFrame));

		if (Sampler == EAnimDatabaseSampler::Nearest)
		{
			const int64 Nearest = Math::ComputeNearestSampleFrame(DatabaseToSequence * DatabaseFrameTime - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

			FQuat4f Rotation = FQuat4f::Identity;
			Math::RotationSampleNearest(Rotation, InSequenceRotations, Nearest);

			return Rotation;
		}
		else if (Sampler == EAnimDatabaseSampler::Linear)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE;
			float Alpha = 0.0f;

			Math::ComputeLinearSampleFramesAndAlpha(
				I0, I1, Alpha,
				DatabaseToSequence * DatabaseFrameTime - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

			FQuat4f Rotation = FQuat4f::Identity;
			Math::RotationSampleLinear(Rotation, InSequenceRotations, I0, I1, Alpha);

			return Rotation;
		}
		else if (Sampler == EAnimDatabaseSampler::Cubic)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE, I2 = INDEX_NONE, I3 = INDEX_NONE;
			float Alpha = 0.0f;

			Math::ComputeCubicSampleFramesAndAlpha(
				I0, I1, I2, I3, Alpha,
				DatabaseToSequence * DatabaseFrameTime - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

			FQuat4f Rotation = FQuat4f::Identity;
			Math::RotationSampleCubic(Rotation, InSequenceRotations, I0, I1, I2, I3, Alpha);

			return Rotation;
		}
		else if (Sampler == EAnimDatabaseSampler::CubicMono)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE, I2 = INDEX_NONE, I3 = INDEX_NONE;
			float Alpha = 0.0f;

			Math::ComputeCubicSampleFramesAndAlpha(
				I0, I1, I2, I3, Alpha,
				DatabaseToSequence * DatabaseFrameTime - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

			FQuat4f Rotation = FQuat4f::Identity;
			Math::RotationSampleCubicMono(Rotation, InSequenceRotations, I0, I1, I2, I3, Alpha);

			return Rotation;
		}
		else
		{
			checkNoEntry();
			return FQuat4f::Identity;
		}
	}

	static inline FVector3f SampleSequenceLinearVelocity(
		const TLearningArrayView<1, const FVector> InSequenceLocations,
		const int32 SequenceStartFrame,
		const int32 SequenceStopFrame,
		const float DatabaseToSequence,
		const float DatabaseFrameTime,
		const float SequenceDeltaTime,
		const EAnimDatabaseSampler Sampler)
	{
		check(InSequenceLocations.Num() == (SequenceStopFrame - SequenceStartFrame));

		if (Sampler == EAnimDatabaseSampler::Nearest)
		{
			const int64 Nearest = Math::ComputeNearestSampleFrame(DatabaseToSequence * DatabaseFrameTime - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

			FVector Location = FVector::ZeroVector; FVector3f LinearVelocity = FVector3f::ZeroVector;
			Math::LocationSampleNearest(Location, LinearVelocity, InSequenceLocations, Nearest);

			return LinearVelocity;
		}
		else if (Sampler == EAnimDatabaseSampler::Linear)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE;
			float Alpha = 0.0f;

			Math::ComputeLinearSampleFramesAndAlpha(
				I0, I1,Alpha,
				DatabaseToSequence * DatabaseFrameTime - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

			FVector Location = FVector::ZeroVector; FVector3f LinearVelocity = FVector3f::ZeroVector;
			Math::LocationSampleLinear(Location, LinearVelocity, InSequenceLocations, I0, I1, Alpha, SequenceDeltaTime);

			return LinearVelocity;
		}
		else if (Sampler == EAnimDatabaseSampler::Cubic)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE, I2 = INDEX_NONE, I3 = INDEX_NONE;
			float Alpha = 0.0f;

			Math::ComputeCubicSampleFramesAndAlpha(
				I0, I1, I2, I3, Alpha,
				DatabaseToSequence * DatabaseFrameTime - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

			FVector Location = FVector::ZeroVector; FVector3f LinearVelocity = FVector3f::ZeroVector;
			Math::LocationSampleCubic(Location, LinearVelocity, InSequenceLocations, I0, I1, I2, I3, Alpha, SequenceDeltaTime);

			return LinearVelocity;
		}
		else if (Sampler == EAnimDatabaseSampler::CubicMono)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE, I2 = INDEX_NONE, I3 = INDEX_NONE;
			float Alpha = 0.0f;

			Math::ComputeCubicSampleFramesAndAlpha(
				I0, I1, I2, I3, Alpha,
				DatabaseToSequence * DatabaseFrameTime - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

			FVector Location = FVector::ZeroVector; FVector3f LinearVelocity = FVector3f::ZeroVector;
			Math::LocationSampleCubicMono(Location, LinearVelocity, InSequenceLocations, I0, I1, I2, I3, Alpha, SequenceDeltaTime);

			return LinearVelocity;
		}
		else
		{
			checkNoEntry();
			return FVector3f::ZeroVector;
		}
	}

	static inline void SampleSequenceTransformVelocity(
		FVector& OutLocation,
		FQuat4f& OutRotation,
		FVector3f& OutScale,
		FVector3f& OutLinearVelocity,
		FVector3f& OutAngularVelocity,
		FVector3f& OutScalarVelocity,
		const TLearningArrayView<1, const FVector> InSequenceLocations,
		const TLearningArrayView<1, const FQuat4f> InSequenceRotations,
		const TLearningArrayView<1, const FVector3f> InSequenceScales,
		const int32 SequenceStartFrame,
		const int32 SequenceStopFrame,
		const float DatabaseToSequence,
		const float DatabaseFrameTime,
		const float SequenceDeltaTime,
		const EAnimDatabaseSampler Sampler)
	{
		check(InSequenceLocations.Num() == (SequenceStopFrame - SequenceStartFrame));
		check(InSequenceRotations.Num() == (SequenceStopFrame - SequenceStartFrame));
		check(InSequenceScales.Num() == (SequenceStopFrame - SequenceStartFrame));

		if (Sampler == EAnimDatabaseSampler::Nearest)
		{
			const int64 Nearest = Math::ComputeNearestSampleFrame(DatabaseToSequence * DatabaseFrameTime - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

			Math::TransformSampleNearest(
				OutLocation, OutRotation, OutScale, OutLinearVelocity, OutAngularVelocity, OutScalarVelocity,
				InSequenceLocations, InSequenceRotations, InSequenceScales, Nearest);
		}
		else if (Sampler == EAnimDatabaseSampler::Linear)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE;
			float Alpha = 0.0f;

			Math::ComputeLinearSampleFramesAndAlpha(
				I0, I1, Alpha,
				DatabaseToSequence * DatabaseFrameTime - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

			Math::TransformSampleLinear(
				OutLocation, OutRotation, OutScale, OutLinearVelocity, OutAngularVelocity, OutScalarVelocity,
				InSequenceLocations, InSequenceRotations, InSequenceScales, I0, I1, Alpha, SequenceDeltaTime);
		}
		else if (Sampler == EAnimDatabaseSampler::Cubic)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE, I2 = INDEX_NONE, I3 = INDEX_NONE;
			float Alpha = 0.0f;

			Math::ComputeCubicSampleFramesAndAlpha(
				I0, I1, I2, I3, Alpha,
				DatabaseToSequence * DatabaseFrameTime - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

			Math::TransformSampleCubic(
				OutLocation, OutRotation, OutScale, OutLinearVelocity, OutAngularVelocity, OutScalarVelocity,
				InSequenceLocations, InSequenceRotations, InSequenceScales, I0, I1, I2, I3, Alpha, SequenceDeltaTime);
		}
		else if (Sampler == EAnimDatabaseSampler::CubicMono)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE, I2 = INDEX_NONE, I3 = INDEX_NONE;
			float Alpha = 0.0f;

			Math::ComputeCubicSampleFramesAndAlpha(
				I0, I1, I2, I3, Alpha,
				DatabaseToSequence * DatabaseFrameTime - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

			Math::TransformSampleCubicMono(
				OutLocation, OutRotation, OutScale, OutLinearVelocity, OutAngularVelocity, OutScalarVelocity,
				InSequenceLocations, InSequenceRotations, InSequenceScales, I0, I1, I2, I3, Alpha, SequenceDeltaTime);
		}
		else
		{
			checkNoEntry();
		}
	}

	static inline void SampleSequenceTransformVelocity(
		const TLearningArrayView<1, FVector3f> OutLocations,
		const TLearningArrayView<1, FQuat4f> OutRotations,
		const TLearningArrayView<1, FVector3f> OutScales,
		const TLearningArrayView<1, FVector3f> OutLinearVelocities,
		const TLearningArrayView<1, FVector3f> OutAngularVelocities,
		const TLearningArrayView<1, FVector3f> OutScalarVelocities,
		const TLearningArrayView<2, const FVector3f> InSequenceLocations,
		const TLearningArrayView<2, const FQuat4f> InSequenceRotations,
		const TLearningArrayView<2, const FVector3f> InSequenceScales,
		const int32 SequenceStartFrame,
		const int32 SequenceStopFrame,
		const float DatabaseToSequence,
		const float DatabaseFrameTime,
		const float SequenceDeltaTime,
		const EAnimDatabaseSampler Sampler)
	{
		check(InSequenceLocations.Num<0>() == (SequenceStopFrame - SequenceStartFrame));
		check(InSequenceRotations.Num<0>() == (SequenceStopFrame - SequenceStartFrame));
		check(InSequenceScales.Num<0>() == (SequenceStopFrame - SequenceStartFrame));

		if (Sampler == EAnimDatabaseSampler::Nearest)
		{
			const int64 Nearest = Math::ComputeNearestSampleFrame(DatabaseToSequence * DatabaseFrameTime - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

			Math::TransformSampleNearest(
				OutLocations,
				OutRotations,
				OutScales,
				OutLinearVelocities,
				OutAngularVelocities,
				OutScalarVelocities,
				InSequenceLocations, InSequenceRotations, InSequenceScales, Nearest);
		}
		else if (Sampler == EAnimDatabaseSampler::Linear)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE;
			float Alpha = 0.0f;

			Math::ComputeLinearSampleFramesAndAlpha(
				I0, I1, Alpha,
				DatabaseToSequence * DatabaseFrameTime - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

			Math::TransformSampleLinear(
				OutLocations,
				OutRotations,
				OutScales,
				OutLinearVelocities,
				OutAngularVelocities,
				OutScalarVelocities,
				InSequenceLocations, InSequenceRotations, InSequenceScales, I0, I1, Alpha, SequenceDeltaTime);
		}
		else if (Sampler == EAnimDatabaseSampler::Cubic)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE, I2 = INDEX_NONE, I3 = INDEX_NONE;
			float Alpha = 0.0f;

			Math::ComputeCubicSampleFramesAndAlpha(
				I0, I1, I2, I3, Alpha,
				DatabaseToSequence * DatabaseFrameTime - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

			Math::TransformSampleCubic(
				OutLocations,
				OutRotations,
				OutScales,
				OutLinearVelocities,
				OutAngularVelocities,
				OutScalarVelocities,
				InSequenceLocations, InSequenceRotations, InSequenceScales, I0, I1, I2, I3, Alpha, SequenceDeltaTime);
		}
		else if (Sampler == EAnimDatabaseSampler::CubicMono)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE, I2 = INDEX_NONE, I3 = INDEX_NONE;
			float Alpha = 0.0f;

			Math::ComputeCubicSampleFramesAndAlpha(
				I0, I1, I2, I3, Alpha,
				DatabaseToSequence * DatabaseFrameTime - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

			Math::TransformSampleCubicMono(
				OutLocations,
				OutRotations,
				OutScales,
				OutLinearVelocities,
				OutAngularVelocities,
				OutScalarVelocities,
				InSequenceLocations, InSequenceRotations, InSequenceScales, I0, I1, I2, I3, Alpha, SequenceDeltaTime);
		}
		else
		{
			checkNoEntry();
		}
	}

	static inline void SampleSequenceCurveActive(
		const TLearningArrayView<1, bool> OutCurveActive,
		const TLearningArrayView<2, const bool> InSequenceCurveActive,
		const int32 SequenceStartFrame,
		const int32 SequenceStopFrame,
		const float DatabaseToSequence,
		const float DatabaseFrameTime,
		const float SequenceDeltaTime)
	{
		check(InSequenceCurveActive.Num() == (SequenceStopFrame - SequenceStartFrame));

		const int64 Nearest = Math::ComputeNearestSampleFrame(
			DatabaseToSequence * DatabaseFrameTime - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

		UE::Learning::Array::Copy(OutCurveActive, InSequenceCurveActive[Nearest]);
	}

	static inline void SampleSequenceCurveValue(
		const TLearningArrayView<1, float> OutCurveValue,
		const TLearningArrayView<1, float> OutCurveVelocity,
		const TLearningArrayView<2, const float> InSequenceCurveValue,
		const int32 SequenceStartFrame,
		const int32 SequenceStopFrame,
		const float DatabaseToSequence,
		const float DatabaseFrameTime,
		const float SequenceDeltaTime,
		const EAnimDatabaseSampler Sampler)
	{
		check(InSequenceCurveValue.Num<1>() == OutCurveValue.Num());
		check(InSequenceCurveValue.Num<1>() == OutCurveVelocity.Num());
		check(InSequenceCurveValue.Num<0>() == (SequenceStopFrame - SequenceStartFrame));
		check(InSequenceCurveValue.Num<0>() == (SequenceStopFrame - SequenceStartFrame));
		check(InSequenceCurveValue.Num<0>() == (SequenceStopFrame - SequenceStartFrame));

		if (Sampler == EAnimDatabaseSampler::Nearest)
		{
			const int64 Nearest = Math::ComputeNearestSampleFrame(DatabaseToSequence * DatabaseFrameTime - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

			Math::ArraySampleNearest(OutCurveValue, OutCurveVelocity, InSequenceCurveValue, Nearest);
		}
		else if (Sampler == EAnimDatabaseSampler::Linear)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE;
			float Alpha = 0.0f;

			Math::ComputeLinearSampleFramesAndAlpha(
				I0, I1, Alpha,
				DatabaseToSequence * DatabaseFrameTime - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

			Math::ArraySampleLinear(OutCurveValue, OutCurveVelocity, InSequenceCurveValue, I0, I1, Alpha, SequenceDeltaTime);
		}
		else if (Sampler == EAnimDatabaseSampler::Cubic)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE, I2 = INDEX_NONE, I3 = INDEX_NONE;
			float Alpha = 0.0f;

			Math::ComputeCubicSampleFramesAndAlpha(
				I0, I1, I2, I3, Alpha,
				DatabaseToSequence * DatabaseFrameTime - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

			Math::ArraySampleCubic(OutCurveValue, OutCurveVelocity, InSequenceCurveValue, I0, I1, I2, I3, Alpha, SequenceDeltaTime);
		}
		else if (Sampler == EAnimDatabaseSampler::CubicMono)
		{
			int64 I0 = INDEX_NONE, I1 = INDEX_NONE, I2 = INDEX_NONE, I3 = INDEX_NONE;
			float Alpha = 0.0f;

			Math::ComputeCubicSampleFramesAndAlpha(
				I0, I1, I2, I3, Alpha,
				DatabaseToSequence * DatabaseFrameTime - SequenceStartFrame, SequenceStopFrame - SequenceStartFrame);

			Math::ArraySampleCubicMono(OutCurveValue, OutCurveVelocity, InSequenceCurveValue, I0, I1, I2, I3, Alpha, SequenceDeltaTime);
		}
		else
		{
			checkNoEntry();
		}
	}

	static void BVHExportBone(
		FString& InOut,
		FString& InOutIndentation,
		TArray<int32>& OutBoneOrder,
		const TArrayView<const int32> BoneParents,
		const TArrayView<const FName> BoneNames,
		const TArrayView<const FVector> BoneOffsets,
		const int32 BoneIndex, 
		const EAnimDatabaseRotationOrder RotationOrder)
	{
		const int32 BoneNum = BoneParents.Num();

		OutBoneOrder.Add(BoneIndex);

		InOut += FString::Printf(TEXT("%sJOINT %s\n"), *InOutIndentation, *BoneNames[BoneIndex].ToString().Replace(TEXT(" "), TEXT("_")));
		InOut += FString::Printf(TEXT("%s{\n"), *InOutIndentation);
		InOutIndentation += TEXT("\t");

		InOut += FString::Printf(TEXT("%sOFFSET %.9g %.9g %.9g\n"), *InOutIndentation,
			(float)BoneOffsets[BoneIndex].X, (float)BoneOffsets[BoneIndex].Y, (float)BoneOffsets[BoneIndex].Z);
		InOut += FString::Printf(TEXT("%sCHANNELS 6 Xposition Yposition Zposition %srotation %srotation %srotation\n"),
			*InOutIndentation,
			UE::AnimDatabase::Private::RotationOrderAxisName(RotationOrder, 0),
			UE::AnimDatabase::Private::RotationOrderAxisName(RotationOrder, 1),
			UE::AnimDatabase::Private::RotationOrderAxisName(RotationOrder, 2));

		bool bEndSize = true;

		for (int32 ChildIdx = 0; ChildIdx < BoneNum; ChildIdx++)
		{
			if (BoneParents[ChildIdx] == BoneIndex)
			{
				BVHExportBone(
					InOut,
					InOutIndentation,
					OutBoneOrder,
					BoneParents,
					BoneNames,
					BoneOffsets,
					ChildIdx,
					RotationOrder);

				bEndSize = false;
			}
		}

		if (bEndSize)
		{
			InOut += FString::Printf(TEXT("%sEnd Site\n"), *InOutIndentation);
			InOut += FString::Printf(TEXT("%s{\n"), *InOutIndentation);
			InOutIndentation += TEXT("\t");
			InOut += FString::Printf(TEXT("%sOFFSET %.9g %.9g %.9g\n"), *InOutIndentation, 0.0f, 0.0f, 0.0f);
			InOutIndentation = InOutIndentation.LeftChop(1);
			InOut += FString::Printf(TEXT("%s}\n"), *InOutIndentation);
		}

		InOutIndentation = InOutIndentation.LeftChop(1);
		InOut += FString::Printf(TEXT("%s}\n"), *InOutIndentation);
	}

	static void BVHExportHeader(
		FString& InOut,
		FString& InOutIndentation,
		TArray<int32>& OutBoneOrder,
		const TArrayView<const int32> BoneParents,
		const TArrayView<const FName> BoneNames,
		const TArrayView<const FVector> RefBoneLocations,
		const TArrayView<const FQuat> RefBoneRotations, 
		const EAnimDatabaseRotationOrder RotationOrder)
	{
		const int32 BoneNum = BoneParents.Num();
		check(BoneNum > 0);
		check(BoneNames.Num() == BoneNum);
		check(RefBoneLocations.Num() == BoneNum);
		check(RefBoneRotations.Num() == BoneNum);

		// Compute Forward Kinematics

		TLearningArray<1, FVector> BoneGlobalLocations;
		TLearningArray<1, FQuat> BoneGlobalRotations;
		BoneGlobalLocations.SetNumUninitialized({ BoneNum });
		BoneGlobalRotations.SetNumUninitialized({ BoneNum });
		Learning::Array::Copy<1, FVector>(BoneGlobalLocations, RefBoneLocations);
		Learning::Array::Copy<1, FQuat>(BoneGlobalRotations, RefBoneRotations);

		for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
		{
			check(BoneParents[BoneIdx] < BoneIdx);
			if (BoneParents[BoneIdx] != INDEX_NONE)
			{
				BoneGlobalLocations[BoneIdx] = BoneGlobalRotations[BoneParents[BoneIdx]].RotateVector(BoneGlobalLocations[BoneIdx]) + BoneGlobalLocations[BoneParents[BoneIdx]];
				BoneGlobalRotations[BoneIdx] = BoneGlobalRotations[BoneParents[BoneIdx]] * BoneGlobalRotations[BoneIdx];
			}
		}

		// Get correct bone offsets

		TLearningArray<1, FVector> BoneOffsets;
		BoneOffsets.SetNumUninitialized({ BoneNum });

		for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
		{
			FVector3f BoneLocation = (FVector3f)BoneGlobalLocations[BoneIdx];
			FQuat4f BoneRotation = (FQuat4f)BoneGlobalRotations[BoneIdx];
			FMatrix44f BoneTransform = FTransform3f(BoneRotation, BoneLocation).ToMatrixNoScale();
			for (int32 Idx = 0; Idx < 4; Idx++) { Swap(BoneTransform.M[1][Idx], BoneTransform.M[2][Idx]); }

			for (int32 Idx = 0; Idx < 3; Idx++)
			{
				const float Y = BoneTransform.M[1][Idx];
				const float Z = BoneTransform.M[2][Idx];
				BoneTransform.M[2][Idx] = +Y;
				BoneTransform.M[1][Idx] = -Z;
			}

			FMatrix44f ParentTransform = FMatrix44f::Identity;
			if (BoneParents[BoneIdx] != -1)
			{
				FVector3f ParentLocation = (FVector3f)BoneGlobalLocations[BoneParents[BoneIdx]];
				FQuat4f ParentRotation = (FQuat4f)BoneGlobalRotations[BoneParents[BoneIdx]];
				ParentTransform = FTransform3f(ParentRotation, ParentLocation).ToMatrixNoScale();
			}

			for (int32 Idx = 0; Idx < 4; Idx++) { Swap(ParentTransform.M[1][Idx], ParentTransform.M[2][Idx]); }

			for (int32 Idx = 0; Idx < 3; Idx++)
			{
				const float Y = ParentTransform.M[1][Idx];
				const float Z = ParentTransform.M[2][Idx];
				ParentTransform.M[2][Idx] = +Y;
				ParentTransform.M[1][Idx] = -Z;
			}

			const FMatrix44f LocalTransform = BoneTransform * ParentTransform.Inverse();
			BoneOffsets[BoneIdx] = FVector(LocalTransform.M[3][0], LocalTransform.M[3][1], LocalTransform.M[3][2]);
		}

		// Output Header and Bones

		InOut = TEXT("");
		InOutIndentation = TEXT("");

		InOut += TEXT("HIERARCHY\n");
		InOut += FString::Printf(TEXT("ROOT %s\n"), *BoneNames[0].ToString().Replace(TEXT(" "), TEXT("_")));
		InOut += FString::Printf(TEXT("%s{\n"), *InOutIndentation);
		InOutIndentation += TEXT("\t");

		InOut += FString::Printf(TEXT("%sOFFSET %.9g %.9g %.9g\n"), *InOutIndentation,
			(float)BoneOffsets[0].X, (float)BoneOffsets[0].Y, (float)BoneOffsets[0].Z);
		InOut += FString::Printf(TEXT("%sCHANNELS 6 Xposition Yposition Zposition %srotation %srotation %srotation\n"), 
			*InOutIndentation,
			UE::AnimDatabase::Private::RotationOrderAxisName(RotationOrder, 0),
			UE::AnimDatabase::Private::RotationOrderAxisName(RotationOrder, 1),
			UE::AnimDatabase::Private::RotationOrderAxisName(RotationOrder, 2));

		OutBoneOrder.Reset();
		OutBoneOrder.Add(0);

		for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
		{
			if (BoneParents[BoneIdx] == 0)
			{
				BVHExportBone(
					InOut,
					InOutIndentation,
					OutBoneOrder,
					BoneParents,
					BoneNames,
					BoneOffsets,
					BoneIdx,
					RotationOrder);
			}
		}

		InOutIndentation = InOutIndentation.LeftChop(1);
		InOut += FString::Printf(TEXT("%s}\n"), *InOutIndentation);
	}

#if WITH_EDITOR

	static inline void OpenControllers(
		TArray<TScriptInterface<IAnimationDataController>>& OutControllers,
		TMap<int32, int32>& OutSequenceControllerIndices,
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const FText& Description,
		const bool bShouldTransact)
	{
		for (const int32 SequenceIdx : FrameRanges.FrameRangeSet->GetEntrySequences())
		{
			if (!OutSequenceControllerIndices.Contains(SequenceIdx))
			{
				if (UAnimSequence* AnimSequence = Database->GetAnimSequence(SequenceIdx))
				{
					TScriptInterface<IAnimationDataController> Controller = AnimSequence->GetDataModel()->GetController();

					const int32 ControllerIdx = OutControllers.Find(Controller);

					if (ControllerIdx == INDEX_NONE)
					{
						Controller->OpenBracket(Description, bShouldTransact);
						OutSequenceControllerIndices.Add(SequenceIdx, OutControllers.Add(Controller));
					}
					else
					{
						OutSequenceControllerIndices.Add(SequenceIdx, ControllerIdx);
					}
				}
			}
		}
	}

	static inline void OpenControllers(
		TArray<TScriptInterface<IAnimationDataController>>& OutControllers,
		TMap<int32, int32>& OutSequenceControllerIndices,
		const UAnimDatabase* Database,
		const TArrayView<const FAnimDatabaseFrameAttribute> FrameAttributes,
		const FText& Description,
		const bool bShouldTransact)
	{
		for (const FAnimDatabaseFrameAttribute& Attribute : FrameAttributes)
		{
			for (const int32 SequenceIdx : Attribute.FrameAttribute->FrameRangeSet.GetEntrySequences())
			{
				if (!OutSequenceControllerIndices.Contains(SequenceIdx))
				{
					if (UAnimSequence* AnimSequence = Database->GetAnimSequence(SequenceIdx))
					{
						TScriptInterface<IAnimationDataController> Controller = AnimSequence->GetDataModel()->GetController();

						const int32 ControllerIdx = OutControllers.Find(Controller);

						if (ControllerIdx == INDEX_NONE)
						{
							Controller->OpenBracket(Description, bShouldTransact);
							OutSequenceControllerIndices.Add(SequenceIdx, OutControllers.Add(Controller));
						}
						else
						{
							OutSequenceControllerIndices.Add(SequenceIdx, ControllerIdx);
						}
					}
				}
			}
		}
	}

	static inline void CloseControllers(const TArray<TScriptInterface<IAnimationDataController>>& Controllers, const bool bShouldTransact)
	{
		for (const TScriptInterface<IAnimationDataController>& Controller : Controllers)
		{
			Controller->CloseBracket(bShouldTransact);
		}
	}

#endif
}

#if WITH_EDITOR

void UAnimDatabaseQuery::UpdateQueryEntries()
{
	if (Database)
	{
		check(QueryRanges.IsValid());
		QueryEntries.SetNum(QueryRanges.FrameRangeSet->GetTotalRangeNum());
		int32 QueryEntryIdx = 0;

		const int32 EntryNum = QueryRanges.FrameRangeSet->GetEntryNum();

		for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
		{
			const int32 EntrySequenceIdx = QueryRanges.FrameRangeSet->GetEntrySequence(EntryIdx);
			const UAnimSequence* AnimSequence = Database->GetAnimSequence(EntrySequenceIdx);
			const bool bIsMirrored = Database->GetIsMirrored(EntrySequenceIdx);

			const int32 RangeNum = QueryRanges.FrameRangeSet->GetEntryRangeNum(EntryIdx);

			for (int32 EntryRangeIdx = 0; EntryRangeIdx < RangeNum; EntryRangeIdx++)
			{
				TSharedPtr<UE::AnimDatabase::Editor::FQueryEntry>& QueryEntry = QueryEntries[QueryEntryIdx];
				if (!QueryEntry.IsValid()) { QueryEntry = MakeShared<UE::AnimDatabase::Editor::FQueryEntry>(); }

				const int32 NewStartFrame = QueryRanges.FrameRangeSet->GetEntryRangeStart(EntryIdx, EntryRangeIdx);
				const int32 NewStopFrame = QueryRanges.FrameRangeSet->GetEntryRangeStop(EntryIdx, EntryRangeIdx) - 1;
				
				// If the entry has changed then reset selection

				if (QueryEntry->Sequence != AnimSequence ||
					QueryEntry->StartFrame != NewStartFrame ||
					QueryEntry->StopFrame != NewStopFrame ||
					QueryEntry->bIsMirrored != bIsMirrored)
				{
					QueryEntry->Sequence = AnimSequence;
					QueryEntry->StartFrame = NewStartFrame;
					QueryEntry->StopFrame = NewStopFrame;
					QueryEntry->bIsMirrored = bIsMirrored;
					QueryEntry->bIsSelected = false;
				}

				// Here we use a hash of the sequence, start, and stop frames to produce the color

				const uint32 HashData[5] =
				{
					(uint32)RangeIdentifierColorSeed,
					GetTypeHash(AnimSequence),
					(uint32)QueryEntry->StartFrame,
					(uint32)QueryEntry->StopFrame,
					(uint32)QueryEntry->bIsMirrored
				};

				const uint32 ColorHash = CityHash32((const char*)HashData, sizeof(HashData));
				const uint8 Hue = ColorHash & 0x000000FF;
				const uint8 Saturation = ((ColorHash & 0x0000FF00) >> 8) % 25 + 230;
				const uint8 Brightness = ((ColorHash & 0x00FF0000) >> 16) % 50 + 205;

				QueryEntry->Color = FLinearColor::MakeFromHSV8(Hue, Saturation, Brightness);

				QueryEntryIdx++;
			}
		}

		check(QueryEntryIdx == QueryEntries.Num());
	}
	else
	{
		QueryEntries.Empty();
	}
}

void UAnimDatabaseQuery::ForceRefresh()
{
	bForceRefresh = true;
}

void UAnimDatabaseQuery::DeleteSelectedFromDatabase()
{
	if (Database)
	{
		TArray<UAnimSequence*> AnimSequences;
		UAnimDatabaseFrameRangesLibrary::FrameRangesAnimSequenceAssets(AnimSequences, Database, SelectedFrameRanges);

		const int32 SequenceNum = AnimSequences.Num();
		for (int32 SequenceIdx = 0; SequenceIdx < SequenceNum; SequenceIdx++)
		{
			Database->Entries.Remove(AnimSequences[SequenceIdx]);
		}
	}
}

bool UAnimDatabaseQuery::Update()
{
	if (Database)
	{
		// Otherwise check for changes to various things to detect update

		bool bQueryUpdateRequired = bForceRefresh;
		bForceRefresh = false;

		// Check if Database has been updated

		const uint32 CurrentContentHash = Database->GetContentHash();
		if (CurrentContentHash != DatabaseContentHash)
		{
			DatabaseContentHash = CurrentContentHash;
			bQueryUpdateRequired = true;
		}

		// Check if ActiveFrameRanges is out-of-date

		if (bQueryUpdateRequired)
		{
			if (FrameRanges)
			{
				FScopedSlowTask SlowTask(0.0f, LOCTEXT("RebuildingQueryRanges", "Rebuilding Query Frame Ranges"));
				SlowTask.MakeDialog();

				QueryRanges = UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFunction(Database, UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromDatabase(Database), FrameRanges);
				if (!QueryRanges.IsValid()) { QueryRanges = UAnimDatabaseFrameRangesLibrary::MakeEmptyFrameRanges(); }
			}
			else
			{
				QueryRanges = UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromDatabase(Database);
			}
			check(QueryRanges.IsValid());

			UpdateQueryEntries();
		}

		check(QueryRanges.IsValid());

		// Update Debug Drawer

		if (bQueryUpdateRequired && DebugDrawer)
		{
			FScopedSlowTask SlowTask(0.0f, LOCTEXT("ReinitializingDebugDrawData", "Reinitializing Debug Draw"));
			SlowTask.MakeDialog();

			DebugDrawer->InitializeDrawDebug(Database, QueryRanges);
		}

		// Update Selection

		ActiveRanges.Reset();

		const int32 QueryRangeNum = QueryEntries.Num();

		for (int32 QueryRangeIdx = 0; QueryRangeIdx < QueryRangeNum; QueryRangeIdx++)
		{
			if (QueryEntries[QueryRangeIdx]->bIsSelected)
			{
				ActiveRanges.Add(QueryRangeIdx);
			}
		}

		if (bQueryUpdateRequired || SelectedRanges != ActiveRanges)
		{
			SelectedRanges = ActiveRanges;
			SelectedFrameRanges = UAnimDatabaseFrameRangesLibrary::FrameRangesGatherRangesFromIndexSet(QueryRanges, SelectedRanges);
		}

		// Return if query update required

		return bQueryUpdateRequired;
	}
		
	return false;
}

void UAnimDatabaseQuery::RunFunction()
{
	if (!FunctionObject)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "RunFunction: No function selected.");
		return;
	}

	if (Database)
	{
		FScopedSlowTask SlowTask(0.0f, LOCTEXT("RunningDatabaseFunction", "Running Database Function"));
		SlowTask.MakeDialog();

		switch (FunctionRanges)
		{
		case EAnimDatabaseFunctionQueryRanges::Database:
		{
			FunctionObject->Run(Database, UAnimDatabaseFrameRangesLibrary::FrameRangesAll(Database), bFunctionShouldTransact);
			return;
		}

		case EAnimDatabaseFunctionQueryRanges::Query:
		{
			FunctionObject->Run(Database, QueryRanges, bFunctionShouldTransact);
			return;
		}

		case EAnimDatabaseFunctionQueryRanges::Selection:
		{
			FunctionObject->Run(Database, SelectedFrameRanges, bFunctionShouldTransact);
			return;
		}

		case EAnimDatabaseFunctionQueryRanges::Custom:
		{
			const FAnimDatabaseFrameRanges CustomRanges = UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFunction(Database, UAnimDatabaseFrameRangesLibrary::FrameRangesAll(Database), CustomFrameRanges);
			if (CustomRanges.IsValid())
			{
				FunctionObject->Run(Database, CustomRanges, bFunctionShouldTransact);
			}
			return;
		}

		default:
			checkNoEntry();
		}
	}
}

void UAnimDatabaseDebugDraw::InitializeDrawDebug_Implementation(const UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges) {}

void UAnimDatabaseDebugDraw::DrawDebug_Implementation(
	const FDebugDrawer& Drawer,
	const FDebugDrawer& CanvasDrawer,
	const FAnimDatabasePoseState& InPoseState,
	const UAnimDatabase* InDatabase,
	const FAnimDatabaseFrameRanges& InFrameRanges,
	const FTransform& RangeViewportTransform,
	const int32 CharacterIdx,
	const int32 SequenceIdx,
	const float SequenceTime,
	const int32 RangeStart,
	const int32 RangeLength,
	const FLinearColor& IdentifierColor) {}

#endif

UAnimDatabase::UAnimDatabase(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
#if WITH_EDITOR
	ViewportSettings = ObjectInitializer.CreateEditorOnlyDefaultSubobject<UAnimDatabaseViewportSettings>(this, TEXT("ViewportSettings"));
	Query = ObjectInitializer.CreateEditorOnlyDefaultSubobject<UAnimDatabaseQuery>(this, TEXT("Query"));
#endif
}

#if WITH_EDITOR

void UAnimDatabase::PostEditChangeProperty(struct FPropertyChangedEvent& Event)
{
	if (Event.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UAnimDatabase, Skeleton))
	{
		// If the skeleton changes then we want to reset the mirror table and entries and issue a callback
		MirrorDataTable = nullptr;
		Entries.Reset();
		OnSkeletonChanged.Broadcast();
	}
	else if (Event.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UAnimDatabase, MirrorDataTable))
	{
		// If the mirror data table changes we want to reset it if it is not compatible with the given skeleton and/or
		// update the Skeleton field if it is empty.
		if (MirrorDataTable && MirrorDataTable->Skeleton && !Skeleton)
		{
			Skeleton = MirrorDataTable->Skeleton;
		}
		else if (MirrorDataTable && MirrorDataTable->Skeleton != Skeleton)
		{
			MirrorDataTable = nullptr;
		}
	}
	else if (Event.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UAnimDatabase, Entries))
	{
		// If the entries change we want to remove any which are not compatible with the given skeleton and/or
		// update the Skeleton field if it is empty.
		const int32 EntryNum = Entries.Num();
		for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
		{
			const UAnimSequence* AnimSequence = Entries[EntryIdx];

			if (AnimSequence && AnimSequence->GetSkeleton() && !Skeleton)
			{
				Skeleton = AnimSequence->GetSkeleton();
			}
			else if (AnimSequence && AnimSequence->GetSkeleton() != Skeleton)
			{
				Entries[EntryIdx] = nullptr;
			}
		}
	}
}

#endif

int32 UAnimDatabase::GetContentHash() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabase::GetContentHash);

	const FString& SkeletonPackageName = Skeleton.GetPath();
	const FString& MirrorTablePackageName = MirrorDataTable.GetPath();

	const int32 SkeletonHash = CityHash32(
		(const char*)SkeletonPackageName.GetCharArray().GetData(),
		SkeletonPackageName.GetCharArray().GetTypeSize() *
		SkeletonPackageName.GetCharArray().Num());

	const int32 MirrorTableHash = CityHash32(
		(const char*)MirrorTablePackageName.GetCharArray().GetData(),
		MirrorTablePackageName.GetCharArray().GetTypeSize() *
		MirrorTablePackageName.GetCharArray().Num());

	const int32 FrameRateHash = CityHash32((const char*)&FrameRate, sizeof(FrameRate));
	const int32 PoseSamplerHash = CityHash32((const char*)&PoseSampler, sizeof(PoseSampler));
	const int32 CurveSamplerHash = CityHash32((const char*)&CurveSampler, sizeof(CurveSampler));

	int32 EntriesHash = 0xac92453a;

	const int32 EntryNum = Entries.Num();

	for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
	{
		UAnimSequence* Sequence = Entries[EntryIdx];

		int32 SequenceHash = 0;
		if (Sequence)
		{
			// Hash Animation Data

#if WITH_EDITOR
			Sequence->WaitOnExistingCompression(true);
			const FIoHash EntryIoHash = Sequence->GetDerivedDataKeyHash(nullptr);
			const int32 AnimDataHash = CityHash32((const char*)&EntryIoHash, sizeof(EntryIoHash));
#else
			const int32 AnimDataHash = 0x7caf490a; // Symbolic random hash value to represent AnimSequence outside of editor
#endif

			// Hash Root Settings

			const int32 RootMotionSettings[3] = { Sequence->bEnableRootMotion, Sequence->bForceRootLock, Sequence->RootMotionRootLock };
			const int32 RootMotionSettingsHash = CityHash32((const char*)&RootMotionSettings, sizeof(RootMotionSettings));

			// Hash Notifies

			int32 NotifiesHash = 0x8da39859;
			
			const int32 NotifyNum = Sequence->Notifies.Num();

			for (int32 NotifyIdx = 0; NotifyIdx < NotifyNum; NotifyIdx++)
			{
				const FAnimNotifyEvent& Notify = Sequence->Notifies[NotifyIdx];

				struct FAnimDatabaseNotifyHashData
				{
					int32 Index = 0;
					FName ClassName = NAME_None;
					float Time = 0.0f;
					float Duration = 0.0f;
				};

				FAnimDatabaseNotifyHashData NotifyHashData;
				FMemory::Memzero(&NotifyHashData, sizeof(NotifyHashData));
				NotifyHashData.Index = NotifyIdx;
				NotifyHashData.ClassName =
					Notify.Notify ? Notify.Notify->GetClass()->GetFName() :
					Notify.NotifyStateClass ? Notify.NotifyStateClass->GetClass()->GetFName() : NAME_None;
				NotifyHashData.Time = Notify.GetTime();
				NotifyHashData.Duration = Notify.GetDuration();

				NotifiesHash ^= CityHash32((const char*)&NotifyHashData, sizeof(NotifyHashData));
			}

			// Combine Hashes

			const int32 SequenceData[3] = { AnimDataHash, RootMotionSettingsHash, NotifiesHash };
			SequenceHash = CityHash32((const char*)&SequenceData, sizeof(SequenceData));
		}
		else
		{
			SequenceHash = 0xcd44cc84;
		}

		const int32 EntryData[2] = { EntryIdx, SequenceHash, };
		EntriesHash ^= CityHash32((const char*)&EntryData, sizeof(EntryData));
	}

	const int32 DatabaseProperties[6] = { SkeletonHash, MirrorTableHash, FrameRateHash, PoseSamplerHash, CurveSamplerHash, EntriesHash };
	return CityHash32((const char*)&DatabaseProperties, sizeof(DatabaseProperties));
}

int32 UAnimDatabase::GetBoneNum() const
{
	return Skeleton ? Skeleton->GetReferenceSkeleton().GetNum() : 0;
}

int32 UAnimDatabase::FindBoneIndex(const FName BoneName) const
{
	return Skeleton ? Skeleton->GetReferenceSkeleton().FindBoneIndex(BoneName) : INDEX_NONE;
}

FName UAnimDatabase::GetBoneName(const int32 BoneIndex) const
{
	return Skeleton ? Skeleton->GetReferenceSkeleton().GetBoneName(BoneIndex) : NAME_None;
}

void UAnimDatabase::GetBoneNames(TArray<FName>& OutBoneNames) const
{
	OutBoneNames.SetNumUninitialized(GetBoneNum());
	GetBoneNamesToArrayView(OutBoneNames);
}

void UAnimDatabase::GetBoneNamesToArrayView(TArrayView<FName> OutBoneNames) const
{
	const int32 BoneNum = GetBoneNum();
	check(BoneNum == OutBoneNames.Num());
	for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
	{
		OutBoneNames[BoneIdx] = GetBoneName(BoneIdx);
	}
}

void UAnimDatabase::FindBoneIndices(TArray<int32>& OutBoneIndices, const TArray<FName>& BoneNames) const
{
	OutBoneIndices.SetNumUninitialized(BoneNames.Num());
	FindBoneIndicesFromArrayViews(OutBoneIndices, BoneNames);
}

void UAnimDatabase::FindBoneIndicesFromArrayViews(const TArrayView<int32> OutBoneIndices, const TArrayView<const FName> BoneNames) const
{
	check(OutBoneIndices.Num() == BoneNames.Num());
	const int32 BoneNum = BoneNames.Num();
	for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
	{
		OutBoneIndices[BoneIdx] = FindBoneIndex(BoneNames[BoneIdx]);
	}
}

int32 UAnimDatabase::GetSequenceNum() const
{
	return MirrorDataTable ? Entries.Num() * 2 : Entries.Num();
}

int32 UAnimDatabase::FindSequenceIndex(UAnimSequence* AnimSequence, const bool bIsMirrored) const
{
	const int32 AnimSequenceArrayIndex = Entries.Find(AnimSequence);
	return AnimSequenceArrayIndex == INDEX_NONE ?
		INDEX_NONE :
		MirrorDataTable && bIsMirrored ? AnimSequenceArrayIndex + Entries.Num() : AnimSequenceArrayIndex;
}

FFrameRate UAnimDatabase::GetFrameRate() const
{
	return FrameRate;
}

USkeleton* UAnimDatabase::GetSkeleton() const
{
	return Skeleton;
}

UMirrorDataTable* UAnimDatabase::GetMirrorDataTable() const
{
	return MirrorDataTable;
}

UAnimSequence* UAnimDatabase::GetAnimSequence(const int32 SequenceIdx) const
{
	const int32 SequenceNum = GetSequenceNum();

	if (SequenceIdx < 0 || SequenceIdx >= SequenceNum)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "UAnimDatabase::GetAnimSequence: Invalid Sequence Index {Index}, must be >= 0 and < {Num}.", SequenceIdx, SequenceNum);
		return nullptr;
	}

	if (MirrorDataTable)
	{
		const int32 EntryNum = Entries.Num();

		return SequenceIdx >= EntryNum ?
			Entries[SequenceIdx - EntryNum] :
			Entries[SequenceIdx];
	}
	else
	{
		return Entries[SequenceIdx];
	}
}

void UAnimDatabase::WaitForCompressionOnAnimSequence(const int32 SequenceIdx) const
{
#if WITH_EDITOR
	if (IsInGameThread())
	{
		if (UAnimSequence* Sequence = GetAnimSequence(SequenceIdx))
		{
			Sequence->WaitOnExistingCompression(true);
			ensure(!Sequence->IsCompressedDataOutOfDate());
		}
	}
#endif
}

void UAnimDatabase::WaitForCompressionOnAnimSequences(const TArray<int32>& SequenceIndices) const
{
	WaitForCompressionOnAnimSequencesFromArrayView(SequenceIndices);
}

void UAnimDatabase::WaitForCompressionOnAnimSequencesFromArrayView(const TArrayView<const int32> SequenceIndices) const
{
	for (const int32 SequenceIdx : SequenceIndices)
	{
		WaitForCompressionOnAnimSequence(SequenceIdx);
	}
}

void UAnimDatabase::WaitForCompressionOnAll() const
{
	const int32 SequenceNum = GetSequenceNum();
	for (int32 SequenceIdx = 0; SequenceIdx < SequenceNum; SequenceIdx++)
	{
		WaitForCompressionOnAnimSequence(SequenceIdx);
	}
}

bool UAnimDatabase::GetIsMirrored(const int32 SequenceIdx) const
{
	const int32 EntryNum = Entries.Num();

	if (MirrorDataTable)
	{
		check(SequenceIdx >= 0 && SequenceIdx < EntryNum * 2);
		return SequenceIdx >= EntryNum;
	}

	check(SequenceIdx >= 0 && SequenceIdx < EntryNum);
	return false;
}

int32 UAnimDatabase::GetSequenceFrameNum(const int32 SequenceIdx) const
{
	if (const UAnimSequence* AnimSequence = GetAnimSequence(SequenceIdx))
	{
		const FFrameRate SequenceFrameRate = AnimSequence->GetSamplingFrameRate();
		const int32 SequenceFrameNum = FMath::Max(AnimSequence->GetNumberOfSampledKeys() - 1, 0);
		return (SequenceFrameNum * FrameRate.Numerator * SequenceFrameRate.Denominator) / FMath::Max(FrameRate.Denominator * SequenceFrameRate.Numerator, 1) + 1;
	}

	return 0;
}

float UAnimDatabase::GetTotalDuration() const
{
	float Total = 0.0f;
	const int32 SequenceNum = GetSequenceNum();
	for (int32 SequenceIdx = 0; SequenceIdx < SequenceNum; SequenceIdx++)
	{
		Total += GetSequenceDuration(SequenceIdx);
	}
	return Total;
}

int32 UAnimDatabase::GetTotalFrameNum() const
{
	int32 Total = 0;
	const int32 SequenceNum = GetSequenceNum();
	for (int32 SequenceIdx = 0; SequenceIdx < SequenceNum; SequenceIdx++)
	{
		Total += GetSequenceFrameNum(SequenceIdx);
	}
	return Total;
}

float UAnimDatabase::GetSequenceTimeFromFrame(const int32 SequenceIdx, const int32 FrameIdx) const
{
	return FMath::Clamp(FrameIdx, 0, FMath::Max(GetSequenceFrameNum(SequenceIdx) - 1, 0)) / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER);
}

int32 UAnimDatabase::GetClosestFrameFromSequenceTime(const int32 SequenceIdx, const float SequenceTime) const
{
	return FMath::Clamp(FMath::RoundToInt(SequenceTime * FrameRate.AsDecimal()), 0, FMath::Max(GetSequenceFrameNum(SequenceIdx) - 1, 0));
}

float UAnimDatabase::GetSequenceDuration(const int32 SequenceIdx) const
{
	const int32 FrameNum = GetSequenceFrameNum(SequenceIdx);
	check(FrameNum >= 0);
	return FrameNum == 0 ? 0.0f : (FrameNum - 1) / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER);
}

FString UAnimDatabase::GetSequenceAssetName(const int32 SequenceIdx) const
{
	if (UAnimSequence* Sequence = GetAnimSequence(SequenceIdx))
	{
		return Sequence->GetName();
	}
	else
	{
		return FString();
	}
}

FString UAnimDatabase::GetSequencePathString(const int32 SequenceIdx) const
{
	if (UAnimSequence* Sequence = GetAnimSequence(SequenceIdx))
	{
		return Sequence->GetPathName(this);
	}
	else
	{
		return FString();
	}
}

int32 UAnimDatabase::GetBoneParent(const int32 BoneIndex) const
{
	if (BoneIndex < 0 || BoneIndex >= GetBoneNum())
	{
		return INDEX_NONE;
	}

	return Skeleton->GetReferenceSkeleton().GetParentIndex(BoneIndex);
}

void UAnimDatabase::GetBoneParents(TArray<int32>& OutParents) const
{
	if (!Skeleton) { OutParents.Empty(); return; }
	OutParents.SetNumUninitialized(GetBoneNum());
	GetBoneParentsToArrayView(OutParents);
}

void UAnimDatabase::GetBoneParentsToArrayView(const TArrayView<int32> OutParents) const
{
	check(Skeleton);

	const int32 BoneNum = GetBoneNum();
	check(OutParents.Num() == BoneNum);

	for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
	{
		OutParents[BoneIdx] = Skeleton->GetReferenceSkeleton().GetParentIndex(BoneIdx);
	}
}

FVector UAnimDatabase::GetBoneReferenceLocation(const int32 BoneIdx)
{
	if (!Skeleton || BoneIdx < 0 || BoneIdx >= Skeleton->GetReferenceSkeleton().GetNum())
	{
		return FVector::ZeroVector;
	}

	return Skeleton->GetReferenceSkeleton().GetRefBonePose()[BoneIdx].GetLocation();
}

void UAnimDatabase::GetBoneReferenceLocations(TArray<FVector>& OutReferenceLocations)
{
	OutReferenceLocations.SetNumUninitialized(GetBoneNum());
	GetBoneReferenceLocationsToArrayView(OutReferenceLocations);
}

void UAnimDatabase::GetBoneReferenceLocationsToArrayView(TArrayView<FVector> OutReferenceLocations)
{
	const int32 BoneNum = GetBoneNum();
	check(OutReferenceLocations.Num() == BoneNum);
	for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
	{
		OutReferenceLocations[BoneIdx] = GetBoneReferenceLocation(BoneIdx);
	}
}

FQuat UAnimDatabase::GetBoneReferenceRotation(const int32 BoneIdx)
{
	if (!Skeleton || BoneIdx < 0 || BoneIdx >= Skeleton->GetReferenceSkeleton().GetNum())
	{
		return FQuat::Identity;
	}

	return Skeleton->GetReferenceSkeleton().GetRefBonePose()[BoneIdx].GetRotation();
}

void UAnimDatabase::GetBoneReferenceRotations(TArray<FQuat>& OutReferenceRotations)
{
	OutReferenceRotations.SetNumUninitialized(GetBoneNum());
	GetBoneReferenceRotationsToArrayView(OutReferenceRotations);
}

void UAnimDatabase::GetBoneReferenceRotationsToArrayView(TArrayView<FQuat> OutReferenceRotations)
{
	const int32 BoneNum = GetBoneNum();
	check(OutReferenceRotations.Num() == BoneNum);
	for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
	{
		OutReferenceRotations[BoneIdx] = GetBoneReferenceRotation(BoneIdx);
	}
}

FVector UAnimDatabase::GetBoneReferenceScale(const int32 BoneIdx)
{
	if (!Skeleton || BoneIdx < 0 || BoneIdx >= Skeleton->GetReferenceSkeleton().GetNum())
	{
		return FVector::OneVector;
	}

	return Skeleton->GetReferenceSkeleton().GetRefBonePose()[BoneIdx].GetScale3D();
}

void UAnimDatabase::GetRootTransform(const TLearningArrayView<1, FTransform> OutTransforms, const int32 SequenceIdx, const int32 FrameStart) const
{
	const int32 FrameNum = OutTransforms.Num();
	if (FrameNum == 0) { return; }

	WaitForCompressionOnAnimSequence(SequenceIdx);

	const UAnimSequence* AnimSequence = GetAnimSequence(SequenceIdx);

	if (!AnimSequence || AnimSequence->GetSkeleton() != Skeleton)
	{
		UE::Learning::Array::Set(OutTransforms, FTransform::Identity);
		return;
	}

	const UMirrorDataTable* MirrorDataTableObject = GetMirrorDataTable();
	const bool bIsMirrored = GetIsMirrored(SequenceIdx);

	float DatabaseToSequence = AnimSequence->GetSamplingFrameRate().AsDecimal() / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER);
	int32 SequenceStartFrame = INDEX_NONE, SequenceStopFrame = INDEX_NONE;
	UE::AnimDatabase::Private::ComputeExtractionRanges(
		SequenceStartFrame, SequenceStopFrame,
		DatabaseToSequence, FrameStart, FrameStart + FrameNum,
		AnimSequence->GetNumberOfSampledKeys());

	TLearningArray<1, FVector, TInlineAllocator<4>> RootLocations; RootLocations.SetNumUninitialized({ SequenceStopFrame - SequenceStartFrame });
	TLearningArray<1, FQuat4f, TInlineAllocator<4>> RootRotations; RootRotations.SetNumUninitialized({ SequenceStopFrame - SequenceStartFrame });
	TLearningArray<1, FVector3f, TInlineAllocator<4>> RootScales; RootScales.SetNumUninitialized({ SequenceStopFrame - SequenceStartFrame });
	UE::AnimDatabase::Private::ExtractRoot(RootLocations, RootRotations, RootScales, AnimSequence, SequenceStartFrame, MirrorDataTableObject, bIsMirrored);

	UE::AnimDatabase::Private::SampleSequenceTransforms(
		OutTransforms, RootLocations, RootRotations, RootScales,
		SequenceStartFrame, SequenceStopFrame, DatabaseToSequence, FrameStart, FrameStart + FrameNum,
		1.0f / FMath::Max(AnimSequence->GetSamplingFrameRate().AsDecimal(), UE_SMALL_NUMBER),
		PoseSampler);
}

void UAnimDatabase::GetRootLocation(const TLearningArrayView<1, FVector> OutLocations, const int32 SequenceIdx, const int32 FrameStart) const
{
	const int32 FrameNum = OutLocations.Num();
	if (FrameNum == 0) { return; }

	WaitForCompressionOnAnimSequence(SequenceIdx);

	const UAnimSequence* AnimSequence = GetAnimSequence(SequenceIdx);

	if (!AnimSequence || AnimSequence->GetSkeleton() != Skeleton)
	{
		UE::Learning::Array::Zero(OutLocations);
		return;
	}

	const UMirrorDataTable* MirrorDataTableObject = GetMirrorDataTable();
	const bool bIsMirrored = GetIsMirrored(SequenceIdx);

	if (AnimSequence->GetSamplingFrameRate() == FrameRate) // Sequence has the same frame rate as database
	{
		UE::AnimDatabase::Private::ExtractRootLocations(OutLocations, AnimSequence, FrameStart, MirrorDataTableObject, bIsMirrored);
		return;
	}

	float DatabaseToSequence = AnimSequence->GetSamplingFrameRate().AsDecimal() / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER);
	int32 SequenceStartFrame = INDEX_NONE, SequenceStopFrame = INDEX_NONE;
	UE::AnimDatabase::Private::ComputeExtractionRanges(
		SequenceStartFrame, SequenceStopFrame,
		DatabaseToSequence, FrameStart, FrameStart + FrameNum,
		AnimSequence->GetNumberOfSampledKeys());

	TLearningArray<1, FVector, TInlineAllocator<4>> RootLocations; RootLocations.SetNumUninitialized({ SequenceStopFrame - SequenceStartFrame });
	UE::AnimDatabase::Private::ExtractRootLocations(RootLocations, AnimSequence, SequenceStartFrame, MirrorDataTableObject, bIsMirrored);

	UE::AnimDatabase::Private::SampleSequenceLocations(
		OutLocations, RootLocations,
		SequenceStartFrame, SequenceStopFrame, DatabaseToSequence, FrameStart, FrameStart + FrameNum,
		1.0f / FMath::Max(AnimSequence->GetSamplingFrameRate().AsDecimal(), UE_SMALL_NUMBER),
		PoseSampler);
}

void UAnimDatabase::GetRootRotation(const TLearningArrayView<1, FQuat4f> OutRotations, const int32 SequenceIdx, const int32 FrameStart) const
{
	const int32 FrameNum = OutRotations.Num();
	if (FrameNum == 0) { return; }

	WaitForCompressionOnAnimSequence(SequenceIdx);

	const UAnimSequence* AnimSequence = GetAnimSequence(SequenceIdx);

	if (!AnimSequence || AnimSequence->GetSkeleton() != Skeleton)
	{
		UE::Learning::Array::Set(OutRotations, FQuat4f::Identity);
		return;
	}

	const UMirrorDataTable* MirrorDataTableObject = GetMirrorDataTable();
	const bool bIsMirrored = GetIsMirrored(SequenceIdx);

	if (AnimSequence->GetSamplingFrameRate() == FrameRate) // Sequence has the same frame rate as database
	{
		UE::AnimDatabase::Private::ExtractRootRotations(OutRotations, AnimSequence, FrameStart, MirrorDataTableObject, bIsMirrored);
		return;
	}

	float DatabaseToSequence = AnimSequence->GetSamplingFrameRate().AsDecimal() / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER);
	int32 SequenceStartFrame = INDEX_NONE, SequenceStopFrame = INDEX_NONE;
	UE::AnimDatabase::Private::ComputeExtractionRanges(
		SequenceStartFrame, SequenceStopFrame,
		DatabaseToSequence, FrameStart, FrameStart + FrameNum,
		AnimSequence->GetNumberOfSampledKeys());

	TLearningArray<1, FQuat4f> RootRotations; RootRotations.SetNumUninitialized({ SequenceStopFrame - SequenceStartFrame });
	UE::AnimDatabase::Private::ExtractRootRotations(RootRotations, AnimSequence, SequenceStartFrame, MirrorDataTableObject, bIsMirrored);

	UE::AnimDatabase::Private::SampleSequenceRotations(
		OutRotations, RootRotations,
		SequenceStartFrame, SequenceStopFrame, DatabaseToSequence, FrameStart, FrameStart + FrameNum,
		1.0f / FMath::Max(AnimSequence->GetSamplingFrameRate().AsDecimal(), UE_SMALL_NUMBER),
		PoseSampler);
}

void UAnimDatabase::GetRootDirection(const TLearningArrayView<1, FVector3f> OutDirections, const int32 SequenceIdx, const int32 FrameStart, const FVector3f ForwardVector) const
{
	const int32 FrameNum = OutDirections.Num();
	if (FrameNum == 0) { return; }

	WaitForCompressionOnAnimSequence(SequenceIdx);

	const UAnimSequence* AnimSequence = GetAnimSequence(SequenceIdx);

	if (!AnimSequence || AnimSequence->GetSkeleton() != Skeleton)
	{
		UE::Learning::Array::Set(OutDirections, ForwardVector);
		return;
	}

	const UMirrorDataTable* MirrorDataTableObject = GetMirrorDataTable();
	const bool bIsMirrored = GetIsMirrored(SequenceIdx);

	if (AnimSequence->GetSamplingFrameRate() == FrameRate)
	{
		UE::AnimDatabase::Private::ExtractRootDirections(OutDirections, AnimSequence, FrameStart, MirrorDataTableObject, bIsMirrored, ForwardVector);
		return;
	}

	float DatabaseToSequence = AnimSequence->GetSamplingFrameRate().AsDecimal() / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER);
	int32 SequenceStartFrame = INDEX_NONE, SequenceStopFrame = INDEX_NONE;
	UE::AnimDatabase::Private::ComputeExtractionRanges(
		SequenceStartFrame, SequenceStopFrame,
		DatabaseToSequence, FrameStart, FrameStart + FrameNum,
		AnimSequence->GetNumberOfSampledKeys());

	TLearningArray<1, FQuat4f> RootRotations; RootRotations.SetNumUninitialized({ SequenceStopFrame - SequenceStartFrame });
	UE::AnimDatabase::Private::ExtractRootRotations(RootRotations, AnimSequence, SequenceStartFrame, MirrorDataTableObject, bIsMirrored);

	UE::AnimDatabase::Private::SampleSequenceDirectionsFromRotations(
		OutDirections, RootRotations,
		SequenceStartFrame, SequenceStopFrame, DatabaseToSequence, FrameStart, FrameStart + FrameNum,
		1.0f / FMath::Max(AnimSequence->GetSamplingFrameRate().AsDecimal(), UE_SMALL_NUMBER), ForwardVector, 
		PoseSampler);
}

void UAnimDatabase::GetRootLinearVelocity(const TLearningArrayView<1, FVector3f> OutLinearVelocities, const int32 SequenceIdx, const int32 FrameStart) const
{
	const int32 FrameNum = OutLinearVelocities.Num();
	if (FrameNum == 0) { return; }

	WaitForCompressionOnAnimSequence(SequenceIdx);

	const UAnimSequence* AnimSequence = GetAnimSequence(SequenceIdx);

	if (!AnimSequence || AnimSequence->GetSkeleton() != Skeleton)
	{
		UE::Learning::Array::Zero(OutLinearVelocities);
		return;
	}

	const UMirrorDataTable* MirrorDataTableObject = GetMirrorDataTable();
	const bool bIsMirrored = GetIsMirrored(SequenceIdx);

	float DatabaseToSequence = AnimSequence->GetSamplingFrameRate().AsDecimal() / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER);
	int32 SequenceStartFrame = INDEX_NONE, SequenceStopFrame = INDEX_NONE;
	UE::AnimDatabase::Private::ComputeExtractionRanges(
		SequenceStartFrame, SequenceStopFrame,
		DatabaseToSequence, FrameStart, FrameStart + FrameNum,
		AnimSequence->GetNumberOfSampledKeys());

	TLearningArray<1, FVector> RootLocations; RootLocations.SetNumUninitialized({ SequenceStopFrame - SequenceStartFrame });
	UE::AnimDatabase::Private::ExtractRootLocations(RootLocations, AnimSequence, SequenceStartFrame, MirrorDataTableObject, bIsMirrored);

	UE::AnimDatabase::Private::SampleSequenceLinearVelocities(
		OutLinearVelocities, RootLocations,
		SequenceStartFrame, SequenceStopFrame, DatabaseToSequence, FrameStart, FrameStart + FrameNum,
		1.0f / FMath::Max(AnimSequence->GetSamplingFrameRate().AsDecimal(), UE_SMALL_NUMBER),
		PoseSampler);
}

void UAnimDatabase::GetRootAngularVelocity(const TLearningArrayView<1, FVector3f> OutAngularVelocities, const int32 SequenceIdx, const int32 FrameStart) const
{
	const int32 FrameNum = OutAngularVelocities.Num();
	if (FrameNum == 0) { return; }

	WaitForCompressionOnAnimSequence(SequenceIdx);

	const UAnimSequence* AnimSequence = GetAnimSequence(SequenceIdx);

	if (!AnimSequence || AnimSequence->GetSkeleton() != Skeleton)
	{
		UE::Learning::Array::Zero(OutAngularVelocities);
		return;
	}

	const UMirrorDataTable* MirrorDataTableObject = GetMirrorDataTable();
	const bool bIsMirrored = GetIsMirrored(SequenceIdx);

	float DatabaseToSequence = AnimSequence->GetSamplingFrameRate().AsDecimal() / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER);
	int32 SequenceStartFrame = INDEX_NONE, SequenceStopFrame = INDEX_NONE;
	UE::AnimDatabase::Private::ComputeExtractionRanges(
		SequenceStartFrame, SequenceStopFrame,
		DatabaseToSequence, FrameStart, FrameStart + FrameNum,
		AnimSequence->GetNumberOfSampledKeys());

	TLearningArray<1, FQuat4f> RootRotations; RootRotations.SetNumUninitialized({ SequenceStopFrame - SequenceStartFrame });
	UE::AnimDatabase::Private::ExtractRootRotations(RootRotations, AnimSequence, SequenceStartFrame, MirrorDataTableObject, bIsMirrored);

	UE::AnimDatabase::Private::SampleSequenceAngularVelocities(
		OutAngularVelocities, RootRotations,
		SequenceStartFrame, SequenceStopFrame, DatabaseToSequence, FrameStart, FrameStart + FrameNum,
		1.0f / FMath::Max(AnimSequence->GetSamplingFrameRate().AsDecimal(), UE_SMALL_NUMBER),
		PoseSampler);
}

void UAnimDatabase::GetPoseRootData(
	const UE::AnimDatabase::FPoseRootDataView& OutPoseRootData,
	const int32 SequenceIdx,
	const int32 FrameStart) const
{
	const int32 FrameNum = OutPoseRootData.GetFrameNum();
	if (FrameNum == 0) { return; }

	WaitForCompressionOnAnimSequence(SequenceIdx);

	const UAnimSequence* AnimSequence = GetAnimSequence(SequenceIdx);

	if (!AnimSequence || AnimSequence->GetSkeleton() != Skeleton)
	{
		UE::AnimDatabase::PoseData::Reset(OutPoseRootData);
		return;
	}

	const UMirrorDataTable* MirrorDataTableObject = GetMirrorDataTable();
	const bool bIsMirrored = GetIsMirrored(SequenceIdx);

	float DatabaseToSequence = AnimSequence->GetSamplingFrameRate().AsDecimal() / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER);
	int32 SequenceStartFrame = INDEX_NONE, SequenceStopFrame = INDEX_NONE;
	UE::AnimDatabase::Private::ComputeExtractionRanges(
		SequenceStartFrame, SequenceStopFrame,
		DatabaseToSequence, FrameStart, FrameStart + FrameNum,
		AnimSequence->GetNumberOfSampledKeys());

	TLearningArray<1, FVector, TInlineAllocator<4>> RootLocations; RootLocations.SetNumUninitialized({ SequenceStopFrame - SequenceStartFrame });
	TLearningArray<1, FQuat4f, TInlineAllocator<4>> RootRotations; RootRotations.SetNumUninitialized({ SequenceStopFrame - SequenceStartFrame });
	TLearningArray<1, FVector3f, TInlineAllocator<4>> RootScales; RootScales.SetNumUninitialized({ SequenceStopFrame - SequenceStartFrame });
	UE::AnimDatabase::Private::ExtractRoot(RootLocations, RootRotations, RootScales, AnimSequence, SequenceStartFrame, MirrorDataTableObject, bIsMirrored);

	UE::AnimDatabase::Private::SampleSequenceTransformsVelocities(
		OutPoseRootData.RootLocations,
		OutPoseRootData.RootRotations,
		OutPoseRootData.RootScales,
		OutPoseRootData.RootLinearVelocities,
		OutPoseRootData.RootAngularVelocities,
		OutPoseRootData.RootScalarVelocities,
		RootLocations, RootRotations, RootScales,
		SequenceStartFrame, SequenceStopFrame, DatabaseToSequence, FrameStart, FrameStart + FrameNum,
		1.0f / FMath::Max(AnimSequence->GetSamplingFrameRate().AsDecimal(), UE_SMALL_NUMBER),
		PoseSampler);
}

void UAnimDatabase::GetPoseLocalBoneData(
	const UE::AnimDatabase::FPoseLocalBoneDataView& OutPoseLocalBoneData,
	const int32 SequenceIdx,
	const int32 FrameStart) const
{
	GetPoseLocalBoneSubsetData(OutPoseLocalBoneData, SequenceIdx, FrameStart, UE::Learning::FIndexSet(0, OutPoseLocalBoneData.GetBoneNum()));
}

void UAnimDatabase::GetPoseLocalBoneSubsetData(
	const UE::AnimDatabase::FPoseLocalBoneDataView& OutPoseLocalBoneData,
	const int32 SequenceIdx,
	const int32 FrameStart,
	const UE::Learning::FIndexSet BoneIndices) const
{
	const int32 FrameNum = OutPoseLocalBoneData.GetFrameNum();
	const int32 BoneNum = BoneIndices.Num();
	if (FrameNum == 0 || BoneNum == 0) { return; }

	WaitForCompressionOnAnimSequence(SequenceIdx);

	const UAnimSequence* AnimSequence = GetAnimSequence(SequenceIdx);

	if (!AnimSequence || AnimSequence->GetSkeleton() != Skeleton)
	{
		UE::AnimDatabase::PoseData::Reset(OutPoseLocalBoneData, Skeleton->GetReferenceSkeleton().GetRefBonePose());
		return;
	}

	const UMirrorDataTable* MirrorDataTableObject = GetMirrorDataTable();
	const bool bIsMirrored = GetIsMirrored(SequenceIdx);

	float DatabaseToSequence = AnimSequence->GetSamplingFrameRate().AsDecimal() / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER);
	int32 SequenceStartFrame = INDEX_NONE, SequenceStopFrame = INDEX_NONE;
	UE::AnimDatabase::Private::ComputeExtractionRanges(
		SequenceStartFrame, SequenceStopFrame,
		DatabaseToSequence, FrameStart, FrameStart + FrameNum,
		AnimSequence->GetNumberOfSampledKeys());

	TLearningArray<2, FVector3f> BoneLocations; BoneLocations.SetNumUninitialized({ SequenceStopFrame - SequenceStartFrame, BoneNum });
	TLearningArray<2, FQuat4f> BoneRotations; BoneRotations.SetNumUninitialized({ SequenceStopFrame - SequenceStartFrame, BoneNum });
	TLearningArray<2, FVector3f> BoneScales; BoneScales.SetNumUninitialized({ SequenceStopFrame - SequenceStartFrame, BoneNum });
	UE::AnimDatabase::Private::ExtractBoneTransforms(BoneLocations, BoneRotations, BoneScales, AnimSequence, SequenceStartFrame, MirrorDataTableObject, bIsMirrored, BoneIndices);

	UE::AnimDatabase::Private::SampleSequenceTransformsVelocities(
		OutPoseLocalBoneData.BoneLocations,
		OutPoseLocalBoneData.BoneRotations,
		OutPoseLocalBoneData.BoneScales,
		OutPoseLocalBoneData.BoneLinearVelocities,
		OutPoseLocalBoneData.BoneAngularVelocities,
		OutPoseLocalBoneData.BoneScalarVelocities,
		BoneLocations, BoneRotations, BoneScales,
		SequenceStartFrame, SequenceStopFrame, DatabaseToSequence, FrameStart, FrameStart + FrameNum,
		1.0f / FMath::Max(AnimSequence->GetSamplingFrameRate().AsDecimal(), UE_SMALL_NUMBER),
		PoseSampler);
}

void UAnimDatabase::GetCurveActiveData(
	const TLearningArrayView<2, bool> OutCurveActive,
	const int32 SequenceIdx,
	const int32 FrameStart,
	const TArrayView<const FName> CurveNames) const
{
	const int32 FrameNum = OutCurveActive.Num<0>();
	const int32 CurveNum = OutCurveActive.Num<1>();
	if (FrameNum == 0 || CurveNum == 0) { return; }

	WaitForCompressionOnAnimSequence(SequenceIdx);

	const UAnimSequence* AnimSequence = GetAnimSequence(SequenceIdx);

	if (!AnimSequence || AnimSequence->GetSkeleton() != Skeleton)
	{
		UE::Learning::Array::Zero(OutCurveActive);
		return;
	}

	const UMirrorDataTable* MirrorDataTableObject = GetMirrorDataTable();
	const bool bIsMirrored = GetIsMirrored(SequenceIdx);

	if (AnimSequence->GetSamplingFrameRate() == FrameRate)
	{
		UE::AnimDatabase::Private::ExtractCurveActiveData(OutCurveActive, AnimSequence, FrameStart, MirrorDataTableObject, bIsMirrored, CurveNames);
		return;
	}

	float DatabaseToSequence = AnimSequence->GetSamplingFrameRate().AsDecimal() / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER);
	int32 SequenceStartFrame = INDEX_NONE, SequenceStopFrame = INDEX_NONE;
	UE::AnimDatabase::Private::ComputeExtractionRanges(
		SequenceStartFrame, SequenceStopFrame,
		DatabaseToSequence, FrameStart, FrameStart + FrameNum,
		AnimSequence->GetNumberOfSampledKeys());

	TLearningArray<2, bool> CurveActive; CurveActive.SetNumUninitialized({ SequenceStopFrame - SequenceStartFrame, CurveNum });
	UE::AnimDatabase::Private::ExtractCurveActiveData(CurveActive, AnimSequence, SequenceStartFrame, MirrorDataTableObject, bIsMirrored, CurveNames);

	UE::AnimDatabase::Private::SampleSequenceCurveActive(
		OutCurveActive, CurveActive,
		SequenceStartFrame, SequenceStopFrame, DatabaseToSequence, FrameStart, FrameStart + FrameNum,
		1.0f / FMath::Max(AnimSequence->GetSamplingFrameRate().AsDecimal(), UE_SMALL_NUMBER));
}

void UAnimDatabase::GetCurveData(
	const TLearningArrayView<2, float> OutCurveValues,
	const TLearningArrayView<2, float> OutCurveVelocities,
	const TLearningArrayView<2, bool> OutCurveActive,
	const int32 SequenceIdx,
	const int32 FrameStart,
	const TArrayView<const FName> CurveNames) const
{
	const int32 FrameNum = OutCurveActive.Num<0>();
	const int32 CurveNum = OutCurveActive.Num<1>();
	if (FrameNum == 0 || CurveNum == 0) { return; }

	WaitForCompressionOnAnimSequence(SequenceIdx);

	const UAnimSequence* AnimSequence = GetAnimSequence(SequenceIdx);

	if (!AnimSequence || AnimSequence->GetSkeleton() != Skeleton)
	{
		UE::Learning::Array::Zero(OutCurveValues);
		UE::Learning::Array::Zero(OutCurveVelocities);
		UE::Learning::Array::Zero(OutCurveActive);
		return;
	}

	const UMirrorDataTable* MirrorDataTableObject = GetMirrorDataTable();
	const bool bIsMirrored = GetIsMirrored(SequenceIdx);

	float DatabaseToSequence = AnimSequence->GetSamplingFrameRate().AsDecimal() / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER);
	int32 SequenceStartFrame = INDEX_NONE, SequenceStopFrame = INDEX_NONE;
	UE::AnimDatabase::Private::ComputeExtractionRanges(
		SequenceStartFrame, SequenceStopFrame,
		DatabaseToSequence, FrameStart, FrameStart + FrameNum,
		AnimSequence->GetNumberOfSampledKeys());

	TLearningArray<2, float> CurveValue; CurveValue.SetNumUninitialized({ SequenceStopFrame - SequenceStartFrame, CurveNum });
	TLearningArray<2, bool> CurveActive; CurveActive.SetNumUninitialized({ SequenceStopFrame - SequenceStartFrame, CurveNum });
	UE::AnimDatabase::Private::ExtractCurveData(CurveValue, CurveActive, AnimSequence, SequenceStartFrame, MirrorDataTableObject, bIsMirrored, CurveNames);

	UE::AnimDatabase::Private::SampleSequenceCurveActive(
		OutCurveActive, CurveActive,
		SequenceStartFrame, SequenceStopFrame, DatabaseToSequence, FrameStart, FrameStart + FrameNum,
		1.0f / FMath::Max(AnimSequence->GetSamplingFrameRate().AsDecimal(), UE_SMALL_NUMBER));

	UE::AnimDatabase::Private::SampleSequenceCurveValue(
		OutCurveValues, OutCurveVelocities, CurveValue,
		SequenceStartFrame, SequenceStopFrame, DatabaseToSequence, FrameStart, FrameStart + FrameNum,
		1.0f / FMath::Max(AnimSequence->GetSamplingFrameRate().AsDecimal(), UE_SMALL_NUMBER),
		CurveSampler);
}

void UAnimDatabase::GetAttributeData(
	const UE::AnimDatabase::FPoseAttributeDataView& OutAttributeData,
	const int32 SequenceIdx,
	const int32 FrameStart,
	const TArrayView<const FAnimDatabaseFrameAttribute> FrameAttributes) const
{
	const int32 FrameAttributeNum = OutAttributeData.GetAttributeNum();
	check(FrameAttributeNum == FrameAttributes.Num());
	const int32 FrameNum = OutAttributeData.GetFrameNum();

	for (int32 FrameAttributeIdx = 0; FrameAttributeIdx < FrameAttributeNum; FrameAttributeIdx++)
	{
		check(FrameAttributes[FrameAttributeIdx].IsValid());
		check(OutAttributeData.GetAttributeSize(FrameAttributeIdx) == FrameAttributes[FrameAttributeIdx].FrameAttribute->GetChannelNum());
		check(OutAttributeData.GetAttributeType(FrameAttributeIdx) == FrameAttributes[FrameAttributeIdx].Type);
	}

	for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
	{
		for (int32 FrameAttributeIdx = 0; FrameAttributeIdx < FrameAttributeNum; FrameAttributeIdx++)
		{
			const int32 FrameAttributeOffset = OutAttributeData.GetAttributeOffset(FrameAttributeIdx);
			const int32 FrameAttributeSize = OutAttributeData.GetAttributeSize(FrameAttributeIdx);

			int32 EntryIdx = INDEX_NONE, RangeIdx = INDEX_NONE, RangeFrame = INDEX_NONE;
			OutAttributeData.AttributeActive[FrameIdx][FrameAttributeIdx] = FrameAttributes[FrameAttributeIdx].FrameAttribute->FrameRangeSet.Find(
				EntryIdx, RangeIdx, RangeFrame, SequenceIdx, FrameStart + FrameIdx);

			if (OutAttributeData.AttributeActive[FrameIdx][FrameAttributeIdx])
			{
				const int32 FrameOffset = FrameAttributes[FrameAttributeIdx].FrameAttribute->FrameRangeSet.GetEntryRangeOffset(EntryIdx, RangeIdx) + RangeFrame;
				for (int32 ChannelIdx = 0; ChannelIdx < FrameAttributeSize; ChannelIdx++)
				{
					OutAttributeData.AttributeData[FrameIdx][FrameAttributeOffset + ChannelIdx] =
						FrameAttributes[FrameAttributeIdx].FrameAttribute->AttributeData[ChannelIdx][FrameOffset];
				}
			}
			else
			{
				UE::Learning::Array::Zero(OutAttributeData.AttributeData[FrameIdx].Slice(FrameAttributeOffset, FrameAttributeSize));
			}
		}
	}
}

void UAnimDatabase::GetPoseData(
	const UE::AnimDatabase::FPoseDataView& OutPoseData,
	const int32 SequenceIdx,
	const int32 FrameStart,
	const TArrayView<const FAnimDatabaseFrameAttribute> FrameAttributes) const
{
	GetPoseSubsetData(OutPoseData, SequenceIdx, FrameStart, UE::Learning::FIndexSet(0, OutPoseData.GetBoneNum()), FrameAttributes);
}

void UAnimDatabase::GetPoseSubsetData(
	const UE::AnimDatabase::FPoseDataView& OutPoseData,
	const int32 SequenceIdx,
	const int32 FrameStart,
	const UE::Learning::FIndexSet BoneIndices,
	const TArrayView<const FAnimDatabaseFrameAttribute> FrameAttributes) const
{
	GetPoseRootData(OutPoseData.RootData, SequenceIdx, FrameStart);
	GetPoseLocalBoneSubsetData(OutPoseData.LocalBoneData, SequenceIdx, FrameStart, BoneIndices);
	GetAttributeData(OutPoseData.AttributeData, SequenceIdx, FrameStart, FrameAttributes);
}

/////////////////////////////

FTransform UAnimDatabase::SampleRootTransform(const int32 SequenceIdx, const float SequenceTime) const
{
	WaitForCompressionOnAnimSequence(SequenceIdx);

	const UAnimSequence* AnimSequence = GetAnimSequence(SequenceIdx);

	if (!AnimSequence || AnimSequence->GetSkeleton() != Skeleton)
	{
		return FTransform::Identity;
	}

	const UMirrorDataTable* MirrorDataTableObject = GetMirrorDataTable();
	const bool bIsMirrored = GetIsMirrored(SequenceIdx);

	float DatabaseToSequence = AnimSequence->GetSamplingFrameRate().AsDecimal() / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER);
	int32 SequenceStartFrame = INDEX_NONE, SequenceStopFrame = INDEX_NONE;
	UE::AnimDatabase::Private::ComputeExtractionRangesForFrameTime(
		SequenceStartFrame, SequenceStopFrame,
		DatabaseToSequence, SequenceTime * FrameRate.AsDecimal(),
		AnimSequence->GetNumberOfSampledKeys());

	TLearningArray<1, FVector, TInlineAllocator<16>> RootLocations; RootLocations.SetNumUninitialized({ SequenceStopFrame - SequenceStartFrame });
	TLearningArray<1, FQuat4f, TInlineAllocator<16>> RootRotations; RootRotations.SetNumUninitialized({ SequenceStopFrame - SequenceStartFrame });
	TLearningArray<1, FVector3f, TInlineAllocator<16>> RootScales; RootScales.SetNumUninitialized({ SequenceStopFrame - SequenceStartFrame });
	UE::AnimDatabase::Private::ExtractRoot(RootLocations, RootRotations, RootScales, AnimSequence, SequenceStartFrame, MirrorDataTableObject, bIsMirrored);

	return UE::AnimDatabase::Private::SampleSequenceTransform(
		RootLocations, RootRotations, RootScales,
		SequenceStartFrame, SequenceStopFrame, DatabaseToSequence, SequenceTime * FrameRate.AsDecimal(),
		1.0f / FMath::Max(AnimSequence->GetSamplingFrameRate().AsDecimal(), UE_SMALL_NUMBER),
		PoseSampler);
}

FVector UAnimDatabase::SampleRootLocation(const int32 SequenceIdx, const float SequenceTime) const
{
	WaitForCompressionOnAnimSequence(SequenceIdx);

	const UAnimSequence* AnimSequence = GetAnimSequence(SequenceIdx);

	if (!AnimSequence || AnimSequence->GetSkeleton() != Skeleton)
	{
		return FVector::ZeroVector;
	}

	const UMirrorDataTable* MirrorDataTableObject = GetMirrorDataTable();
	const bool bIsMirrored = GetIsMirrored(SequenceIdx);

	float DatabaseToSequence = AnimSequence->GetSamplingFrameRate().AsDecimal() / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER);
	int32 SequenceStartFrame = INDEX_NONE, SequenceStopFrame = INDEX_NONE;
	UE::AnimDatabase::Private::ComputeExtractionRangesForFrameTime(
		SequenceStartFrame, SequenceStopFrame,
		DatabaseToSequence, SequenceTime * FrameRate.AsDecimal(),
		AnimSequence->GetNumberOfSampledKeys());

	TLearningArray<1, FVector, TInlineAllocator<16>> RootLocations; RootLocations.SetNumUninitialized({ SequenceStopFrame - SequenceStartFrame });
	UE::AnimDatabase::Private::ExtractRootLocations(RootLocations, AnimSequence, SequenceStartFrame, MirrorDataTableObject, bIsMirrored);

	return UE::AnimDatabase::Private::SampleSequenceLocation(
		RootLocations,
		SequenceStartFrame, SequenceStopFrame, DatabaseToSequence, SequenceTime * FrameRate.AsDecimal(),
		1.0f / FMath::Max(AnimSequence->GetSamplingFrameRate().AsDecimal(), UE_SMALL_NUMBER),
		PoseSampler);
}

FQuat4f UAnimDatabase::SampleRootRotation(const int32 SequenceIdx, const float SequenceTime) const
{
	WaitForCompressionOnAnimSequence(SequenceIdx);

	const UAnimSequence* AnimSequence = GetAnimSequence(SequenceIdx);

	if (!AnimSequence || AnimSequence->GetSkeleton() != Skeleton)
	{
		return FQuat4f::Identity;
	}

	const UMirrorDataTable* MirrorDataTableObject = GetMirrorDataTable();
	const bool bIsMirrored = GetIsMirrored(SequenceIdx);

	float DatabaseToSequence = AnimSequence->GetSamplingFrameRate().AsDecimal() / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER);
	int32 SequenceStartFrame = INDEX_NONE, SequenceStopFrame = INDEX_NONE;
	UE::AnimDatabase::Private::ComputeExtractionRangesForFrameTime(
		SequenceStartFrame, SequenceStopFrame,
		DatabaseToSequence, SequenceTime * FrameRate.AsDecimal(),
		AnimSequence->GetNumberOfSampledKeys());

	TLearningArray<1, FQuat4f, TInlineAllocator<16>> RootRotations; RootRotations.SetNumUninitialized({ SequenceStopFrame - SequenceStartFrame });
	UE::AnimDatabase::Private::ExtractRootRotations(RootRotations, AnimSequence, SequenceStartFrame, MirrorDataTableObject, bIsMirrored);

	return UE::AnimDatabase::Private::SampleSequenceRotation(
		RootRotations,
		SequenceStartFrame, SequenceStopFrame, DatabaseToSequence, SequenceTime * FrameRate.AsDecimal(),
		1.0f / FMath::Max(AnimSequence->GetSamplingFrameRate().AsDecimal(), UE_SMALL_NUMBER),
		PoseSampler);
}

FVector3f UAnimDatabase::SampleRootDirection(const int32 SequenceIdx, const float SequenceTime, const FVector3f ForwardVector) const
{
	return SampleRootRotation(SequenceIdx, SequenceTime).RotateVector(ForwardVector);
}

FVector3f UAnimDatabase::SampleRootLinearVelocity(const int32 SequenceIdx, const float SequenceTime) const
{
	WaitForCompressionOnAnimSequence(SequenceIdx);

	const UAnimSequence* AnimSequence = GetAnimSequence(SequenceIdx);

	if (!AnimSequence || AnimSequence->GetSkeleton() != Skeleton)
	{
		return FVector3f::ZeroVector;
	}

	const UMirrorDataTable* MirrorDataTableObject = GetMirrorDataTable();
	const bool bIsMirrored = GetIsMirrored(SequenceIdx);

	float DatabaseToSequence = AnimSequence->GetSamplingFrameRate().AsDecimal() / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER);
	int32 SequenceStartFrame = INDEX_NONE, SequenceStopFrame = INDEX_NONE;
	UE::AnimDatabase::Private::ComputeExtractionRangesForFrameTime(
		SequenceStartFrame, SequenceStopFrame,
		DatabaseToSequence, SequenceTime * FrameRate.AsDecimal(),
		AnimSequence->GetNumberOfSampledKeys());

	TLearningArray<1, FVector, TInlineAllocator<16>> RootLocations; RootLocations.SetNumUninitialized({ SequenceStopFrame - SequenceStartFrame });
	UE::AnimDatabase::Private::ExtractRootLocations(RootLocations, AnimSequence, SequenceStartFrame, MirrorDataTableObject, bIsMirrored);

	return UE::AnimDatabase::Private::SampleSequenceLinearVelocity(
		RootLocations,
		SequenceStartFrame, SequenceStopFrame, DatabaseToSequence, SequenceTime * FrameRate.AsDecimal(),
		1.0f / FMath::Max(AnimSequence->GetSamplingFrameRate().AsDecimal(), UE_SMALL_NUMBER),
		PoseSampler);
}

void UAnimDatabase::SampleAttributeData(
	const UE::AnimDatabase::FPoseAttributeDataView& OutAttributeData,
	const TLearningArrayView<1, const int32> SequenceIndices,
	const TLearningArrayView<1, const float> SequenceTimes,
	const TLearningArrayView<1, const FAnimDatabaseFrameAttribute> FrameAttributes) const
{
	const int32 FrameAttributeNum = OutAttributeData.GetAttributeNum();
	check(FrameAttributeNum == FrameAttributes.Num());
	check(SequenceIndices.Num() == OutAttributeData.GetFrameNum());
	check(SequenceTimes.Num() == OutAttributeData.GetFrameNum());
	const int32 FrameNum = SequenceTimes.Num();

	for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
	{
		for (int32 FrameAttributeIdx = 0; FrameAttributeIdx < FrameAttributeNum; FrameAttributeIdx++)
		{
			check(FrameAttributes[FrameAttributeIdx].IsValid());

			const int32 FrameAttributeOffset = OutAttributeData.GetAttributeOffset(FrameAttributeIdx);
			const int32 FrameAttributeSize = OutAttributeData.GetAttributeSize(FrameAttributeIdx);
			const EAnimDatabaseAttributeType FrameAttributeType = OutAttributeData.GetAttributeType(FrameAttributeIdx);

			OutAttributeData.AttributeActive[FrameIdx][FrameAttributeIdx] = FrameAttributes[FrameAttributeIdx].FrameAttribute->FrameRangeSet.ContainsTime(SequenceIndices[FrameIdx], SequenceTimes[FrameIdx], 1.0f / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER));

			if (FrameAttributeType == FrameAttributes[FrameAttributeIdx].Type && OutAttributeData.AttributeActive[FrameIdx][FrameAttributeIdx])
			{
				switch (FrameAttributeType)
				{
				case EAnimDatabaseAttributeType::Null:
				{
					break;
				}

				case EAnimDatabaseAttributeType::Bool:
				{
					bool bBoolValue = false;
					UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleBool(bBoolValue, FrameAttributes[FrameAttributeIdx], SequenceIndices[FrameIdx], SequenceTimes[FrameIdx], FrameRate);
					OutAttributeData.AttributeData[FrameIdx][FrameAttributeOffset] = bBoolValue ? 1.0f : 0.0f;
					break;
				}

				case EAnimDatabaseAttributeType::Float:
				{
					UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleFloat(OutAttributeData.AttributeData[FrameIdx][FrameAttributeOffset], FrameAttributes[FrameAttributeIdx], SequenceIndices[FrameIdx], SequenceTimes[FrameIdx], FrameRate);
					break;
				}

				case EAnimDatabaseAttributeType::Angle:
				{
					UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleAngleRadians(OutAttributeData.AttributeData[FrameIdx][FrameAttributeOffset], FrameAttributes[FrameAttributeIdx], SequenceIndices[FrameIdx], SequenceTimes[FrameIdx], FrameRate);
					break;
				}

				case EAnimDatabaseAttributeType::Location:
				{
					FVector3f LocationValue = FVector3f::ZeroVector;
					UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleLocationFloat(LocationValue, FrameAttributes[FrameAttributeIdx], SequenceIndices[FrameIdx], SequenceTimes[FrameIdx], FrameRate);
					OutAttributeData.AttributeData[FrameIdx][FrameAttributeOffset + 0] = LocationValue.X;
					OutAttributeData.AttributeData[FrameIdx][FrameAttributeOffset + 1] = LocationValue.Y;
					OutAttributeData.AttributeData[FrameIdx][FrameAttributeOffset + 2] = LocationValue.Z;
					break;
				}

				case EAnimDatabaseAttributeType::LinearVelocity:
				{
					FVector3f LinearVelocityValue = FVector3f::ZeroVector;
					UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleLinearVelocityFloat(LinearVelocityValue, FrameAttributes[FrameAttributeIdx], SequenceIndices[FrameIdx], SequenceTimes[FrameIdx], FrameRate);
					OutAttributeData.AttributeData[FrameIdx][FrameAttributeOffset + 0] = LinearVelocityValue.X;
					OutAttributeData.AttributeData[FrameIdx][FrameAttributeOffset + 1] = LinearVelocityValue.Y;
					OutAttributeData.AttributeData[FrameIdx][FrameAttributeOffset + 2] = LinearVelocityValue.Z;
					break;
				}

				case EAnimDatabaseAttributeType::AngularVelocity:
				{
					FVector3f AngularVelocityValue = FVector3f::ZeroVector;
					UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleAngularVelocityFloat(AngularVelocityValue, FrameAttributes[FrameAttributeIdx], SequenceIndices[FrameIdx], SequenceTimes[FrameIdx], FrameRate);
					OutAttributeData.AttributeData[FrameIdx][FrameAttributeOffset + 0] = AngularVelocityValue.X;
					OutAttributeData.AttributeData[FrameIdx][FrameAttributeOffset + 1] = AngularVelocityValue.Y;
					OutAttributeData.AttributeData[FrameIdx][FrameAttributeOffset + 2] = AngularVelocityValue.Z;
					break;
				}

				case EAnimDatabaseAttributeType::ScalarVelocity:
				{
					FVector3f ScalarVelocityValue = FVector3f::ZeroVector;
					UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleScalarVelocityFloat(ScalarVelocityValue, FrameAttributes[FrameAttributeIdx], SequenceIndices[FrameIdx], SequenceTimes[FrameIdx], FrameRate);
					OutAttributeData.AttributeData[FrameIdx][FrameAttributeOffset + 0] = ScalarVelocityValue.X;
					OutAttributeData.AttributeData[FrameIdx][FrameAttributeOffset + 1] = ScalarVelocityValue.Y;
					OutAttributeData.AttributeData[FrameIdx][FrameAttributeOffset + 2] = ScalarVelocityValue.Z;
					break;
				}

				case EAnimDatabaseAttributeType::Rotation:
				{
					FQuat4f RotationValue = FQuat4f::Identity;
					UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleRotationAsQuatFloat(RotationValue, FrameAttributes[FrameAttributeIdx], SequenceIndices[FrameIdx], SequenceTimes[FrameIdx], FrameRate);
					OutAttributeData.AttributeData[FrameIdx][FrameAttributeOffset + 0] = RotationValue.X;
					OutAttributeData.AttributeData[FrameIdx][FrameAttributeOffset + 1] = RotationValue.Y;
					OutAttributeData.AttributeData[FrameIdx][FrameAttributeOffset + 2] = RotationValue.Z;
					OutAttributeData.AttributeData[FrameIdx][FrameAttributeOffset + 3] = RotationValue.W;
					break;
				}

				case EAnimDatabaseAttributeType::Scale:
				{
					FVector3f ScaleValue = FVector3f::OneVector;
					UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleScaleFloat(ScaleValue, FrameAttributes[FrameAttributeIdx], SequenceIndices[FrameIdx], SequenceTimes[FrameIdx], FrameRate);
					OutAttributeData.AttributeData[FrameIdx][FrameAttributeOffset + 0] = ScaleValue.X;
					OutAttributeData.AttributeData[FrameIdx][FrameAttributeOffset + 1] = ScaleValue.Y;
					OutAttributeData.AttributeData[FrameIdx][FrameAttributeOffset + 2] = ScaleValue.Z;
					break;
				}

				case EAnimDatabaseAttributeType::Direction:
				{
					FVector3f DirectionValue = FVector3f::ForwardVector;
					UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleDirectionFloat(DirectionValue, FrameAttributes[FrameAttributeIdx], SequenceIndices[FrameIdx], SequenceTimes[FrameIdx], FrameRate);
					OutAttributeData.AttributeData[FrameIdx][FrameAttributeOffset + 0] = DirectionValue.X;
					OutAttributeData.AttributeData[FrameIdx][FrameAttributeOffset + 1] = DirectionValue.Y;
					OutAttributeData.AttributeData[FrameIdx][FrameAttributeOffset + 2] = DirectionValue.Z;
					break;
				}

				case EAnimDatabaseAttributeType::Transform:
				{
					FTransform3f TransformValue = FTransform3f::Identity;
					UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleTransformFloat(TransformValue, FrameAttributes[FrameAttributeIdx], SequenceIndices[FrameIdx], SequenceTimes[FrameIdx], FrameRate);
					OutAttributeData.AttributeData[FrameIdx][FrameAttributeOffset + 0] = TransformValue.GetLocation().X;
					OutAttributeData.AttributeData[FrameIdx][FrameAttributeOffset + 1] = TransformValue.GetLocation().Y;
					OutAttributeData.AttributeData[FrameIdx][FrameAttributeOffset + 2] = TransformValue.GetLocation().Z;
					OutAttributeData.AttributeData[FrameIdx][FrameAttributeOffset + 3] = TransformValue.GetRotation().X;
					OutAttributeData.AttributeData[FrameIdx][FrameAttributeOffset + 4] = TransformValue.GetRotation().Y;
					OutAttributeData.AttributeData[FrameIdx][FrameAttributeOffset + 5] = TransformValue.GetRotation().Z;
					OutAttributeData.AttributeData[FrameIdx][FrameAttributeOffset + 6] = TransformValue.GetRotation().W;
					OutAttributeData.AttributeData[FrameIdx][FrameAttributeOffset + 7] = TransformValue.GetScale3D().X;
					OutAttributeData.AttributeData[FrameIdx][FrameAttributeOffset + 8] = TransformValue.GetScale3D().Y;
					OutAttributeData.AttributeData[FrameIdx][FrameAttributeOffset + 9] = TransformValue.GetScale3D().Z;
					break;
				}

				case EAnimDatabaseAttributeType::Event:
				{
					bool bTimeUntilEventKnown = false;
					float TimeUntilEvent = UE_MAX_FLT;
					UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleEvent(bTimeUntilEventKnown, TimeUntilEvent, FrameAttributes[FrameAttributeIdx], SequenceIndices[FrameIdx], SequenceTimes[FrameIdx], FrameRate);
					OutAttributeData.AttributeData[FrameIdx][FrameAttributeOffset + 0] = bTimeUntilEventKnown ? 1.0f : 0.0f;
					OutAttributeData.AttributeData[FrameIdx][FrameAttributeOffset + 1] = TimeUntilEvent;
					break;
				}

				default:
				{
					checkNoEntry();
					break;
				}

				}
			}
			else
			{
				UE::Learning::Array::Zero(OutAttributeData.AttributeData[FrameIdx].Slice(FrameAttributeOffset, FrameAttributeSize));
			}
		}
	}
}

void UAnimDatabase::SamplePoseRootData(
	const UE::AnimDatabase::FPoseRootDataView& OutPoseRootData,
	const TLearningArrayView<1, const int32> SequenceIndices,
	const TLearningArrayView<1, const float> SequenceTimes) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabase::SamplePoseRootData);

	WaitForCompressionOnAnimSequencesFromArrayView(SequenceIndices);

	const int32 FrameNum = SequenceTimes.Num();
	if (FrameNum == 0) { return; }

	check(OutPoseRootData.GetFrameNum() == SequenceIndices.Num());
	check(OutPoseRootData.GetFrameNum() == SequenceTimes.Num());

	const UMirrorDataTable* MirrorDataTableObject = GetMirrorDataTable();

	TLearningArray<1, FVector, TInlineAllocator<16>> RootLocations;
	TLearningArray<1, FQuat4f, TInlineAllocator<16>> RootRotations;
	TLearningArray<1, FVector3f, TInlineAllocator<16>> RootScales;

	for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabase::SamplePoseRootData::Frame);

		const UAnimSequence* AnimSequence = GetAnimSequence(SequenceIndices[FrameIdx]);

		if (!AnimSequence || AnimSequence->GetSkeleton() != Skeleton)
		{
			UE::AnimDatabase::PoseData::Reset(OutPoseRootData.Slice(FrameIdx, 1));
			continue;
		}

		const bool bIsMirrored = GetIsMirrored(SequenceIndices[FrameIdx]);

		float DatabaseToSequence = AnimSequence->GetSamplingFrameRate().AsDecimal() / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER);
		int32 SequenceStartFrame = INDEX_NONE, SequenceStopFrame = INDEX_NONE;
		UE::AnimDatabase::Private::ComputeExtractionRangesForFrameTime(
			SequenceStartFrame, SequenceStopFrame,
			DatabaseToSequence, SequenceTimes[FrameIdx] * FrameRate.AsDecimal(),
			AnimSequence->GetNumberOfSampledKeys());

		RootLocations.SetNumUninitialized({ SequenceStopFrame - SequenceStartFrame });
		RootRotations.SetNumUninitialized({ SequenceStopFrame - SequenceStartFrame });
		RootScales.SetNumUninitialized({ SequenceStopFrame - SequenceStartFrame });
		UE::AnimDatabase::Private::ExtractRoot(RootLocations, RootRotations, RootScales, AnimSequence, SequenceStartFrame, MirrorDataTableObject, bIsMirrored);

		UE::AnimDatabase::Private::SampleSequenceTransformVelocity(
			OutPoseRootData.RootLocations[FrameIdx],
			OutPoseRootData.RootRotations[FrameIdx],
			OutPoseRootData.RootScales[FrameIdx],
			OutPoseRootData.RootLinearVelocities[FrameIdx],
			OutPoseRootData.RootAngularVelocities[FrameIdx],
			OutPoseRootData.RootScalarVelocities[FrameIdx],
			RootLocations, RootRotations, RootScales,
			SequenceStartFrame, SequenceStopFrame, DatabaseToSequence, SequenceTimes[FrameIdx] * FrameRate.AsDecimal(),
			1.0f / FMath::Max(AnimSequence->GetSamplingFrameRate().AsDecimal(), UE_SMALL_NUMBER),
			PoseSampler);
	}
}

void UAnimDatabase::SamplePoseLocalBoneData(
	const UE::AnimDatabase::FPoseLocalBoneDataView& OutPoseLocalBoneData,
	const TLearningArrayView<1, const int32> SequenceIndices,
	const TLearningArrayView<1, const float> SequenceTimes) const
{
	SamplePoseLocalBoneSubsetData(OutPoseLocalBoneData, SequenceIndices, SequenceTimes, UE::Learning::FIndexSet(0, GetBoneNum()));
}

void UAnimDatabase::SamplePoseLocalBoneSubsetData(
	const UE::AnimDatabase::FPoseLocalBoneDataView& OutPoseLocalBoneData,
	const TLearningArrayView<1, const int32> SequenceIndices,
	const TLearningArrayView<1, const float> SequenceTimes,
	const UE::Learning::FIndexSet BoneIndices) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabase::SamplePoseLocalBoneSubsetData);

	WaitForCompressionOnAnimSequencesFromArrayView(SequenceIndices);

	const int32 FrameNum = SequenceTimes.Num();
	const int32 BoneNum = BoneIndices.Num();
	if (FrameNum == 0 || BoneNum == 0) { return; }

	check(OutPoseLocalBoneData.GetFrameNum() == SequenceIndices.Num());
	check(OutPoseLocalBoneData.GetFrameNum() == SequenceTimes.Num());
	check(OutPoseLocalBoneData.GetBoneNum() == BoneNum);
	check(UE::AnimDatabase::Math::BoneIndicesAreSortedAndUnique(BoneIndices));

	const UMirrorDataTable* MirrorDataTableObject = GetMirrorDataTable();

	TLearningArray<2, FVector3f> BoneLocations;
	TLearningArray<2, FQuat4f> BoneRotations;
	TLearningArray<2, FVector3f> BoneScales;

	for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
	{
		const UAnimSequence* AnimSequence = GetAnimSequence(SequenceIndices[FrameIdx]);

		if (!AnimSequence || AnimSequence->GetSkeleton() != Skeleton)
		{
			UE::AnimDatabase::PoseData::Reset(OutPoseLocalBoneData.Slice(FrameIdx, 1), Skeleton->GetReferenceSkeleton().GetRefBonePose());
			continue;
		}

		const bool bIsMirrored = GetIsMirrored(SequenceIndices[FrameIdx]);

		float DatabaseToSequence = AnimSequence->GetSamplingFrameRate().AsDecimal() / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER);
		int32 SequenceStartFrame = INDEX_NONE, SequenceStopFrame = INDEX_NONE;
		UE::AnimDatabase::Private::ComputeExtractionRangesForFrameTime(
			SequenceStartFrame, SequenceStopFrame,
			DatabaseToSequence, SequenceTimes[FrameIdx] * FrameRate.AsDecimal(),
			AnimSequence->GetNumberOfSampledKeys());

		BoneLocations.SetNumUninitialized({ SequenceStopFrame - SequenceStartFrame, BoneNum });
		BoneRotations.SetNumUninitialized({ SequenceStopFrame - SequenceStartFrame, BoneNum });
		BoneScales.SetNumUninitialized({ SequenceStopFrame - SequenceStartFrame, BoneNum });
		UE::AnimDatabase::Private::ExtractBoneTransforms(BoneLocations, BoneRotations, BoneScales, AnimSequence, SequenceStartFrame, MirrorDataTableObject, bIsMirrored, BoneIndices);

		UE::AnimDatabase::Private::SampleSequenceTransformVelocity(
			OutPoseLocalBoneData.BoneLocations[FrameIdx],
			OutPoseLocalBoneData.BoneRotations[FrameIdx],
			OutPoseLocalBoneData.BoneScales[FrameIdx],
			OutPoseLocalBoneData.BoneLinearVelocities[FrameIdx],
			OutPoseLocalBoneData.BoneAngularVelocities[FrameIdx],
			OutPoseLocalBoneData.BoneScalarVelocities[FrameIdx],
			BoneLocations, BoneRotations, BoneScales,
			SequenceStartFrame, SequenceStopFrame, DatabaseToSequence, SequenceTimes[FrameIdx] * FrameRate.AsDecimal(),
			1.0f / FMath::Max(AnimSequence->GetSamplingFrameRate().AsDecimal(), UE_SMALL_NUMBER),
			PoseSampler);
	}
}


void UAnimDatabase::SamplePoseData(
	const UE::AnimDatabase::FPoseDataView& OutPoseData,
	const TLearningArrayView<1, const int32> SequenceIndices,
	const TLearningArrayView<1, const float> SequenceTimes,
	const TLearningArrayView<1, const FAnimDatabaseFrameAttribute> FrameAttributes) const
{
	SamplePoseSubsetData(OutPoseData, SequenceIndices, SequenceTimes, UE::Learning::FIndexSet(0, GetBoneNum()), FrameAttributes);
}

void UAnimDatabase::SamplePoseSubsetData(
	const UE::AnimDatabase::FPoseDataView& OutPoseData,
	const TLearningArrayView<1, const int32> SequenceIndices,
	const TLearningArrayView<1, const float> SequenceTimes,
	const UE::Learning::FIndexSet BoneIndices,
	const TLearningArrayView<1, const FAnimDatabaseFrameAttribute> FrameAttributes) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabase::SampleSequencePoseData);

	SamplePoseRootData(OutPoseData.RootData, SequenceIndices, SequenceTimes);
	SamplePoseLocalBoneSubsetData(OutPoseData.LocalBoneData, SequenceIndices, SequenceTimes, BoneIndices);
	SampleAttributeData(OutPoseData.AttributeData, SequenceIndices, SequenceTimes, FrameAttributes);
}

void UAnimDatabase::SampleCurveActiveData(
	const TLearningArrayView<2, bool> OutCurveActive,
	const TLearningArrayView<1, const int32> SequenceIndices,
	const TLearningArrayView<1, const float> SequenceTimes,
	const TArrayView<const FName> CurveNames) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabase::SampleCurveActiveData);

	WaitForCompressionOnAnimSequencesFromArrayView(SequenceIndices);

	const int32 FrameNum = OutCurveActive.Num<0>();
	const int32 CurveNum = CurveNames.Num();
	if (FrameNum == 0 || CurveNum == 0) { return; }

	check(OutCurveActive.Num<1>() == CurveNum);
	check(SequenceIndices.Num() == FrameNum);
	check(SequenceTimes.Num() == FrameNum);

	const UMirrorDataTable* MirrorDataTableObject = GetMirrorDataTable();

	TLearningArray<2, bool, TInlineAllocator<16>> CurveActive;

	for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabase::SampleCurveActiveData::Frame);

		const UAnimSequence* AnimSequence = GetAnimSequence(SequenceIndices[FrameIdx]);

		if (!AnimSequence || AnimSequence->GetSkeleton() != Skeleton)
		{
			UE::Learning::Array::Zero(OutCurveActive.Slice(FrameIdx, 1));
			continue;
		}

		const bool bIsMirrored = GetIsMirrored(SequenceIndices[FrameIdx]);

		float DatabaseToSequence = AnimSequence->GetSamplingFrameRate().AsDecimal() / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER);
		int32 SequenceStartFrame = INDEX_NONE, SequenceStopFrame = INDEX_NONE;
		UE::AnimDatabase::Private::ComputeExtractionRangesForFrameTime(
			SequenceStartFrame, SequenceStopFrame,
			DatabaseToSequence, SequenceTimes[FrameIdx] * FrameRate.AsDecimal(),
			AnimSequence->GetNumberOfSampledKeys());

		CurveActive.SetNumUninitialized({ SequenceStopFrame - SequenceStartFrame, CurveNum });
		UE::AnimDatabase::Private::ExtractCurveActiveData(CurveActive, AnimSequence, SequenceStartFrame, MirrorDataTableObject, bIsMirrored, CurveNames);

		UE::AnimDatabase::Private::SampleSequenceCurveActive(
			OutCurveActive[FrameIdx],
			CurveActive,
			SequenceStartFrame, SequenceStopFrame, DatabaseToSequence, SequenceTimes[FrameIdx] * FrameRate.AsDecimal(),
			1.0f / FMath::Max(AnimSequence->GetSamplingFrameRate().AsDecimal(), UE_SMALL_NUMBER));
	}
}

void UAnimDatabase::SampleCurveData(
	const TLearningArrayView<2, float> OutCurveValues,
	const TLearningArrayView<2, float> OutCurveVelocities,
	const TLearningArrayView<2, bool> OutCurveActive,
	const TLearningArrayView<1, const int32> SequenceIndices,
	const TLearningArrayView<1, const float> SequenceTimes,
	const TArrayView<const FName> CurveNames) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabase::SampleCurveData);

	const int32 FrameNum = OutCurveActive.Num<0>();
	const int32 CurveNum = CurveNames.Num();
	if (FrameNum == 0 || CurveNum == 0) { return; }

	WaitForCompressionOnAnimSequencesFromArrayView(SequenceIndices);

	check(OutCurveValues.Num<1>() == CurveNum);
	check(OutCurveActive.Num<1>() == CurveNum);
	check(OutCurveVelocities.Num<1>() == CurveNum);
	check(SequenceIndices.Num() == FrameNum);
	check(SequenceTimes.Num() == FrameNum);

	const UMirrorDataTable* MirrorDataTableObject = GetMirrorDataTable();

	TLearningArray<2, float, TInlineAllocator<16>> CurveValue;
	TLearningArray<2, bool, TInlineAllocator<16>> CurveActive;

	for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabase::SampleCurveData::Frame);

		const UAnimSequence* AnimSequence = GetAnimSequence(SequenceIndices[FrameIdx]);

		if (!AnimSequence || AnimSequence->GetSkeleton() != Skeleton)
		{
			UE::Learning::Array::Zero(OutCurveValues.Slice(FrameIdx, 1));
			UE::Learning::Array::Zero(OutCurveVelocities.Slice(FrameIdx, 1));
			UE::Learning::Array::Zero(OutCurveActive.Slice(FrameIdx, 1));
			continue;
		}

		const bool bIsMirrored = GetIsMirrored(SequenceIndices[FrameIdx]);

		float DatabaseToSequence = AnimSequence->GetSamplingFrameRate().AsDecimal() / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER);
		int32 SequenceStartFrame = INDEX_NONE, SequenceStopFrame = INDEX_NONE;
		UE::AnimDatabase::Private::ComputeExtractionRangesForFrameTime(
			SequenceStartFrame, SequenceStopFrame,
			DatabaseToSequence, SequenceTimes[FrameIdx] * FrameRate.AsDecimal(),
			AnimSequence->GetNumberOfSampledKeys());

		CurveValue.SetNumUninitialized({ SequenceStopFrame - SequenceStartFrame, CurveNum });
		CurveActive.SetNumUninitialized({ SequenceStopFrame - SequenceStartFrame, CurveNum });
		UE::AnimDatabase::Private::ExtractCurveData(CurveValue, CurveActive, AnimSequence, SequenceStartFrame, MirrorDataTableObject, bIsMirrored, CurveNames);

		UE::AnimDatabase::Private::SampleSequenceCurveActive(
			OutCurveActive[FrameIdx],
			CurveActive,
			SequenceStartFrame, SequenceStopFrame, DatabaseToSequence, SequenceTimes[FrameIdx] * FrameRate.AsDecimal(),
			1.0f / FMath::Max(AnimSequence->GetSamplingFrameRate().AsDecimal(), UE_SMALL_NUMBER));

		UE::AnimDatabase::Private::SampleSequenceCurveValue(
			OutCurveValues[FrameIdx],
			OutCurveVelocities[FrameIdx],
			CurveValue,
			SequenceStartFrame, SequenceStopFrame, DatabaseToSequence, SequenceTimes[FrameIdx] * FrameRate.AsDecimal(),
			1.0f / FMath::Max(AnimSequence->GetSamplingFrameRate().AsDecimal(), UE_SMALL_NUMBER),
			CurveSampler);
	}
}

#if WITH_EDITOR

void UAnimDatabaseFunction::Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact) {}

#endif

void UAnimDatabaseLibrary::GetAnimSequenceAnimNotifyClasses(TArray<TSubclassOf<UAnimNotify>>& OutClasses, UAnimSequence* AnimSequence)
{
#if WITH_EDITOR
	OutClasses.Reset();

	if (!AnimSequence)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "GetAnimSequenceAnimNotifyClasses: AnimSequence is null.");
		return;
	}

	FAnimNotifyContext Notifies;
	AnimSequence->GetAnimNotifies(0.0f, AnimSequence->GetPlayLength(), Notifies);

	for (const FAnimNotifyEventReference& AnimNotifyRef : Notifies.ActiveNotifies)
	{
		if (AnimNotifyRef.GetNotify()->Notify)
		{
			OutClasses.AddUnique(AnimNotifyRef.GetNotify()->Notify->GetClass());
		}
	}
#endif
}

void UAnimDatabaseLibrary::GetAnimSequenceAnimNotifyStateClasses(TArray<TSubclassOf<UAnimNotifyState>>& OutClasses, UAnimSequence* AnimSequence)
{
#if WITH_EDITOR
	OutClasses.Reset();

	if (!AnimSequence)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "GetAnimSequenceAnimNotifyStateClasses: AnimSequence is null.");
		return;
	}

	FAnimNotifyContext Notifies;
	AnimSequence->GetAnimNotifies(0.0f, AnimSequence->GetPlayLength(), Notifies);

	for (const FAnimNotifyEventReference& AnimNotifyRef : Notifies.ActiveNotifies)
	{
		if (AnimNotifyRef.GetNotify()->NotifyStateClass)
		{
			OutClasses.AddUnique(AnimNotifyRef.GetNotify()->NotifyStateClass->GetClass());
		}
	}
#endif
}

void UAnimDatabaseLibrary::GetAnimSequenceAnimNotifyTimes(TArray<float>& OutTimes, const TSubclassOf<UAnimNotify>& Class, UAnimSequence* AnimSequence)
{
#if WITH_EDITOR
	OutTimes.Reset();

	if (!AnimSequence)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "GetAnimSequenceAnimNotifyTimes: AnimSequence is null.");
		return;
	}

	FAnimNotifyContext Notifies;
	AnimSequence->GetAnimNotifies(0.0f, AnimSequence->GetPlayLength(), Notifies);

	TArray<FAnimNotifyEventReference, TInlineAllocator<64>> SortedReferences;
	SortedReferences = Notifies.ActiveNotifies;
	SortedReferences.Sort([](const FAnimNotifyEventReference& A, const FAnimNotifyEventReference& B) { return A.GetNotify()->GetTime() < B.GetNotify()->GetTime(); });

	for (const FAnimNotifyEventReference& AnimNotifyRef : SortedReferences)
	{
		if (AnimNotifyRef.GetNotify()->Notify && AnimNotifyRef.GetNotify()->Notify->IsA(Class))
		{
			OutTimes.Add(AnimNotifyRef.GetNotify()->GetTime());
		}
	}
#endif
}

void UAnimDatabaseLibrary::GetAnimSequenceAnimNotifyStateTimesAndDurations(TArray<float>& OutTimes, TArray<float>& OutDurations, const TSubclassOf<UAnimNotifyState>& Class, UAnimSequence* AnimSequence)
{
#if WITH_EDITOR
	OutTimes.Reset();
	OutDurations.Reset();

	if (!AnimSequence)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "GetAnimSequenceAnimNotifyStateTimesAndDurations: AnimSequence is null.");
		return;
	}

	FAnimNotifyContext Notifies;
	AnimSequence->GetAnimNotifies(0.0f, AnimSequence->GetPlayLength(), Notifies);

	TArray<FAnimNotifyEventReference, TInlineAllocator<64>> SortedReferences;
	SortedReferences = Notifies.ActiveNotifies;
	SortedReferences.Sort([](const FAnimNotifyEventReference& A, const FAnimNotifyEventReference& B) { return A.GetNotify()->GetTime() < B.GetNotify()->GetTime(); });

	for (const FAnimNotifyEventReference& AnimNotifyRef : SortedReferences)
	{
		if (AnimNotifyRef.GetNotify()->NotifyStateClass && AnimNotifyRef.GetNotify()->NotifyStateClass->IsA(Class))
		{
			OutTimes.Add(AnimNotifyRef.GetNotify()->GetTime());
			OutDurations.Add(AnimNotifyRef.GetNotify()->GetDuration());
		}
	}
#endif
}

void UAnimDatabaseLibrary::SetAnimSequencePoseData(
	UAnimSequence* AnimSequence, 
	TScriptInterface<IAnimationDataController> Controller, 
	const int32 StartFrame, 
	const UE::AnimDatabase::FPoseDataConstView& PoseData,
	const UE::Learning::FIndexSet UsedBones,
	const bool bShouldTransact)
{
#if WITH_EDITOR
	Controller->OpenBracket(LOCTEXT("KeyAnimSequence", "Keying Animation Sequence"), bShouldTransact);

	const int32 FrameNum = PoseData.GetFrameNum();
	const int32 BoneNum = PoseData.GetBoneNum();

	TArray<FVector> BonePositions;
	TArray<FQuat> BoneRotations;
	TArray<FVector> BoneScales;
	BonePositions.SetNumUninitialized(FrameNum);
	BoneRotations.SetNumUninitialized(FrameNum);
	BoneScales.SetNumUninitialized(FrameNum);

	const FReferenceSkeleton ReferenceSkeleton = AnimSequence->GetSkeleton()->GetReferenceSkeleton();
	check(ReferenceSkeleton.GetRefBonePose().Num() == BoneNum);

	FInt32Range FrameRange(StartFrame, StartFrame + FrameNum);

	for (int32 BoneIdx : UsedBones)
	{
		const FTransform RefTransform = ReferenceSkeleton.GetRefBonePose()[BoneIdx];
		const FName BoneName = ReferenceSkeleton.GetBoneName(BoneIdx);
		check(BoneName != NAME_None);
		if (!Controller->GetModel()->IsValidBoneTrackName(BoneName)) { continue; }

		if (BoneIdx == 0)
		{
			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				BonePositions[FrameIdx] = PoseData.RootData.RootLocations[FrameIdx];
				BoneRotations[FrameIdx] = ((FQuat)PoseData.RootData.RootRotations[FrameIdx]).GetNormalized();
				BoneScales[FrameIdx] = (FVector)PoseData.RootData.RootScales[FrameIdx];
			}

			Controller->UpdateBoneTrackKeys(BoneName, FrameRange, BonePositions, BoneRotations, BoneScales, bShouldTransact);
		}
		else
		{
			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				BonePositions[FrameIdx] = (FVector)PoseData.LocalBoneData.BoneLocations[FrameIdx][BoneIdx];
				BoneRotations[FrameIdx] = ((FQuat)PoseData.LocalBoneData.BoneRotations[FrameIdx][BoneIdx]).GetNormalized();
				BoneScales[FrameIdx] = (FVector)PoseData.LocalBoneData.BoneScales[FrameIdx][BoneIdx];
			}

			Controller->UpdateBoneTrackKeys(BoneName, FrameRange, BonePositions, BoneRotations, BoneScales, bShouldTransact);
		}
	}

	Controller->NotifyPopulated();
	Controller->CloseBracket(bShouldTransact);
#endif
}

void UAnimDatabaseLibrary::AddAnimNotifiesToSequence(UAnimSequence* AnimSequence, const FName TrackName, const TSubclassOf<UAnimNotify>& Class, const TArray<float>& Times)
{
	AddAnimNotifiesToSequenceArrayView(AnimSequence, TrackName, Class, Times);
}

void UAnimDatabaseLibrary::AddAnimNotifiesToSequenceArrayView(UAnimSequence* AnimSequence, const FName TrackName, const TSubclassOf<UAnimNotify>& Class, const TArrayView<const float> Times)
{
#if WITH_EDITOR
	if (!AnimSequence)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "AddAnimNotifiesToSequenceArrayView: AnimSequence is null.");
		return;
	}

	int32 AnimNotifyTrack = AnimSequence->AnimNotifyTracks.IndexOfByPredicate(
		[&](const FAnimNotifyTrack& Track)
		{
			return Track.TrackName == TrackName;
		});

	if (AnimNotifyTrack == INDEX_NONE)
	{
		AnimNotifyTrack = AnimSequence->AnimNotifyTracks.Add(FAnimNotifyTrack(TrackName, FLinearColor::Red));
	}

	const int32 TimeNum = Times.Num();

	for (int32 TimeIdx = 0; TimeIdx < TimeNum; TimeIdx++)
	{
		FAnimNotifyEvent& NewEvent = AnimSequence->Notifies.AddDefaulted_GetRef();

		NewEvent.NotifyName = NAME_None;
		NewEvent.Link(AnimSequence, Times[TimeIdx]);
		NewEvent.TriggerTimeOffset = GetTriggerTimeOffsetForType(AnimSequence->CalculateOffsetForNotify(Times[TimeIdx]));
		NewEvent.TrackIndex = AnimNotifyTrack;
		NewEvent.NotifyStateClass = nullptr;
		NewEvent.Guid = FGuid::NewGuid();

		if (Class)
		{
			NewEvent.Notify = NewObject<UAnimNotify>(AnimSequence, Class, NAME_None, RF_Transactional);
			if (NewEvent.Notify)
			{
				NewEvent.NotifyName = FName(*NewEvent.Notify->GetNotifyName());
			}
		}
		else
		{
			NewEvent.Notify = nullptr;
		}
	}

	// Refresh all cached data
	AnimSequence->RefreshCacheData();
	AnimSequence->Modify(true);
#endif
}

void UAnimDatabaseLibrary::RemoveAllAnimNotifiesAndAnimNotifyStatesFromSequence(UAnimSequence* AnimSequence)
{
#if WITH_EDITOR
	if (!AnimSequence)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "RemoveAllAnimNotifiesAndAnimNotifyStatesFromSequence: AnimSequence is null.");
		return;
	}

	AnimSequence->AnimNotifyTracks.Empty();
	AnimSequence->Notifies.Empty();
	AnimSequence->RefreshCacheData();
	AnimSequence->Modify(true);
#endif
}

void UAnimDatabaseLibrary::AddAnimNotifyStatesToSequence(UAnimSequence* AnimSequence, const FName TrackName, const TSubclassOf<UAnimNotifyState>& Class, const TArray<float>& Times, const TArray<float>& Durations)
{
	AddAnimNotifyStatesToSequenceArrayView(AnimSequence, TrackName, Class, Times, Durations);
}

void UAnimDatabaseLibrary::AddAnimNotifyStatesToSequenceArrayView(UAnimSequence* AnimSequence, const FName TrackName, const TSubclassOf<UAnimNotifyState>& Class, const TArrayView<const float> Times, const TArrayView<const float> Durations)
{
#if WITH_EDITOR
	if (!AnimSequence)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "AddAnimNotifyStatesToSequenceArrayView: AnimSequence is null.");
		return;
	}

	int32 AnimNotifyTrack = AnimSequence->AnimNotifyTracks.IndexOfByPredicate(
		[&](const FAnimNotifyTrack& Track)
		{
			return Track.TrackName == TrackName;
		});

	if (AnimNotifyTrack == INDEX_NONE)
	{
		AnimNotifyTrack = AnimSequence->AnimNotifyTracks.Add(FAnimNotifyTrack(TrackName, FLinearColor::Red));
	}

	const int32 TimeNum = FMath::Min(Times.Num(), Durations.Num());

	for (int32 TimeIdx = 0; TimeIdx < TimeNum; TimeIdx++)
	{
		FAnimNotifyEvent& NewEvent = AnimSequence->Notifies.AddDefaulted_GetRef();

		NewEvent.NotifyName = NAME_None;
		NewEvent.Link(AnimSequence, Times[TimeIdx]);
		NewEvent.TriggerTimeOffset = GetTriggerTimeOffsetForType(AnimSequence->CalculateOffsetForNotify(Times[TimeIdx]));
		NewEvent.TrackIndex = AnimNotifyTrack;
		NewEvent.Notify = nullptr;
		NewEvent.Guid = FGuid::NewGuid();

		if (Class)
		{
			NewEvent.NotifyStateClass = NewObject<UAnimNotifyState>(AnimSequence, Class, NAME_None, RF_Transactional);
			if (NewEvent.NotifyStateClass)
			{
				NewEvent.NotifyName = FName(*NewEvent.NotifyStateClass->GetNotifyName());
				NewEvent.SetDuration(Durations[TimeIdx]);
				NewEvent.EndLink.Link(AnimSequence, NewEvent.EndLink.GetTime());
			}
		}
		else
		{
			NewEvent.NotifyStateClass = nullptr;
		}
	}

	// Refresh all cached data
	AnimSequence->RefreshCacheData();
	AnimSequence->Modify(true);
#endif
}

void UAnimDatabaseLibrary::EnableRootMotionAndForceRootLockOnSequences(const TArray<UAnimSequence*>& Sequences)
{
	EnableRootMotionAndForceRootLockOnSequencesArrayView(Sequences);
}

void UAnimDatabaseLibrary::EnableRootMotionAndForceRootLockOnSequencesArrayView(TConstArrayView<UAnimSequence*> Sequences)
{
#if WITH_EDITOR
	for (UAnimSequence* Sequence : Sequences)
	{
		if (Sequence)
		{
			Sequence->bEnableRootMotion = true;
			Sequence->bForceRootLock = true;
			Sequence->Modify(true);
		}
	}
#endif
}

void UAnimDatabaseLibrary::DatabaseRemoveNotifies(UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges)
{
#if WITH_EDITOR
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "DatabaseRemoveNotifies: Database is null.");
		return;
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "DatabaseRemoveNotifies: Invalid Frame Ranges.");
		return;
	}

	const int32 EntryNum = FrameRanges.FrameRangeSet->GetEntryNum();

	for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
	{
		const int32 SequenceIdx = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);

		if (Database->GetIsMirrored(SequenceIdx)) { continue; }

		if (UAnimSequence* AnimSequence = Database->GetAnimSequence(SequenceIdx))
		{
			const int32 SequenceFrameNum = Database->GetSequenceFrameNum(SequenceIdx);
			bool bModified = false;

			int32 NotifyIdx = 0;
			while (NotifyIdx < AnimSequence->Notifies.Num())
			{
				const float Time = AnimSequence->Notifies[NotifyIdx].GetTriggerTime();
				const float Duration = AnimSequence->Notifies[NotifyIdx].GetDuration();

				if (AnimSequence->Notifies[NotifyIdx].Notify)
				{
					const int32 AnimNotifyTime = FMath::Clamp(
						FMath::RoundToInt(Time * Database->GetFrameRate().AsDecimal()),
						0, SequenceFrameNum - 1);

					if (FrameRanges.FrameRangeSet->Contains(SequenceIdx, AnimNotifyTime))
					{
						bModified = true;
						AnimSequence->Notifies.RemoveAt(NotifyIdx);
					}
					else
					{
						NotifyIdx++;
					}
				}
				else if (AnimSequence->Notifies[NotifyIdx].NotifyStateClass)
				{
					const int32 AnimNotifyStart = FMath::Clamp(
						FMath::RoundToInt(Time * Database->GetFrameRate().AsDecimal()),
						0, SequenceFrameNum - 1);

					const int32 AnimNotifyStop = FMath::Clamp(
						FMath::RoundToInt((Time + Duration) * Database->GetFrameRate().AsDecimal() + 1.0f),
						0, SequenceFrameNum);

					const int32 AnimNotifyLength = AnimNotifyStop - AnimNotifyStart;

					if (AnimNotifyLength > 0 && FrameRanges.FrameRangeSet->IntersectsRange(SequenceIdx, AnimNotifyStart, AnimNotifyLength))
					{
						bModified = true;
						AnimSequence->Notifies.RemoveAt(NotifyIdx);
					}
					else
					{
						NotifyIdx++;
					}
				}
			}

			if (bModified)
			{
				AnimSequence->RefreshCacheData();
				AnimSequence->Modify(true);
			}
		}
	}
#endif
}

void UAnimDatabaseLibrary::DatabaseExportNotifies(UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FFilePath& Path)
{
#if WITH_EDITOR
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "DatabaseExportNotifies: Database is null.");
		return;
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "DatabaseExportNotifies: Invalid Frame Ranges.");
		return;
	}

	TSharedRef<FJsonObject> Object = MakeShared<FJsonObject>();
	Object->SetStringField(TEXT("Database"), Database->GetName());
	Object->SetStringField(TEXT("Skeleton"), Database->GetSkeleton() ? Database->GetSkeleton()->GetName() : TEXT(""));
	Object->SetStringField(TEXT("MirrorTable"), Database->GetMirrorDataTable() ? Database->GetMirrorDataTable()->GetName() : TEXT(""));

	TSharedPtr<FJsonObject> FrameRate = MakeShared<FJsonObject>();
	FrameRate->SetNumberField(TEXT("Numerator"), Database->GetFrameRate().Numerator);
	FrameRate->SetNumberField(TEXT("Denominator"), Database->GetFrameRate().Denominator);

	Object->SetObjectField(TEXT("FrameRate"), FrameRate);

	TArray<TSharedPtr<FJsonValue>> SequenceEntries;
	TArray<float> Times, Durations;
	TArray<TSharedPtr<FJsonValue>> AnimNotifies;
	TArray<TSharedPtr<FJsonValue>> AnimNotifyStates;
	TArray<TSharedPtr<FJsonValue>> AnimNotifyTimes;
	TArray<TSharedPtr<FJsonValue>> AnimNotifyStateTimes;
	TArray<TSharedPtr<FJsonValue>> AnimNotifyStateDurations;
	TArray<TSubclassOf<UAnimNotify>> SequenceAnimNotifies;
	TArray<TSubclassOf<UAnimNotifyState>> SequenceAnimNotifyStates;

	const int32 EntryNum = FrameRanges.FrameRangeSet->GetEntryNum();

	for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
	{
		const int32 SequenceIdx = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);
		const int32 SequenceFrameNum = Database->GetSequenceFrameNum(SequenceIdx);

		if (Database->GetIsMirrored(SequenceIdx)) { continue; }

		if (UAnimSequence* AnimSequence = Database->GetAnimSequence(SequenceIdx))
		{
			// Convert Anim Notifies to JSON

			GetAnimSequenceAnimNotifyClasses(SequenceAnimNotifies, AnimSequence);
			AnimNotifies.Reset();

			const int32 AnimNotifyNum = SequenceAnimNotifies.Num();
			for (int32 AnimNotifyIdx = 0; AnimNotifyIdx < AnimNotifyNum; AnimNotifyIdx++)
			{
				GetAnimSequenceAnimNotifyTimes(Times, SequenceAnimNotifies[AnimNotifyIdx], AnimSequence);

				AnimNotifyTimes.Reset();

				const int32 TimeNum = Times.Num();

				for (int32 TimeIdx = 0; TimeIdx < TimeNum; TimeIdx++)
				{
					const int32 AnimNotifyTime = FMath::Clamp(
						FMath::RoundToInt(Times[TimeIdx] * Database->GetFrameRate().AsDecimal()),
						0, SequenceFrameNum - 1);

					if (FrameRanges.FrameRangeSet->Contains(SequenceIdx, AnimNotifyTime))
					{
						AnimNotifyTimes.Add(MakeShared<FJsonValueNumber>(Times[TimeIdx]));
					}
				}

				if (!AnimNotifyTimes.IsEmpty())
				{
					TSharedPtr<FJsonObject> AnimNotifyEntry = MakeShared<FJsonObject>();
					AnimNotifyEntry->SetStringField(TEXT("Name"), SequenceAnimNotifies[AnimNotifyIdx]->GetName());
					AnimNotifyEntry->SetArrayField(TEXT("Times"), AnimNotifyTimes);
					AnimNotifies.Add(MakeShared<FJsonValueObject>(AnimNotifyEntry));
				}
			}

			// Convert Anim Notify States to JSON

			GetAnimSequenceAnimNotifyStateClasses(SequenceAnimNotifyStates, AnimSequence);
			AnimNotifyStates.Reset();

			const int32 AnimNotifyStateNum = SequenceAnimNotifyStates.Num();
			for (int32 AnimNotifyStateIdx = 0; AnimNotifyStateIdx < AnimNotifyStateNum; AnimNotifyStateIdx++)
			{
				GetAnimSequenceAnimNotifyStateTimesAndDurations(Times, Durations, SequenceAnimNotifyStates[AnimNotifyStateIdx], AnimSequence);

				AnimNotifyStateTimes.Reset();
				AnimNotifyStateDurations.Reset();
				
				const int32 TimeNum = Times.Num();
				
				for (int32 TimeIdx = 0; TimeIdx < TimeNum; TimeIdx++)
				{
					const int32 AnimNotifyStart = FMath::Clamp(
						FMath::RoundToInt(Times[TimeIdx] * Database->GetFrameRate().AsDecimal()),
						0, SequenceFrameNum - 1);

					const int32 AnimNotifyStop = FMath::Clamp(
						FMath::RoundToInt((Times[TimeIdx] + Durations[TimeIdx]) * Database->GetFrameRate().AsDecimal() + 1.0f),
						0, SequenceFrameNum);

					const int32 AnimNotifyLength = AnimNotifyStop - AnimNotifyStart;

					if (AnimNotifyLength > 0 && FrameRanges.FrameRangeSet->IntersectsRange(SequenceIdx, AnimNotifyStart, AnimNotifyLength))
					{
						AnimNotifyStateTimes.Add(MakeShared<FJsonValueNumber>(Times[TimeIdx]));
						AnimNotifyStateDurations.Add(MakeShared<FJsonValueNumber>(Durations[TimeIdx]));
					}
				}
				if (!AnimNotifyStateTimes.IsEmpty())
				{
					TSharedPtr<FJsonObject> AnimNotifyStateEntry = MakeShared<FJsonObject>();
					AnimNotifyStateEntry->SetStringField(TEXT("Name"), SequenceAnimNotifyStates[AnimNotifyStateIdx]->GetName());
					AnimNotifyStateEntry->SetArrayField(TEXT("Times"), AnimNotifyStateTimes);
					AnimNotifyStateEntry->SetArrayField(TEXT("Durations"), AnimNotifyStateDurations);
					AnimNotifyStates.Add(MakeShared<FJsonValueObject>(AnimNotifyStateEntry));
				}
			}

			// Write sequence if any notifies or notify states present

			if (!AnimNotifies.IsEmpty() || !AnimNotifyStates.IsEmpty())
			{
				TSharedRef<FJsonObject> SequenceEntry = MakeShared<FJsonObject>();
				SequenceEntry->SetStringField(TEXT("Sequence"), AnimSequence->GetName());
				SequenceEntry->SetArrayField(TEXT("Notifies"), AnimNotifies);
				SequenceEntry->SetArrayField(TEXT("NotifyStates"), AnimNotifyStates);
				SequenceEntries.Add(MakeShared<FJsonValueObject>(SequenceEntry));
			}
		}
	}

	Object->SetArrayField(TEXT("Sequences"), SequenceEntries);

	FString JsonString;
	TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(&JsonString, 0);
	if (FJsonSerializer::Serialize(Object, JsonWriter, true))
	{
		if (!FFileHelper::SaveStringToFile(JsonString, *Path.FilePath))
		{
			UE_LOGFMT(LogAnimDatabase, Error, "DatabaseExportNotifies: Failed to save to file {Filename}.", *Path.FilePath);
		}
	}
#endif
}

void UAnimDatabaseLibrary::LoadAllBlueprintAssetsOfClass(UClass* Class)
{
#if WITH_EDITOR
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	FARFilter FilterA;
	FilterA.ClassPaths.Add(UBlueprint::StaticClass()->GetClassPathName());
	FilterA.bRecursiveClasses = true;
	FilterA.TagsAndValues.Add(TEXT("ParentClass"), FAssetData(Class).GetExportTextName());

	TArray<FAssetData> AssetDataList;
	AssetRegistryModule.Get().GetAssets(FilterA, AssetDataList);

	for (const FAssetData& AssetData : AssetDataList)
	{
		// Load Asset
		AssetData.GetAsset();

		// Load Generated Class
		FString GeneratedClassPath;
		if (AssetData.GetTagValue(TEXT("GeneratedClass"), GeneratedClassPath))
		{
			LoadObject<UClass>(nullptr, *FPackageName::ExportTextPathToObjectPath(GeneratedClassPath));
		}
	}
#endif
}

void UAnimDatabaseLibrary::DatabaseImportNotifies(UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FFilePath& Path)
{
#if WITH_EDITOR
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "DatabaseImportNotifies: Database is null.");
		return;
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "DatabaseImportNotifies: Invalid Frame Ranges.");
		return;
	}

	FString JsonString;
	if (!FFileHelper::LoadFileToString(JsonString, *Path.FilePath))
	{
		UE_LOGFMT(LogAnimDatabase, Error, "DatabaseImportNotifies: Failed to load from file {Filename}.", *Path.FilePath);
		return;
	}

	TSharedPtr<FJsonObject> Object;
	TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(JsonString);

	if (FJsonSerializer::Deserialize(Reader, Object) && Object.IsValid())
	{
		// First we need to make sure all blueprint defined AnimNotifies and AnimNotifyStates are loaded so we can match them by name
		LoadAllBlueprintAssetsOfClass(UAnimNotify::StaticClass());
		LoadAllBlueprintAssetsOfClass(UAnimNotifyState::StaticClass());

		const FString& DatabaseName = Object->GetStringField(TEXT("Database"));
		const FString& SkeletonName = Object->GetStringField(TEXT("Skeleton"));
		const FString& MirrorTableName = Object->GetStringField(TEXT("MirrorTable"));

		if (DatabaseName != Database->GetName())
		{
			UE_LOGFMT(LogAnimDatabase, Warning, "DatabaseImportNotifies: Database Name does not match. Got {Actual}, Expected {Expected}.", *DatabaseName, *Database->GetName());
		}

		if (Database->GetSkeleton() && SkeletonName != Database->GetSkeleton()->GetName())
		{
			UE_LOGFMT(LogAnimDatabase, Warning, "DatabaseImportNotifies: Skeleton Name does not match. Got {Actual}, Expected {Expected}.", *SkeletonName, *Database->GetSkeleton()->GetName());
		}

		if (Database->GetMirrorDataTable() && MirrorTableName != Database->GetMirrorDataTable()->GetName())
		{
			UE_LOGFMT(LogAnimDatabase, Warning, "DatabaseImportNotifies: Mirror Table Name does not match. Got {Actual}, Expected {Expected}.", *MirrorTableName, *Database->GetMirrorDataTable()->GetName());
		}

		const TSharedPtr<FJsonObject> FrameRate = Object->GetObjectField(TEXT("FrameRate"));

		if (FrameRate)
		{
			const int32 Numerator = FrameRate->GetIntegerField(TEXT("Numerator"));
			const int32 Denominator = FrameRate->GetIntegerField(TEXT("Denominator"));

			if (Numerator != Database->GetFrameRate().Numerator ||
				Denominator != Database->GetFrameRate().Denominator)
			{
				UE_LOGFMT(LogAnimDatabase, Warning, "DatabaseImportNotifies: FrameRate doesn't match. Got {Num0}/{Denom0}, expected {Num1}/{Denom1}.",
					Numerator, Denominator, Database->GetFrameRate().Numerator, Database->GetFrameRate().Denominator);
			}
		}
		else
		{
			UE_LOGFMT(LogAnimDatabase, Warning, "DatabaseImportNotifies: No FrameRate Found.");
		}

		const TArray<TSharedPtr<FJsonValue>>& SequenceEntries = Object->GetArrayField(TEXT("Sequences"));

		if (SequenceEntries.IsEmpty())
		{
			UE_LOGFMT(LogAnimDatabase, Warning, "DatabaseImportNotifies: No Sequences Found.");
		}

		TArray<float> Times, Durations;

		const int32 EntryNum = SequenceEntries.Num();

		for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
		{
			if (TSharedPtr<FJsonObject> SequenceEntry = SequenceEntries[EntryIdx]->AsObject())
			{
				const FString& SequenceName = SequenceEntry->GetStringField(TEXT("Sequence"));

				// Find the matching anim sequence asset by name
				TObjectPtr<UAnimSequence>* AnimSequencePtr = Database->Entries.FindByPredicate(
					[SequenceName](TObjectPtr<UAnimSequence> TestSequence) { return TestSequence->GetName() == SequenceName; });

				// If we found a matching sequence and it is not null then load it
				if (AnimSequencePtr)
				{
					// Check if there is a corresponding sequence in the current frame ranges
					const int32 SequenceIdx = Database->FindSequenceIndex(*AnimSequencePtr, false);
					if (SequenceIdx == INDEX_NONE) { continue; }
					if (!FrameRanges.FrameRangeSet->ContainsSequence(SequenceIdx)) { continue; }

					const int32 SequenceFrameNum = Database->GetSequenceFrameNum(SequenceIdx);

					// Iterate over the notifies listed
					const TArray<TSharedPtr<FJsonValue>>& Notifies = SequenceEntry->GetArrayField(TEXT("Notifies"));
					const int32 NotifyNum = Notifies.Num();
					for (int32 NotifyIdx = 0; NotifyIdx < NotifyNum; NotifyIdx++)
					{
						if (TSharedPtr<FJsonObject> NotifyObject = Notifies[NotifyIdx]->AsObject())
						{
							const FString& NotifyClassName = NotifyObject->GetStringField(TEXT("Name"));

							if (UClass* AnimNotifyClass = FindFirstObjectSafe<UClass>(NotifyClassName))
							{
								const TArray<TSharedPtr<FJsonValue>>& TimesArray = NotifyObject->GetArrayField(TEXT("Times"));

								Times.Reset();

								const int32 TimesNum = TimesArray.Num();

								for (int32 TimesIdx = 0; TimesIdx < TimesNum; TimesIdx++)
								{
									const int32 AnimNotifyTime = FMath::Clamp(
										FMath::RoundToInt(TimesArray[TimesIdx]->AsNumber() * Database->GetFrameRate().AsDecimal()),
										0, SequenceFrameNum - 1);

									if (FrameRanges.FrameRangeSet->Contains(SequenceIdx, AnimNotifyTime))
									{
										Times.Add(TimesArray[TimesIdx]->AsNumber());
									}
								}

								FString TrackName = AnimNotifyClass->GetName();
								if (TrackName.EndsWith(TEXT("_C")))
								{
									TrackName = TrackName.LeftChop(2);
								}

								AddAnimNotifiesToSequenceArrayView(*AnimSequencePtr, FName(TrackName), AnimNotifyClass, Times);
							}
							else
							{
								UE_LOGFMT(LogAnimDatabase, Warning, "DatabaseImportNotifies: No Anim Notify Class {Name} Found.", *NotifyClassName);
							}
						}
					}

					const TArray<TSharedPtr<FJsonValue>>& NotifyStates = SequenceEntry->GetArrayField(TEXT("NotifyStates"));
					const int32 NotifyStateNum = NotifyStates.Num();
					for (int32 NotifyStateIdx = 0; NotifyStateIdx < NotifyStateNum; NotifyStateIdx++)
					{
						if (TSharedPtr<FJsonObject> NotifyStateObject = NotifyStates[NotifyStateIdx]->AsObject())
						{
							const FString& NotifyStateClassName = NotifyStateObject->GetStringField(TEXT("Name"));

							if (UClass* AnimNotifyStateClass = FindFirstObjectSafe<UClass>(NotifyStateClassName))
							{
								const TArray<TSharedPtr<FJsonValue>>& TimesArray = NotifyStateObject->GetArrayField(TEXT("Times"));
								const TArray<TSharedPtr<FJsonValue>>& DurationsArray = NotifyStateObject->GetArrayField(TEXT("Durations"));

								Times.Reset();
								Durations.Reset();

								const int32 TimesNum = FMath::Min(TimesArray.Num(), DurationsArray.Num());

								for (int32 TimesIdx = 0; TimesIdx < TimesNum; TimesIdx++)
								{
									const int32 AnimNotifyStart = FMath::Clamp(
										FMath::RoundToInt(TimesArray[TimesIdx]->AsNumber() * Database->GetFrameRate().AsDecimal()),
										0, SequenceFrameNum - 1);

									const int32 AnimNotifyStop = FMath::Clamp(
										FMath::RoundToInt((TimesArray[TimesIdx]->AsNumber() + DurationsArray[TimesIdx]->AsNumber()) * Database->GetFrameRate().AsDecimal() + 1.0f),
										0, SequenceFrameNum);

									const int32 AnimNotifyLength = AnimNotifyStop - AnimNotifyStart;

									if (AnimNotifyLength > 0 && FrameRanges.FrameRangeSet->IntersectsRange(SequenceIdx, AnimNotifyStart, AnimNotifyLength))
									{
										Times.Add(TimesArray[TimesIdx]->AsNumber());
										Durations.Add(DurationsArray[TimesIdx]->AsNumber());
									}
								}

								FString TrackName = AnimNotifyStateClass->GetName();
								if (TrackName.EndsWith(TEXT("_C")))
								{
									TrackName = TrackName.LeftChop(2);
								}

								AddAnimNotifyStatesToSequenceArrayView(*AnimSequencePtr, FName(TrackName), AnimNotifyStateClass, Times, Durations);
							}
							else
							{
								UE_LOGFMT(LogAnimDatabase, Warning, "DatabaseImportNotifies: No Anim Notify State Class {Name} Found.", *NotifyStateClassName);
							}
						}
					}
				}
				else
				{
					UE_LOGFMT(LogAnimDatabase, Warning, "DatabaseImportNotifies: No Sequence {Name} Found in Database.", *SequenceName);
				}
			}
			else
			{
				UE_LOGFMT(LogAnimDatabase, Warning, "DatabaseImportNotifies: No Sequence Entry Found.");
			}
		}
	}
	else
	{
		UE_LOGFMT(LogAnimDatabase, Error, "DatabaseImportNotifies: Failed to parse JSON.");
	}
#endif
}

void UAnimDatabaseLibrary::DatabaseExportAsBVH(UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FDirectoryPath& Path, const bool bAddRootAxesChangeRotation, const EAnimDatabaseRotationOrder RotationOrder)
{
#if WITH_EDITOR
	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "DatabaseExportAsBVH: Invalid Frame Ranges.");
		return;
	}

	if (Path.Path.IsEmpty())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "DatabaseExportAsBVH: Directory is empty.");
		return;
	}

	if (!FPaths::DirectoryExists(Path.Path))
	{
		UE_LOGFMT(LogAnimDatabase, Error, "DatabaseExportAsBVH: Directory {Directory} doesn't exist.", Path.Path);
		return;
	}

	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "DatabaseExportAsBVH: Database is null.");
		return;
	}

	const int32 BoneNum = Database->GetBoneNum();

	TArray<int32> BoneParents;
	Database->GetBoneParents(BoneParents);

	TArray<FName> BoneNames;
	Database->GetBoneNames(BoneNames);

	TArray<FVector> RefBoneLocations;
	Database->GetBoneReferenceLocations(RefBoneLocations);

	TArray<FQuat> RefBoneRotations;
	Database->GetBoneReferenceRotations(RefBoneRotations);

	TArray<int32> BoneOrder;

	FString Indentation;
	FString BVHHeader;
	UE::AnimDatabase::Private::BVHExportHeader(
		BVHHeader,
		Indentation,
		BoneOrder,
		BoneParents,
		BoneNames,
		RefBoneLocations,
		RefBoneRotations,
		RotationOrder);

	UE::Learning::FrameRangeSet::ParallelForEachRange(*FrameRanges.FrameRangeSet, [
		&FrameRanges,
		Database,
		&BoneNames,
		&BoneParents,
		&BoneOrder,
		bAddRootAxesChangeRotation,
		RotationOrder,
		&Path,
		&BVHHeader](
			const int32 TotalRangeIdx, 
			const int32 EntryIdx, 
			const int32 RangeIdx) {

			const int32 SequenceIdx = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);
			const int32 RangeNum = FrameRanges.FrameRangeSet->GetEntryRangeNum(EntryIdx);

			if (UAnimSequence* AnimSequence = Database->GetAnimSequence(SequenceIdx))
			{
				const int32 FrameStart = FrameRanges.FrameRangeSet->GetEntryRangeStart(EntryIdx, RangeIdx);
				const int32 FrameNum = FrameRanges.FrameRangeSet->GetEntryRangeLength(EntryIdx, RangeIdx);
				const int32 BoneNum = Database->GetBoneNum();

				UE::AnimDatabase::FPoseData PoseData;
				UE::AnimDatabase::FPoseGlobalBoneData GlobalPoseData;

				PoseData.Resize(FrameNum, BoneNum, {}, {});
				Database->GetPoseData(PoseData.View(), SequenceIdx, FrameStart, {});

				GlobalPoseData.Resize(FrameNum, BoneNum);
				UE::AnimDatabase::PoseData::ForwardKinematics(
					GlobalPoseData.View(),
					PoseData.LocalBoneData.ConstView(),
					PoseData.RootData.ConstView(),
					BoneParents);

				FString BVHContents = TEXT("MOTION\n");
				BVHContents += FString::Printf(TEXT("Frames: %i\n"), FrameNum);
				BVHContents += FString::Printf(TEXT("Frame Time: %f\n"), 1.0f / FMath::Max(Database->GetFrameRate().AsDecimal(), UE_SMALL_NUMBER));

				for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
				{
					for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
					{
						FVector3f BoneLocation = (FVector3f)GlobalPoseData.BoneLocations[FrameIdx][BoneOrder[BoneIdx]];
						FQuat4f BoneRotation = GlobalPoseData.BoneRotations[FrameIdx][BoneOrder[BoneIdx]];
						FMatrix44f BoneTransform = FTransform3f(BoneRotation, BoneLocation).ToMatrixNoScale();
						for (int32 Idx = 0; Idx < 4; Idx++) { Swap(BoneTransform.M[1][Idx], BoneTransform.M[2][Idx]); }

						for (int32 Idx = 0; Idx < 3; Idx++)
						{
							const float Y = BoneTransform.M[1][Idx];
							const float Z = BoneTransform.M[2][Idx];
							BoneTransform.M[2][Idx] = +Y;
							BoneTransform.M[1][Idx] = -Z;
						}

						FMatrix44f ParentTransform = FMatrix44f::Identity;
						if (BoneParents[BoneOrder[BoneIdx]] != -1)
						{
							FVector3f ParentLocation = (FVector3f)GlobalPoseData.BoneLocations[FrameIdx][BoneParents[BoneOrder[BoneIdx]]];
							FQuat4f ParentRotation = GlobalPoseData.BoneRotations[FrameIdx][BoneParents[BoneOrder[BoneIdx]]];
							ParentTransform = FTransform3f(ParentRotation, ParentLocation).ToMatrixNoScale();
						}

						for (int32 Idx = 0; Idx < 4; Idx++) { Swap(ParentTransform.M[1][Idx], ParentTransform.M[2][Idx]); }

						for (int32 Idx = 0; Idx < 3; Idx++)
						{
							const float Y = ParentTransform.M[1][Idx];
							const float Z = ParentTransform.M[2][Idx];
							ParentTransform.M[2][Idx] = +Y;
							ParentTransform.M[1][Idx] = -Z;
						}

						FMatrix44f LocalTransform = BoneTransform * ParentTransform.Inverse();
						FQuat4f LocalRotation = UE::AnimDatabase::Math::MakeQuatFromMatrixRightHanded(LocalTransform).GetNormalized();
						FVector3f LocalLocation = FVector3f(LocalTransform.M[3][0], LocalTransform.M[3][1], LocalTransform.M[3][2]);

						if (bAddRootAxesChangeRotation && BoneIdx == 0)
						{
							const FQuat4f Adjustment = FQuat4f::MakeFromRotationVector(FVector3f(-UE_HALF_PI, 0.0f, 0.0f));
							LocalRotation = Adjustment * LocalRotation;
							LocalLocation = Adjustment.RotateVector(LocalLocation);
						}

						FVector3f LocalEuler = FMath::RadiansToDegrees(UE::AnimDatabase::Private::QuatToEuler(LocalRotation, RotationOrder));

						BVHContents += FString::Printf(TEXT("%.9g %.9g %.9g %.9g %.9g %.9g "),
							LocalLocation.X, LocalLocation.Y, LocalLocation.Z,
							LocalEuler.X, LocalEuler.Y, LocalEuler.Z);
					}

					BVHContents += TEXT("\n");
				}

				FString FileName = AnimSequence->GetName() + FString::Printf(TEXT("_%i_%i"), FrameStart, FrameStart + FrameNum);
				if (Database->GetIsMirrored(SequenceIdx)) { FileName += TEXT("_mirrored"); }
				FileName += TEXT(".bvh");

				const FString FilePath = Path.Path + TEXT("/") + FileName;

				if (!FFileHelper::SaveStringToFile(BVHHeader + BVHContents, *FilePath))
				{
					UE_LOGFMT(LogAnimDatabase, Error, "DatabaseExportAsBVH: Failed to save to file {Filename}.", *FilePath);
				}
			}
		});
#endif
}

void UAnimDatabaseLibrary::DatabaseImportFromBVH(UAnimDatabase* Database, const FDirectoryPath& ImportPath, const FDirectoryPath& AssetPath, const bool bRemoveRootAxesChangeRotation, const bool bAddToDatabase, const bool bIgnoreMirrored)
{
#if WITH_EDITOR
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "DatabaseImportFromBVH: Database is null.");
		return;
	}

	if (ImportPath.Path.IsEmpty())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "DatabaseImportFromBVH: ImportPath is empty.");
		return;
	}

	if (AssetPath.Path.IsEmpty())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "DatabaseImportFromBVH: AssetPath is empty.");
		return;
	}

	if (!FPaths::DirectoryExists(ImportPath.Path))
	{
		UE_LOGFMT(LogAnimDatabase, Error, "DatabaseImportFromBVH: Directory {Directory} doesn't exist.", ImportPath.Path);
		return;
	}

	if (!FPaths::DirectoryExists(AssetPath.Path))
	{
		UE_LOGFMT(LogAnimDatabase, Error, "DatabaseImportFromBVH: Directory {Directory} doesn't exist.", AssetPath.Path);
		return;
	}


	IFileManager& FileManager = IFileManager::Get();

	TArray<FString> FoundFiles;
	FileManager.FindFiles(FoundFiles, *(ImportPath.Path + TEXT("/*.bvh")), true, false);

	FScopedSlowTask SlowTask(FoundFiles.Num(), LOCTEXT("ImportingFromBVH", "Importing From BVH"));
	SlowTask.MakeDialog(true);

	for (const FString& File : FoundFiles)
	{
		SlowTask.EnterProgressFrame();
		if (SlowTask.ShouldCancel()) { break; }

		if (bIgnoreMirrored && File.EndsWith(TEXT("_mirrored.bvh"))) { continue; }

		FString BVHContents;
		if (FFileHelper::LoadFileToString(BVHContents, *(ImportPath.Path + TEXT("/") + File)))
		{
			int32 ActiveIdx = INDEX_NONE;
			bool bEndSite = false;
			int32 FrameNum = 0;
			int32 FrameIdx = 0;
			float FrameTime = 0.0f;
			int32 ChannelNum = 0;
			EAnimDatabaseRotationOrder RotationOrder = EAnimDatabaseRotationOrder::XYZ;
			TArray<int32> LocationOrder = { 0, 1, 2 };

			TArray<FString> BoneNames;
			TArray<int32> BoneParents;
			TArray<FQuat4f> BoneOrients;
			TArray<FVector3f> BoneOffsets;
			TLearningArray<2, FVector3f> BoneLocations;
			TLearningArray<2, FQuat4f> BoneRotations;

			TArray<FString> Lines;
			BVHContents.ParseIntoArrayLines(Lines);
			for (const FString& Line : Lines)
			{
				if (Line.Contains(TEXT("HIERARCHY"))) { continue; }
				if (Line.Contains(TEXT("MOTION"))) { continue; }

				if (Line.Contains(TEXT("ROOT")) || Line.Contains(TEXT("JOINT")))
				{
					TArray<FString> Parts;
					Line.ParseIntoArray(Parts, TEXT(" "), true);
					BoneNames.Add(ensure(Parts.Num() == 2) ? Parts[1] : TEXT(""));
					BoneOrients.Add(FQuat4f::Identity);
					BoneOffsets.Add(FVector3f::ZeroVector);
					BoneParents.Add(ActiveIdx);
					ActiveIdx = BoneParents.Num() - 1;
					continue;
				}

				if (Line.Contains(TEXT("{"))) { continue; }

				if (Line.Contains(TEXT("}")))
				{
					if (bEndSite)
					{
						bEndSite = false;
					}
					else
					{
						ActiveIdx = ActiveIdx == INDEX_NONE ? INDEX_NONE : BoneParents[ActiveIdx];
					}
					continue;
				}

				if (Line.Contains(TEXT("OFFSET")))
				{
					TArray<FString> Parts;
					Line.ParseIntoArray(Parts, TEXT(" "), true);

					if (!bEndSite && ActiveIdx != INDEX_NONE)
					{
						BoneOffsets[ActiveIdx] = ensure(Parts.Num() == 4) ? 
							FVector3f(
								FCString::Atof(*Parts[1]),
								FCString::Atof(*Parts[2]),
								FCString::Atof(*Parts[3])) : FVector3f::ZeroVector;
					}
					continue;
				}

				if (Line.Contains(TEXT("CHANNELS")))
				{
					TArray<FString> Parts;
					Line.ParseIntoArray(Parts, TEXT(" "), true);
					
					if (ensure(Parts.Num() > 1))
					{
						ChannelNum = FCString::Atoi(*Parts[1]);
					}

					if (Parts.Num() == 2 + 3)
					{
						RotationOrder = UE::AnimDatabase::Private::RotationOrderFromStrings(Parts[2], Parts[3], Parts[4]);
					}
					else if (Parts.Num() == 2 + 6)
					{
						UE::AnimDatabase::Private::LocationOrderFromStrings(LocationOrder, Parts[2], Parts[3], Parts[4]);
						RotationOrder = UE::AnimDatabase::Private::RotationOrderFromStrings(Parts[5], Parts[6], Parts[7]);
					}
					continue;
				}

				if (Line.Contains(TEXT("End Site")))
				{
					bEndSite = true;
					continue;
				}

				if (Line.Contains(TEXT("Frames: ")))
				{
					TArray<FString> Parts;
					Line.ParseIntoArray(Parts, TEXT(" "), true);
					FrameNum = ensure(Parts.Num() == 2) ? FCString::Atoi(*Parts[1]) : 0;
					BoneLocations.SetNumUninitialized({ FrameNum, BoneNames.Num() });
					BoneRotations.SetNumUninitialized({ FrameNum, BoneNames.Num() });
					for (int32 FillFrameIdx = 0; FillFrameIdx < FrameNum; FillFrameIdx++)
					{
						UE::Learning::Array::Copy(BoneLocations[FillFrameIdx], TLearningArrayView<1, const FVector3f>(BoneOffsets));
						UE::Learning::Array::Copy(BoneRotations[FillFrameIdx], TLearningArrayView<1, const FQuat4f>(BoneOrients));
					}
					continue;
				}

				if (Line.Contains(TEXT("Frame Time: ")))
				{
					TArray<FString> Parts;
					Line.ParseIntoArray(Parts, TEXT(" "), true);
					FrameTime = ensure(Parts.Num() == 3) ? FCString::Atof(*Parts[2]) : 0;
					continue;
				}

				const int32 BoneNum = BoneNames.Num();

				TArray<FString> Parts;
				Line.ParseIntoArray(Parts, TEXT(" "), true);

				if (ChannelNum == 3)
				{
					if (ensure(Parts.Num() == ChannelNum * BoneNum + ChannelNum && FrameIdx < FrameNum))
					{
						BoneLocations[FrameIdx][0] = FVector3f(
							FCString::Atof(*Parts[0]),
							FCString::Atof(*Parts[1]), 
							FCString::Atof(*Parts[2]));

						for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
						{
							BoneRotations[FrameIdx][BoneIdx] = UE::AnimDatabase::Private::QuatFromEuler(FVector3f(
								FMath::DegreesToRadians(FCString::Atof(*Parts[3 + BoneIdx * 3 + 0])),
								FMath::DegreesToRadians(FCString::Atof(*Parts[3 + BoneIdx * 3 + 1])),
								FMath::DegreesToRadians(FCString::Atof(*Parts[3 + BoneIdx * 3 + 2]))), RotationOrder);
						}
					}

					FrameIdx++;
					continue;
				}
				
				if (ChannelNum == 6)
				{
					if (ensure(Parts.Num() == ChannelNum * BoneNum && FrameIdx < FrameNum))
					{
						for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
						{
							BoneLocations[FrameIdx][BoneIdx] = FVector3f(
								FCString::Atof(*Parts[BoneIdx * 6 + LocationOrder[0]]),
								FCString::Atof(*Parts[BoneIdx * 6 + LocationOrder[1]]),
								FCString::Atof(*Parts[BoneIdx * 6 + LocationOrder[2]]));

							BoneRotations[FrameIdx][BoneIdx] = UE::AnimDatabase::Private::QuatFromEuler(FVector3f(
								FMath::DegreesToRadians(FCString::Atof(*Parts[BoneIdx * 6 + 3])),
								FMath::DegreesToRadians(FCString::Atof(*Parts[BoneIdx * 6 + 4])),
								FMath::DegreesToRadians(FCString::Atof(*Parts[BoneIdx * 6 + 5]))), RotationOrder);
						}
					}

					FrameIdx++;
					continue;
				}

				// Didn't parse line correctly...
				ensure(false);
			}

			if (!ensure(FrameIdx == FrameNum && FrameNum > 0)) { continue; }

			const int32 BVHBoneNum = BoneNames.Num();

			if (bRemoveRootAxesChangeRotation && BVHBoneNum > 0)
			{
				for (int32 LocalFrameIdx = 0; LocalFrameIdx < FrameNum; LocalFrameIdx++)
				{
					const FQuat4f Adjustment = FQuat4f::MakeFromRotationVector(FVector3f(+UE_HALF_PI, 0.0f, 0.0f));
					BoneRotations[LocalFrameIdx][0] = Adjustment * BoneRotations[LocalFrameIdx][0];
					BoneLocations[LocalFrameIdx][0] = Adjustment.RotateVector(BoneLocations[LocalFrameIdx][0]);
				}
			}

			const int32 DatabaseBoneNum = Database->GetBoneNum();

			UE::AnimDatabase::FPoseData PoseData;
			PoseData.Resize(FrameNum, DatabaseBoneNum, {}, {});
			UE::AnimDatabase::PoseData::Reset(PoseData.View(), Database->GetSkeleton()->GetReferenceSkeleton().GetRefBonePose());

			TLearningArray<2, FVector3f> GlobalBoneLocations = BoneLocations;
			TLearningArray<2, FQuat4f> GlobalBoneRotations = BoneRotations;
			for (int32 LocalFrameIdx = 0; LocalFrameIdx < FrameNum; LocalFrameIdx++)
			{
				for (int32 BoneIdx = 0; BoneIdx < BVHBoneNum; BoneIdx++)
				{
					const int32 ParentIdx = BoneParents[BoneIdx];
					check(ParentIdx < BoneIdx);
					if (ParentIdx != INDEX_NONE)
					{
						GlobalBoneLocations[LocalFrameIdx][BoneIdx] = GlobalBoneRotations[LocalFrameIdx][ParentIdx].RotateVector(GlobalBoneLocations[LocalFrameIdx][BoneIdx]) + GlobalBoneLocations[LocalFrameIdx][ParentIdx];
						GlobalBoneRotations[LocalFrameIdx][BoneIdx] = GlobalBoneRotations[LocalFrameIdx][ParentIdx] * GlobalBoneRotations[LocalFrameIdx][BoneIdx];
					}
				}
			}

			for (int32 LocalFrameIdx = 0; LocalFrameIdx < FrameNum; LocalFrameIdx++)
			{
				for (int32 BoneIdx = 0; BoneIdx < BVHBoneNum; BoneIdx++)
				{
					const int32 DatabaseBoneIdx = Database->FindBoneIndex(FName(BoneNames[BoneIdx]));
					if (DatabaseBoneIdx == INDEX_NONE) { continue; }

					const FQuat4f BoneRotation = GlobalBoneRotations[LocalFrameIdx][BoneIdx];
					const FVector3f BoneLocation = GlobalBoneLocations[LocalFrameIdx][BoneIdx];

					FMatrix44f BoneTransform = FTransform3f(GlobalBoneRotations[LocalFrameIdx][BoneIdx], GlobalBoneLocations[LocalFrameIdx][BoneIdx]).ToMatrixNoScale();
					for (int32 Idx = 0; Idx < 4; Idx++) { Swap(BoneTransform.M[1][Idx], BoneTransform.M[2][Idx]); }

					for (int32 Idx = 0; Idx < 3; Idx++)
					{
						const float Y = BoneTransform.M[1][Idx];
						const float Z = BoneTransform.M[2][Idx];
						BoneTransform.M[2][Idx] = +Y;
						BoneTransform.M[1][Idx] = -Z;
					}

					FMatrix44f ParentTransform = FMatrix44f::Identity;
					if (BoneParents[BoneIdx] != INDEX_NONE)
					{
						FVector3f ParentLocation = GlobalBoneLocations[LocalFrameIdx][BoneParents[BoneIdx]];
						FQuat4f ParentRotation = GlobalBoneRotations[LocalFrameIdx][BoneParents[BoneIdx]];
						ParentTransform = FTransform3f(ParentRotation, ParentLocation).ToMatrixNoScale();
					}

					for (int32 Idx = 0; Idx < 4; Idx++) { Swap(ParentTransform.M[1][Idx], ParentTransform.M[2][Idx]); }

					for (int32 Idx = 0; Idx < 3; Idx++)
					{
						const float Y = ParentTransform.M[1][Idx];
						const float Z = ParentTransform.M[2][Idx];
						ParentTransform.M[2][Idx] = +Y;
						ParentTransform.M[1][Idx] = -Z;
					}

					const FMatrix44f LocalTransform = BoneTransform * ParentTransform.Inverse();
					const FQuat4f LocalRotation = UE::AnimDatabase::Math::MakeQuatFromMatrixRightHanded(LocalTransform).GetNormalized();
					const FVector3f LocalLocation = FVector3f(LocalTransform.M[3][0], LocalTransform.M[3][1], LocalTransform.M[3][2]);

					if (BoneIdx == 0)
					{
						PoseData.RootData.RootLocations[LocalFrameIdx] = (FVector)LocalLocation;
						PoseData.RootData.RootRotations[LocalFrameIdx] = LocalRotation;
					}
					else
					{
						PoseData.LocalBoneData.BoneLocations[LocalFrameIdx][DatabaseBoneIdx] = LocalLocation;
						PoseData.LocalBoneData.BoneRotations[LocalFrameIdx][DatabaseBoneIdx] = LocalRotation;
					}
				}
			}


			TArray<int32> UsedBones;
			UE::AnimDatabase::PoseData::ComputeUsedBones(UsedBones, PoseData.LocalBoneData.ConstView(), Database->Skeleton->GetReferenceSkeleton().GetRefBonePose());

			// Create Asset

			FString AssetName = File;
			AssetName.RemoveFromEnd(TEXT(".bvh"));

			FString PathPackageName;
			if (!FPackageName::TryConvertFilenameToLongPackageName(AssetPath.Path, PathPackageName))
			{
				UE_LOGFMT(LogAnimDatabase, Error, "DatabaseImportFromBVH: Unable to convert path '{Path}' to package name.", AssetPath.Path);
				continue;
			}

			const FString PackageName = PathPackageName + TEXT("/") + AssetName;

			UPackage* Package = CreatePackage(*PackageName);
			if (!Package)
			{
				UE_LOGFMT(LogAnimDatabase, Error, "DatabaseImportFromBVH: Unable to create package '{PackageName}'.", PackageName);
				continue;
			}
			Package->FullyLoad();

			// Find or Create Anim Sequence

			UAnimSequence* NewAnimSequence = FindObject<UAnimSequence>(nullptr, PackageName + TEXT(".") + AssetName);
			if (!NewAnimSequence)
			{
				NewAnimSequence = NewObject<UAnimSequence>(Package, FName(*AssetName), RF_Public | RF_Standalone);
			}

			if (!NewAnimSequence)
			{
				UE_LOGFMT(LogAnimDatabase, Error, "DatabaseImportFromBVH: Unable to create new AnimSequence.");
				continue;
			}

			// Setup Sequence

			const FFrameRate AnimFrameRate = FFrameRate(FMath::RoundToInt(1.0f / FMath::Max(FrameTime, UE_SMALL_NUMBER)), 1);
			if (AnimFrameRate != Database->GetFrameRate())
			{
				UE_LOGFMT(LogAnimDatabase, Warning, "DatabaseImportFromBVH: Animation FrameRate ({0}/{1}) does not match database ({2}/{0}).", 
					AnimFrameRate.Numerator, AnimFrameRate.Denominator, Database->GetFrameRate().Numerator, Database->GetFrameRate().Denominator);
			}

			const bool bShouldTransact = false;

			NewAnimSequence->ResetAnimation();
			NewAnimSequence->SetSkeleton(Database->Skeleton);
			NewAnimSequence->bForceRootLock = true;
			NewAnimSequence->bEnableRootMotion = true;

			TScriptInterface<IAnimationDataController> Controller = NewAnimSequence->GetDataModel()->GetController();
			Controller->OpenBracket(LOCTEXT("CreateAnimSequence", "Creating Animation Sequence"), bShouldTransact);
			Controller->InitializeModel();
			Controller->SetFrameRate(AnimFrameRate, bShouldTransact);
			Controller->SetNumberOfFrames(FrameNum - 1, bShouldTransact);

			// Key initial bone data to reference pose

			TArray<FVector> AssetBonePositions;
			TArray<FQuat> AssetBoneRotations;
			TArray<FVector> AssetBoneScales;
			AssetBonePositions.SetNumUninitialized(FrameNum);
			AssetBoneRotations.SetNumUninitialized(FrameNum);
			AssetBoneScales.SetNumUninitialized(FrameNum);

			const FReferenceSkeleton ReferenceSkeleton = Database->GetSkeleton()->GetReferenceSkeleton();

			for (int32 LocalBoneIdx : UsedBones)
			{
				const FTransform RefTransform = ReferenceSkeleton.GetRefBonePose()[LocalBoneIdx];
				const FName BoneName = ReferenceSkeleton.GetBoneName(LocalBoneIdx);
				check(BoneName != NAME_None);

				if (!Controller->GetModel()->IsValidBoneTrackName(BoneName))
				{
					Controller->AddBoneCurve(BoneName, bShouldTransact);
				}

				UE::Learning::Array::Set<1, FVector>(AssetBonePositions, RefTransform.GetLocation());
				UE::Learning::Array::Set<1, FQuat>(AssetBoneRotations, RefTransform.GetRotation());
				UE::Learning::Array::Set<1, FVector>(AssetBoneScales, RefTransform.GetScale3D());

				Controller->SetBoneTrackKeys(BoneName, AssetBonePositions, AssetBoneRotations, AssetBoneScales, bShouldTransact);
			}

			// Set it to the actual animation data

			SetAnimSequencePoseData(NewAnimSequence, Controller, 0, PoseData.ConstView(), UsedBones, bShouldTransact);

			Controller->NotifyPopulated();
			Controller->CloseBracket(bShouldTransact);

			NewAnimSequence->RefreshCacheData();

			// Inform the asset registry about the creation.
			Package->MarkPackageDirty();
			FAssetRegistryModule::AssetCreated(NewAnimSequence);

			// Add to Database
			if (bAddToDatabase)
			{
				Database->Entries.AddUnique(NewAnimSequence);
				Database->Modify();
			}
		}
	}
#endif
}

void UAnimDatabaseLibrary::DatabaseExportAsAnimSequences(UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FDirectoryPath& Path, const FString& AssetNameFormatString, const bool bExportMirrored, const bool bShouldTransact)
{
#if WITH_EDITOR
	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "DatabaseExportAsAnimSequences: Invalid Frame Ranges.");
		return;
	}

	if (Path.Path.IsEmpty())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "DatabaseExportAsAnimSequences: Directory is empty.");
		return;
	}

	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "DatabaseExportAsAnimSequences: Database is null.");
		return;
	}

	const int32 BoneNum = Database->GetBoneNum();

	UE::AnimDatabase::FPoseData PoseData;
	UE::AnimDatabase::FPoseGlobalBoneData GlobalPoseData;

	FScopedSlowTask SlowTask(FrameRanges.FrameRangeSet->GetTotalRangeNum(), LOCTEXT("ExportingToAnimSequences", "Exporting to Anim Sequences"));
	SlowTask.MakeDialog(true);

	const int32 EntryNum = FrameRanges.FrameRangeSet->GetEntryNum();

	for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
	{
		const int32 SequenceIdx = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);
		const int32 RangeNum = FrameRanges.FrameRangeSet->GetEntryRangeNum(EntryIdx);

		if (!bExportMirrored && Database->GetIsMirrored(SequenceIdx)) { continue; }

		if (UAnimSequence* AnimSequence = Database->GetAnimSequence(SequenceIdx))
		{
			for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
			{
				SlowTask.EnterProgressFrame();
				if (SlowTask.ShouldCancel()) { return; }

				const int32 FrameStart = FrameRanges.FrameRangeSet->GetEntryRangeStart(EntryIdx, RangeIdx);
				const int32 FrameNum = FrameRanges.FrameRangeSet->GetEntryRangeLength(EntryIdx, RangeIdx);

				if (FrameNum < 1)
				{
					UE_LOGFMT(LogAnimDatabase, Error, "DatabaseExportAsAnimSequences: Frame range too short. Must be at least 1 frames, got {FrameNum}.", FrameNum);
					continue;
				}

				PoseData.Resize(FrameNum, BoneNum, {}, {});
				Database->GetPoseData(PoseData.View(), SequenceIdx, FrameStart, {});

				TArray<int32> UsedBones;
				UE::AnimDatabase::PoseData::ComputeUsedBones(UsedBones, PoseData.LocalBoneData.ConstView(), Database->Skeleton->GetReferenceSkeleton().GetRefBonePose());

				// Create Asset

				FString AssetName = AssetNameFormatString;
				AssetName = AssetName.Replace(TEXT("{SequenceName}"), *AnimSequence->GetName());
				AssetName = AssetName.Replace(TEXT("{SequenceIdx}"), *FString::FromInt(SequenceIdx));
				AssetName = AssetName.Replace(TEXT("{RangeIdx}"), *FString::FromInt(RangeIdx));
				AssetName = AssetName.Replace(TEXT("{StartFrame}"), *FString::FromInt(FrameStart));
				AssetName = AssetName.Replace(TEXT("{EndFrame}"), *FString::FromInt(FrameStart + FrameNum));
				AssetName = AssetName.Replace(TEXT("{Mirrored}"), Database->GetIsMirrored(SequenceIdx) ? TEXT("Mirrored") : TEXT("Unmirrored"));

				FString PathPackageName;
				if (!FPackageName::TryConvertFilenameToLongPackageName(Path.Path, PathPackageName))
				{
					UE_LOGFMT(LogAnimDatabase, Error, "DatabaseExportAsAnimSequences: Unable to convert path '{Path}' to package name.", Path.Path);
					continue;
				}

				const FString PackageName = PathPackageName + TEXT("/") + AssetName;

				UPackage* Package = CreatePackage(*PackageName);
				if (!Package)
				{
					UE_LOGFMT(LogAnimDatabase, Error, "DatabaseExportAsAnimSequences: Unable to create package '{PackageName}'.", PackageName);
					continue;
				}
				Package->FullyLoad();

				// Find or Create Anim Sequence

				UAnimSequence* NewAnimSequence = FindObject<UAnimSequence>(nullptr, PackageName + TEXT(".") + AssetName);
				if (!NewAnimSequence)
				{
					NewAnimSequence = NewObject<UAnimSequence>(Package, FName(*AssetName), RF_Public | RF_Standalone);
				}

				if (!NewAnimSequence)
				{
					UE_LOGFMT(LogAnimDatabase, Error, "DatabaseExportAsAnimSequences: Unable to create new AnimSequence.");
					continue;
				}

				// Setup Sequence

				NewAnimSequence->ResetAnimation();
				NewAnimSequence->SetSkeleton(Database->Skeleton);
				NewAnimSequence->bForceRootLock = true;
				NewAnimSequence->bEnableRootMotion = true;

				TScriptInterface<IAnimationDataController> Controller = NewAnimSequence->GetDataModel()->GetController();
				Controller->OpenBracket(LOCTEXT("CreateAnimSequence", "Creating Animation Sequence"), bShouldTransact);
				Controller->InitializeModel();
				Controller->SetFrameRate(Database->GetFrameRate(), bShouldTransact);
				Controller->SetNumberOfFrames(FrameNum - 1, bShouldTransact);

				// Key initial bone data to reference pose

				TArray<FVector> BonePositions;
				TArray<FQuat> BoneRotations;
				TArray<FVector> BoneScales;
				BonePositions.SetNumUninitialized(FrameNum);
				BoneRotations.SetNumUninitialized(FrameNum);
				BoneScales.SetNumUninitialized(FrameNum);

				const FReferenceSkeleton ReferenceSkeleton = AnimSequence->GetSkeleton()->GetReferenceSkeleton();
				check(ReferenceSkeleton.GetRefBonePose().Num() == BoneNum);

				for (int32 BoneIdx : UsedBones)
				{
					const FTransform RefTransform = ReferenceSkeleton.GetRefBonePose()[BoneIdx];
					const FName BoneName = ReferenceSkeleton.GetBoneName(BoneIdx);
					check(BoneName != NAME_None);

					if (!Controller->GetModel()->IsValidBoneTrackName(BoneName))
					{
						Controller->AddBoneCurve(BoneName, bShouldTransact);
					}

					UE::Learning::Array::Set<1, FVector>(BonePositions, RefTransform.GetLocation());
					UE::Learning::Array::Set<1, FQuat>(BoneRotations, RefTransform.GetRotation());
					UE::Learning::Array::Set<1, FVector>(BoneScales, RefTransform.GetScale3D());

					Controller->SetBoneTrackKeys(BoneName, BonePositions, BoneRotations, BoneScales, bShouldTransact);
				}

				// Set it to the actual animation data

				SetAnimSequencePoseData(NewAnimSequence, Controller, 0, PoseData.ConstView(), UsedBones, bShouldTransact);

				Controller->NotifyPopulated();
				Controller->CloseBracket(bShouldTransact);

				NewAnimSequence->RefreshCacheData();

				// Inform the asset registry about the creation.
				Package->MarkPackageDirty();
				FAssetRegistryModule::AssetCreated(NewAnimSequence);
			}
		}
	}
#endif
}

void UAnimDatabaseLibrary::DatabaseExportRootMotion(UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FFilePath& Path)
{
#if WITH_EDITOR
	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "DatabaseExportRootMotion: Invalid Frame Ranges.");
		return;
	}

	if (Path.FilePath.IsEmpty())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "DatabaseExportRootMotion: Directory is empty.");
		return;
	}

	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "DatabaseExportRootMotion: Database is null.");
		return;
	}

	TSharedRef<FJsonObject> Object = MakeShared<FJsonObject>();
	Object->SetStringField(TEXT("Database"), Database->GetName());
	Object->SetStringField(TEXT("Skeleton"), Database->GetSkeleton() ? Database->GetSkeleton()->GetName() : TEXT(""));
	Object->SetStringField(TEXT("MirrorTable"), Database->GetMirrorDataTable() ? Database->GetMirrorDataTable()->GetName() : TEXT(""));

	TSharedPtr<FJsonObject> FrameRate = MakeShared<FJsonObject>();
	FrameRate->SetNumberField(TEXT("Numerator"), Database->GetFrameRate().Numerator);
	FrameRate->SetNumberField(TEXT("Denominator"), Database->GetFrameRate().Denominator);

	Object->SetObjectField(TEXT("FrameRate"), FrameRate);

	TLearningArray<1, FVector> RootLocations;
	TLearningArray<1, FQuat4f> RootRotations;
	TArray<TSharedPtr<FJsonValue>> SequenceEntries;
	TArray<TSharedPtr<FJsonValue>> RangeLocations;
	TArray<TSharedPtr<FJsonValue>> RangeRotations;
	TArray<TSharedPtr<FJsonValue>> RangeEntries;
	TArray<TSharedPtr<FJsonValue>> Location;
	TArray<TSharedPtr<FJsonValue>> Rotation;

	Location.SetNum(3);
	Rotation.SetNum(4);

	const int32 EntryNum = FrameRanges.FrameRangeSet->GetEntryNum();

	for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
	{
		const int32 SequenceIdx = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);
		const int32 RangeNum = FrameRanges.FrameRangeSet->GetEntryRangeNum(EntryIdx);

		if (UAnimSequence* AnimSequence = Database->GetAnimSequence(SequenceIdx))
		{
			RangeEntries.Reset();

			for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
			{
				const int32 FrameStart = FrameRanges.FrameRangeSet->GetEntryRangeStart(EntryIdx, RangeIdx);
				const int32 FrameNum = FrameRanges.FrameRangeSet->GetEntryRangeLength(EntryIdx, RangeIdx);

				RootLocations.SetNumUninitialized({ FrameNum });
				RootRotations.SetNumUninitialized({ FrameNum });
				Database->GetRootLocation(RootLocations, SequenceIdx, FrameStart);
				Database->GetRootRotation(RootRotations, SequenceIdx, FrameStart);

				RangeLocations.Reset();
				RangeRotations.Reset();
				for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
				{
					Location[0] = MakeShared<FJsonValueNumber>(RootLocations[FrameIdx].X);
					Location[1] = MakeShared<FJsonValueNumber>(RootLocations[FrameIdx].Y);
					Location[2] = MakeShared<FJsonValueNumber>(RootLocations[FrameIdx].Z);

					Rotation[0] = MakeShared<FJsonValueNumber>(RootRotations[FrameIdx].X);
					Rotation[1] = MakeShared<FJsonValueNumber>(RootRotations[FrameIdx].Y);
					Rotation[2] = MakeShared<FJsonValueNumber>(RootRotations[FrameIdx].Z);
					Rotation[3] = MakeShared<FJsonValueNumber>(RootRotations[FrameIdx].W);

					RangeLocations.Add(MakeShared<FJsonValueArray>(Location));
					RangeRotations.Add(MakeShared<FJsonValueArray>(Rotation));
				}

				TSharedRef<FJsonObject> RangeEntry = MakeShared<FJsonObject>();
				RangeEntry->SetNumberField(TEXT("FrameStart"), FrameStart);
				RangeEntry->SetNumberField(TEXT("FrameNum"), FrameNum);
				RangeEntry->SetArrayField(TEXT("Locations"), RangeLocations);
				RangeEntry->SetArrayField(TEXT("Rotations"), RangeRotations);
				RangeEntries.Add(MakeShared<FJsonValueObject>(RangeEntry));
			}

			TSharedRef<FJsonObject> SequenceEntry = MakeShared<FJsonObject>();
			SequenceEntry->SetStringField(TEXT("Sequence"), AnimSequence->GetName());
			SequenceEntry->SetArrayField(TEXT("Ranges"), RangeEntries);
			SequenceEntries.Add(MakeShared<FJsonValueObject>(SequenceEntry));
		}
	}

	Object->SetArrayField(TEXT("Sequences"), SequenceEntries);

	FString JsonString;
	TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(&JsonString, 0);
	if (FJsonSerializer::Serialize(Object, JsonWriter, true))
	{
		if (!FFileHelper::SaveStringToFile(JsonString, *Path.FilePath))
		{
			UE_LOGFMT(LogAnimDatabase, Error, "DatabaseExportRootMotion: Failed to save to file {Filename}.", *Path.FilePath);
		}
	}
#endif
}

void UAnimDatabaseLibrary::DatabaseAddCurve(UAnimDatabase* Database, const FName CurveName, const FAnimDatabaseFrameAttribute& FloatFrameAttribute, const FLinearColor Color, const bool bSparseKeys, const bool bShouldTransact)
{
	DatabaseAddCurvesFromArrayViews(Database, { CurveName }, { FloatFrameAttribute }, { Color }, bSparseKeys, bShouldTransact);
}

void UAnimDatabaseLibrary::DatabaseAddCurves(UAnimDatabase* Database, const TArray<FName>& CurveNames, const TArray<FAnimDatabaseFrameAttribute>& FloatFrameAttributes, const TArray<FLinearColor>& Colors, const bool bSparseKeys, const bool bShouldTransact)
{
	DatabaseAddCurvesFromArrayViews(Database, CurveNames, FloatFrameAttributes, Colors, bSparseKeys, bShouldTransact);
}

void UAnimDatabaseLibrary::DatabaseAddCurvesFromArrayViews(UAnimDatabase* Database, const TArrayView<const FName> CurveNames, const TArrayView<const FAnimDatabaseFrameAttribute> FloatFrameAttributes, const TArrayView<const FLinearColor> Colors, const bool bSparseKeys, const bool bShouldTransact)
{
#if WITH_EDITOR
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseLibrary::DatabaseAddCurvesFromArrayViews);

	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "DatabaseAddCurvesFromArrayViews: Database is null.");
		return;
	}

	if (CurveNames.Num() != FloatFrameAttributes.Num())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "DatabaseAddCurvesFromArrayViews: Number of Curve names and Float FrameAttributes must match. Got {Actual} and {Expected}.", CurveNames.Num(), FloatFrameAttributes.Num());
		return;
	}

	const int32 CurveNum = CurveNames.Num();

	for (int32 CurveIdx = 0; CurveIdx < CurveNum; CurveIdx++)
	{
		if (FloatFrameAttributes[CurveIdx].Type != EAnimDatabaseAttributeType::Float)
		{
			UE_LOGFMT(LogAnimDatabase, Error, "DatabaseAddCurvesFromArrayViews: Incorrect FrameAttribute Type. Got {Type}. Expected Float.", *UEnum::GetValueAsString(FloatFrameAttributes[CurveIdx].Type));
			return;
		}
	}

	// To avoid triggering recompression during modification we need to open brackets on all the controllers for all the to-be-modified animation 
	// sequences once, and then close them all at the end.

	TArray<TScriptInterface<IAnimationDataController>> Controllers;
	TMap<int32, int32> SequenceControllerIndices;

	UE::AnimDatabase::Private::OpenControllers(
		Controllers, SequenceControllerIndices, Database, FloatFrameAttributes,
		LOCTEXT("AddCurveScript", "Adding Curve"), bShouldTransact);

	for (int32 CurveIdx = 0; CurveIdx < CurveNum; CurveIdx++)
	{
		const FAnimDatabaseFrameAttribute FilteredFrameAttribute = UAnimDatabaseFrameAttributeLibrary::FrameAttributeIntersection(FloatFrameAttributes[CurveIdx],
			UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromNotMirrored(Database, UAnimDatabaseFrameAttributeLibrary::FrameAttributeFrameRanges(FloatFrameAttributes[CurveIdx])));

		const int32 TotalRangeNum = FilteredFrameAttribute.FrameAttribute->FrameRangeSet.GetTotalRangeNum();

		TLearningArray<1, int32> TagRangeSequenceIndices;
		TagRangeSequenceIndices.SetNumUninitialized({ TotalRangeNum });
		UE::Learning::FrameRangeSet::AllRangeSequences(TagRangeSequenceIndices, FilteredFrameAttribute.FrameAttribute->FrameRangeSet);

		TArray<FRichCurveKey> RichCurveKeys;

		for (int32 RangeIdx = 0; RangeIdx < TotalRangeNum; RangeIdx++)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseLibrary::DatabaseAddCurvesFromArrayViews::Range);
			check(!Database->GetIsMirrored(TagRangeSequenceIndices[RangeIdx]));

			TScriptInterface<IAnimationDataController> Controller = Controllers[SequenceControllerIndices[TagRangeSequenceIndices[RangeIdx]]];

			const FAnimationCurveIdentifier CurveId(CurveNames[CurveIdx], ERawCurveTrackTypes::RCT_Float);

			// Add Curve if it doesn't exist
			if (!Controller->GetModel()->FindCurve(CurveId))
			{
				Controller->AddCurve(CurveId, AACF_DefaultCurve, bShouldTransact);
			}

			if (Colors.IsValidIndex(CurveIdx))
			{
				Controller->SetCurveColor(CurveId, Colors[CurveIdx]);
			}

			// Add Curve Keys
			const int32 RangeFrameNum = FilteredFrameAttribute.FrameAttribute->FrameRangeSet.GetAllRangeLengths()[RangeIdx];
			const int32 FrameAttributeRangeOffset = FilteredFrameAttribute.FrameAttribute->FrameRangeSet.GetAllRangeOffsets()[RangeIdx];

			RichCurveKeys.Reset(RangeFrameNum);

			const int32 StartFrame = FilteredFrameAttribute.FrameAttribute->FrameRangeSet.GetAllRangeStarts()[RangeIdx];

			// TODO: Check for Linear Interpolation
			float PrevSetValue = UE_MAX_FLT;
			for (int32 FrameIdx = 0; FrameIdx < RangeFrameNum; FrameIdx++)
			{
				const float Time = (StartFrame + FrameIdx) / FMath::Max(Database->GetFrameRate().AsDecimal(), UE_SMALL_NUMBER);
				const float CurrValue = FilteredFrameAttribute.FrameAttribute->AttributeData[0][FrameAttributeRangeOffset + FrameIdx];
				const float NextValue = FilteredFrameAttribute.FrameAttribute->AttributeData[0][FrameAttributeRangeOffset + FMath::Min(FrameIdx + 1, FMath::Max(RangeFrameNum - 1, 0))];
				if (!bSparseKeys || (CurrValue != PrevSetValue) || (NextValue != PrevSetValue))
				{
					RichCurveKeys.Add(FRichCurveKey(Time, CurrValue));
					PrevSetValue = CurrValue;
				}
			}

			Controller->SetCurveKeys(CurveId, RichCurveKeys, bShouldTransact);
		}
	}

	UE::AnimDatabase::Private::CloseControllers(Controllers, bShouldTransact);
#endif
}

void UAnimDatabaseLibrary::DatabaseAddAnimNotifyState(UAnimDatabase* Database, const FName AnimNotifyStateTrackName, const TSubclassOf<UAnimNotifyState> Class, const FAnimDatabaseFrameRanges& FrameRanges, const FLinearColor Color, const bool bShouldTransact)
{
	DatabaseAddAnimNotifyStatesFromArrayViews(Database, { AnimNotifyStateTrackName }, { Class }, { FrameRanges }, { Color }, bShouldTransact);
}

void UAnimDatabaseLibrary::DatabaseAddAnimNotifyStates(UAnimDatabase* Database, const TArray<FName>& AnimNotifyStateTrackNames, const TArray<TSubclassOf<UAnimNotifyState>>& Classes, const TArray<FAnimDatabaseFrameRanges>& FrameRanges, const TArray<FLinearColor>& Colors, const bool bShouldTransact)
{
	DatabaseAddAnimNotifyStatesFromArrayViews(Database, AnimNotifyStateTrackNames, Classes, FrameRanges, Colors, bShouldTransact);
}

void UAnimDatabaseLibrary::DatabaseAddAnimNotifyStatesFromArrayViews(UAnimDatabase* Database, const TArrayView<const FName> AnimNotifyStateTrackNames, const TArrayView<const TSubclassOf<UAnimNotifyState>> Classes, const TArrayView<const FAnimDatabaseFrameRanges> FrameRanges, const TArrayView<const FLinearColor> Colors, const bool bShouldTransact)
{
#if WITH_EDITOR
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseLibrary::DatabaseAddAnimNotifyStatesFromArrayViews);

	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "DatabaseAddAnimNotifyStatesFromArrayViews: Database is null.");
		return;
	}

	if (AnimNotifyStateTrackNames.Num() != Classes.Num() ||
		AnimNotifyStateTrackNames.Num() != FrameRanges.Num())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "DatabaseAddAnimNotifyStatesFromArrayViews: Number of Track Names, Classes, and FrameRanges must match. Got {Tracks}, {Classes}, and {Ranges}.", AnimNotifyStateTrackNames.Num(), Classes.Num(), FrameRanges.Num());
		return;
	}

	const int32 TrackNum = AnimNotifyStateTrackNames.Num();

	for (int32 TrackIdx = 0; TrackIdx < TrackNum; TrackIdx++)
	{
		const FAnimDatabaseFrameRanges FilteredFrameRanges = UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromNotMirrored(Database, FrameRanges[TrackIdx]);

		const int32 EntryNum = FilteredFrameRanges.FrameRangeSet->GetEntryNum();

		for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
		{
			check(!Database->GetIsMirrored(FilteredFrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx)));

			if (UAnimSequence* AnimSequence = Database->GetAnimSequence(FilteredFrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx)))
			{
				FScopedTransaction Transaction(LOCTEXT("AddAnimNotifyStatesTransaction", "Adding Anim Notify States"), bShouldTransact);
				AnimSequence->Modify();

				int32 AnimNotifyTrack = AnimSequence->AnimNotifyTracks.IndexOfByPredicate(
					[&](const FAnimNotifyTrack& Track)
					{
						return Track.TrackName == AnimNotifyStateTrackNames[TrackIdx];
					});

				if (AnimNotifyTrack == INDEX_NONE)
				{
					AnimNotifyTrack = AnimSequence->AnimNotifyTracks.Add(FAnimNotifyTrack(AnimNotifyStateTrackNames[TrackIdx], FLinearColor::Red));
				}

				if (Colors.IsValidIndex(TrackIdx))
				{
					AnimSequence->AnimNotifyTracks[AnimNotifyTrack].TrackColor = Colors[TrackIdx];
				}

				const int32 EntryRangeNum = FilteredFrameRanges.FrameRangeSet->GetEntryRangeNum(EntryIdx);

				for (int32 RangeIdx = 0; RangeIdx < EntryRangeNum; RangeIdx++)
				{
					const float StartTime = FilteredFrameRanges.FrameRangeSet->GetEntryRangeStart(EntryIdx, RangeIdx) / FMath::Max(Database->GetFrameRate().AsDecimal(), UE_SMALL_NUMBER);
					const float Duration = (FilteredFrameRanges.FrameRangeSet->GetEntryRangeLength(EntryIdx, RangeIdx) - 1) / FMath::Max(Database->GetFrameRate().AsDecimal(), UE_SMALL_NUMBER);

					FAnimNotifyEvent& NewEvent = AnimSequence->Notifies.AddDefaulted_GetRef();

					NewEvent.NotifyName = NAME_None;
					NewEvent.Link(AnimSequence, StartTime);
					NewEvent.TriggerTimeOffset = GetTriggerTimeOffsetForType(AnimSequence->CalculateOffsetForNotify(StartTime));
					NewEvent.TrackIndex = AnimNotifyTrack;
					NewEvent.Notify = nullptr;
					NewEvent.Guid = FGuid::NewGuid();

					if (Classes[TrackIdx])
					{
						NewEvent.NotifyStateClass = NewObject<UAnimNotifyState>(AnimSequence, Classes[TrackIdx], NAME_None, RF_Transactional);
						if (NewEvent.NotifyStateClass)
						{
							NewEvent.NotifyName = FName(*NewEvent.NotifyStateClass->GetNotifyName());
							NewEvent.SetDuration(Duration);
							NewEvent.EndLink.Link(AnimSequence, NewEvent.EndLink.GetTime());
						}
					}
					else
					{
						NewEvent.NotifyStateClass = nullptr;
					}
				}

				AnimSequence->RefreshCacheData();
				AnimSequence->MarkPackageDirty();
			}
		}
	}
#endif
}

void UAnimDatabaseLibrary::DatabaseAddAnimNotifyObject(UAnimDatabase* Database, const FName& TrackName, UAnimNotify* AnimNotify, const FAnimDatabaseFrames& Frames, const FLinearColor Color, const bool bShouldTransact)
{
	DatabaseAddAnimNotifyObjectsFromArrayViews(Database, { TrackName }, { AnimNotify }, { Frames }, { Color }, bShouldTransact);
}

void UAnimDatabaseLibrary::DatabaseAddAnimNotifyObjects(UAnimDatabase* Database, const TArray<FName>& AnimNotifyTrackNames, const TArray<UAnimNotify*>& AnimNotifies, const TArray<FAnimDatabaseFrames>& Frames, const TArray<FLinearColor>& Colors, const bool bShouldTransact)
{
	DatabaseAddAnimNotifyObjectsFromArrayViews(Database, AnimNotifyTrackNames, AnimNotifies, Frames, Colors, bShouldTransact);
}

void UAnimDatabaseLibrary::DatabaseAddAnimNotifyObjectsFromArrayViews(UAnimDatabase* Database, const TArrayView<const FName> AnimNotifyTrackNames, const TArrayView<UAnimNotify* const> AnimNotifies, const TArrayView<const FAnimDatabaseFrames> Frames, const TArrayView<const FLinearColor> Colors, const bool bShouldTransact)
{
#if WITH_EDITOR
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseLibrary::DatabaseAddAnimNotifyObject);

	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "DatabaseAddAnimNotifyObjectsFromArrayViews: Database is nullptr.");
		return;
	}

	if (AnimNotifyTrackNames.Num() != AnimNotifies.Num() ||
		AnimNotifyTrackNames.Num() != Frames.Num())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "DatabaseAddAnimNotifyObjectsFromArrayViews: Number of Track Names, Notifies, and Frames must match. Got {Tracks}, {Notifies}, and {Frames}.", AnimNotifyTrackNames.Num(), AnimNotifies.Num(), Frames.Num());
		return;
	}

	const int32 TrackNum = AnimNotifyTrackNames.Num();

	for (int32 TrackIdx = 0; TrackIdx < TrackNum; TrackIdx++)
	{
		if (!AnimNotifies[TrackIdx])
		{
			UE_LOGFMT(LogAnimDatabase, Error, "DatabaseAddAnimNotifyObjectsFromArrayViews: AnimNotify is nullptr.");
			continue;
		}

		const FAnimDatabaseFrames FilteredFrames = UAnimDatabaseFrameRangesLibrary::MakeFramesAtFrameRangesStarts(
			UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromNotMirrored(Database, UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFrames(Frames[TrackIdx])));
		
		const int32 EntryNum = FilteredFrames.FrameSet->GetEntryNum();

		for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
		{
			check(!Database->GetIsMirrored(FilteredFrames.FrameSet->GetEntrySequence(EntryIdx)));

			if (UAnimSequence* AnimSequence = Database->GetAnimSequence(FilteredFrames.FrameSet->GetEntrySequence(EntryIdx)))
			{
				FScopedTransaction Transaction(LOCTEXT("AddAnimNotifyObjectsTransaction", "Adding Anim Notify Objects"), bShouldTransact);
				AnimSequence->Modify();

				int32 AnimNotifyTrack = AnimSequence->AnimNotifyTracks.IndexOfByPredicate(
					[&](const FAnimNotifyTrack& Track)
					{
						return Track.TrackName == AnimNotifyTrackNames[TrackIdx];
					});

				if (AnimNotifyTrack == INDEX_NONE)
				{
					AnimNotifyTrack = AnimSequence->AnimNotifyTracks.Add(FAnimNotifyTrack(AnimNotifyTrackNames[TrackIdx], FLinearColor::Red));
				}

				if (Colors.IsValidIndex(TrackIdx))
				{
					AnimSequence->AnimNotifyTracks[AnimNotifyTrack].TrackColor = Colors[TrackIdx];
				}

				const int32 EntryFrameNum = FilteredFrames.FrameSet->GetEntryFrameNum(EntryIdx);

				for (int32 FrameIdx = 0; FrameIdx < EntryFrameNum; FrameIdx++)
				{
					const float StartTime = FilteredFrames.FrameSet->GetEntryFrame(EntryIdx, FrameIdx) / FMath::Max(Database->GetFrameRate().AsDecimal(), UE_SMALL_NUMBER);

					FAnimNotifyEvent& NewEvent = AnimSequence->Notifies.AddDefaulted_GetRef();

					NewEvent.NotifyName = NAME_None;
					NewEvent.Link(AnimSequence, StartTime);
					NewEvent.TriggerTimeOffset = GetTriggerTimeOffsetForType(AnimSequence->CalculateOffsetForNotify(StartTime));
					NewEvent.TrackIndex = AnimNotifyTrack;
					NewEvent.Notify = nullptr;
					NewEvent.Guid = FGuid::NewGuid();
					NewEvent.Notify = DuplicateObject(AnimNotifies[TrackIdx], AnimSequence);
					NewEvent.NotifyName = FName(*NewEvent.Notify->GetNotifyName());
					NewEvent.EndLink.Link(AnimSequence, NewEvent.EndLink.GetTime());
				}

				AnimSequence->RefreshCacheData();
				AnimSequence->MarkPackageDirty();
			}
		}
	}
#endif
}

void UAnimDatabaseLibrary::DatabaseAddAnimNotifyStateObject(UAnimDatabase* Database, const FName TrackName, UAnimNotifyState* AnimNotifyState, const FAnimDatabaseFrameRanges& FrameRanges, const FLinearColor Color, const bool bShouldTransact)
{
	DatabaseAddAnimNotifyStateObjectsFromArrayViews(Database, { TrackName }, { AnimNotifyState }, { FrameRanges }, { Color }, bShouldTransact);
}

void UAnimDatabaseLibrary::DatabaseAddAnimNotifyStateObjects(UAnimDatabase* Database, const TArray<FName>& AnimNotifyStateTrackNames, const TArray<UAnimNotifyState*>& AnimNotifyStates, const TArray<FAnimDatabaseFrameRanges>& FrameRanges, const TArray<FLinearColor>& Colors, const bool bShouldTransact)
{
	DatabaseAddAnimNotifyStateObjectsFromArrayViews(Database, AnimNotifyStateTrackNames, AnimNotifyStates, FrameRanges, Colors, bShouldTransact);
}

void UAnimDatabaseLibrary::DatabaseAddAnimNotifyStateObjectsFromArrayViews(UAnimDatabase* Database, const TArrayView<const FName> AnimNotifyStateTrackNames, const TArrayView<UAnimNotifyState* const> AnimNotifyStates, const TArrayView<const FAnimDatabaseFrameRanges> FrameRanges, const TArrayView<const FLinearColor> Colors, const bool bShouldTransact)
{
#if WITH_EDITOR
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseLibrary::DatabaseAddAnimNotifyStateObjectsFromArrayViews);

	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "DatabaseAddAnimNotifyStateObjectsFromArrayViews: Database is nullptr.");
		return;
	}

	if (AnimNotifyStateTrackNames.Num() != AnimNotifyStates.Num() ||
		AnimNotifyStateTrackNames.Num() != FrameRanges.Num())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "DatabaseAddAnimNotifyStateObjectsFromArrayViews: Number of Track Names, NotifyStates, and FrameRanges must match. Got {Tracks}, {NotifyStates}, and {Ranges}.", AnimNotifyStateTrackNames.Num(), AnimNotifyStates.Num(), FrameRanges.Num());
		return;
	}

	const int32 TrackNum = AnimNotifyStateTrackNames.Num();

	for (int32 TrackIdx = 0; TrackIdx < TrackNum; TrackIdx++)
	{
		if (!AnimNotifyStates[TrackIdx])
		{
			UE_LOGFMT(LogAnimDatabase, Error, "DatabaseAddAnimNotifyStateObjectsFromArrayViews: AnimNotifyState is nullptr.");
			continue;
		}

		const FAnimDatabaseFrameRanges FilteredFrameRanges = UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromNotMirrored(Database, FrameRanges[TrackIdx]);

		const int32 EntryNum = FilteredFrameRanges.FrameRangeSet->GetEntryNum();

		for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
		{
			check(!Database->GetIsMirrored(FilteredFrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx)));

			if (UAnimSequence* AnimSequence = Database->GetAnimSequence(FilteredFrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx)))
			{
				FScopedTransaction Transaction(LOCTEXT("AddAnimNotifyStateObjectsTransaction", "Adding Anim Notify State Objects"), bShouldTransact);
				AnimSequence->Modify();

				int32 AnimNotifyTrack = AnimSequence->AnimNotifyTracks.IndexOfByPredicate(
					[&](const FAnimNotifyTrack& Track)
					{
						return Track.TrackName == AnimNotifyStateTrackNames[TrackIdx];
					});

				if (AnimNotifyTrack == INDEX_NONE)
				{
					AnimNotifyTrack = AnimSequence->AnimNotifyTracks.Add(FAnimNotifyTrack(AnimNotifyStateTrackNames[TrackIdx], FLinearColor::Red));
				}

				if (Colors.IsValidIndex(TrackIdx))
				{
					AnimSequence->AnimNotifyTracks[AnimNotifyTrack].TrackColor = Colors[TrackIdx];
				}

				const int32 EntryRangeNum = FilteredFrameRanges.FrameRangeSet->GetEntryRangeNum(EntryIdx);

				for (int32 RangeIdx = 0; RangeIdx < EntryRangeNum; RangeIdx++)
				{
					const float StartTime = FilteredFrameRanges.FrameRangeSet->GetEntryRangeStart(EntryIdx, RangeIdx) / FMath::Max(Database->GetFrameRate().AsDecimal(), UE_SMALL_NUMBER);
					const float Duration = (FilteredFrameRanges.FrameRangeSet->GetEntryRangeLength(EntryIdx, RangeIdx) - 1) / FMath::Max(Database->GetFrameRate().AsDecimal(), UE_SMALL_NUMBER);

					FAnimNotifyEvent& NewEvent = AnimSequence->Notifies.AddDefaulted_GetRef();

					NewEvent.NotifyName = NAME_None;
					NewEvent.Link(AnimSequence, StartTime);
					NewEvent.TriggerTimeOffset = GetTriggerTimeOffsetForType(AnimSequence->CalculateOffsetForNotify(StartTime));
					NewEvent.TrackIndex = AnimNotifyTrack;
					NewEvent.Notify = nullptr;
					NewEvent.Guid = FGuid::NewGuid();
					NewEvent.NotifyStateClass = DuplicateObject(AnimNotifyStates[TrackIdx], AnimSequence);
					NewEvent.NotifyName = FName(*NewEvent.NotifyStateClass->GetNotifyName());
					NewEvent.SetDuration(Duration);
					NewEvent.EndLink.Link(AnimSequence, NewEvent.EndLink.GetTime());
				}

				AnimSequence->RefreshCacheData();
				AnimSequence->MarkPackageDirty();
			}
		}
	}
#endif
}

void UAnimDatabaseLibrary::DatabaseAddAnimNotify(UAnimDatabase* Database, const FName AnimNotifyTrackName, const TSubclassOf<UAnimNotify> Class, const FAnimDatabaseFrames& Frames, const FLinearColor Color, const bool bShouldTransact)
{
	DatabaseAddAnimNotifiesFromArrayViews(Database, { AnimNotifyTrackName }, { Class }, { Frames }, { Color }, bShouldTransact);
}

void UAnimDatabaseLibrary::DatabaseAddAnimNotifies(UAnimDatabase* Database, const TArray<FName>& AnimNotifyTrackNames, const TArray<TSubclassOf<UAnimNotify>>& Classes, const TArray<FAnimDatabaseFrames>& Frames, const TArray<FLinearColor>& Colors, const bool bShouldTransact)
{
	DatabaseAddAnimNotifiesFromArrayViews(Database, AnimNotifyTrackNames, Classes, Frames, Colors, bShouldTransact);
}

void UAnimDatabaseLibrary::DatabaseAddAnimNotifiesFromArrayViews(UAnimDatabase* Database, const TArrayView<const FName> AnimNotifyTrackNames, const TArrayView<const TSubclassOf<UAnimNotify>> Classes, const TArrayView<const FAnimDatabaseFrames> Frames, const TArrayView<const FLinearColor> Colors, const bool bShouldTransact)
{
#if WITH_EDITOR

	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseLibrary::DatabaseAddAnimNotifiesFromArrayViews);

	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "DatabaseAddAnimNotifiesFromArrayViews: Database is nullptr.");
		return;
	}

	if (AnimNotifyTrackNames.Num() != Classes.Num() ||
		AnimNotifyTrackNames.Num() != Frames.Num())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "DatabaseAddAnimNotifiesFromArrayViews: Number of Track Names, Classes, and Frames must match. Got {Tracks}, {Classes}, and {Frames}.", AnimNotifyTrackNames.Num(), Classes.Num(), Frames.Num());
		return;
	}

	const int32 TrackNum = AnimNotifyTrackNames.Num();

	for (int32 TrackIdx = 0; TrackIdx < TrackNum; TrackIdx++)
	{
		const FAnimDatabaseFrames FilteredFrames = UAnimDatabaseFrameRangesLibrary::MakeFramesAtFrameRangesStarts(
			UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromNotMirrored(Database, UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFrames(Frames[TrackIdx])));

		const int32 EntryNum = FilteredFrames.FrameSet->GetEntryNum();

		for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
		{
			check(!Database->GetIsMirrored(FilteredFrames.FrameSet->GetEntrySequence(EntryIdx)));

			if (UAnimSequence* AnimSequence = Database->GetAnimSequence(FilteredFrames.FrameSet->GetEntrySequence(EntryIdx)))
			{
				FScopedTransaction Transaction(LOCTEXT("AddAnimNotifiesTransaction", "Adding Anim Notifies"), bShouldTransact);
				AnimSequence->Modify();

				int32 AnimNotifyTrack = AnimSequence->AnimNotifyTracks.IndexOfByPredicate(
					[&](const FAnimNotifyTrack& Track)
					{
						return Track.TrackName == AnimNotifyTrackNames[TrackIdx];
					});

				if (AnimNotifyTrack == INDEX_NONE)
				{
					AnimNotifyTrack = AnimSequence->AnimNotifyTracks.Add(FAnimNotifyTrack(AnimNotifyTrackNames[TrackIdx], FLinearColor::Red));
				}

				if (Colors.IsValidIndex(TrackIdx))
				{
					AnimSequence->AnimNotifyTracks[AnimNotifyTrack].TrackColor = Colors[TrackIdx];
				}

				const int32 EntryFrameNum = FilteredFrames.FrameSet->GetEntryFrameNum(EntryIdx);

				for (int32 FrameIdx = 0; FrameIdx < EntryFrameNum; FrameIdx++)
				{
					const float StartTime = FilteredFrames.FrameSet->GetEntryFrame(EntryIdx, FrameIdx) / FMath::Max(Database->GetFrameRate().AsDecimal(), UE_SMALL_NUMBER);

					FAnimNotifyEvent& NewEvent = AnimSequence->Notifies.AddDefaulted_GetRef();
					NewEvent.NotifyName = NAME_None;
					NewEvent.Link(AnimSequence, StartTime);
					NewEvent.TriggerTimeOffset = GetTriggerTimeOffsetForType(AnimSequence->CalculateOffsetForNotify(StartTime));
					NewEvent.TrackIndex = AnimNotifyTrack;
					NewEvent.NotifyStateClass = nullptr;
					NewEvent.Guid = FGuid::NewGuid();

					if (Classes[TrackIdx])
					{
						NewEvent.Notify = NewObject<UAnimNotify>(AnimSequence, Classes[TrackIdx], NAME_None, RF_Transactional);
						if (NewEvent.Notify)
						{
							NewEvent.NotifyName = FName(*NewEvent.Notify->GetNotifyName());
						}
					}
					else
					{
						NewEvent.Notify = nullptr;
					}
				}

				AnimSequence->RefreshCacheData();
				AnimSequence->MarkPackageDirty();
			}
		}
	}
#endif
}

void UAnimDatabaseLibrary::DatabaseRemoveAnimNotifiesFromTrack(UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName AnimNotifyTrackName, const bool bShouldTransact)
{
	DatabaseRemoveAnimNotifiesFromTracksArrayView(Database, FrameRanges, { AnimNotifyTrackName }, bShouldTransact);
}

void UAnimDatabaseLibrary::DatabaseRemoveAnimNotifiesFromTracks(UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArray<FName>& AnimNotifyTrackNames, const bool bShouldTransact)
{
	DatabaseRemoveAnimNotifiesFromTracksArrayView(Database, FrameRanges, AnimNotifyTrackNames, bShouldTransact);
}

void UAnimDatabaseLibrary::DatabaseRemoveAnimNotifiesFromTracksArrayView(UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const FName> AnimNotifyTrackNames, const bool bShouldTransact )
{
#if WITH_EDITOR

	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseLibrary::DatabaseRemoveAnimNotifiesFromTracksArrayView);

	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "DatabaseRemoveAnimNotifiesFromTracksArrayView: Database is nullptr.");
		return;
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "DatabaseRemoveAnimNotifiesFromTracksArrayView: Invalid FrameRanges.");
		return;
	}

	const int32 TrackNum = AnimNotifyTrackNames.Num();

	for (int32 TrackIdx = 0; TrackIdx < TrackNum; TrackIdx++)
	{
		const FAnimDatabaseFrameRanges FilteredFrameRanges = UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromNotMirrored(Database, FrameRanges);

		const int32 EntryNum = FilteredFrameRanges.FrameRangeSet->GetEntryNum();

		for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
		{
			check(!Database->GetIsMirrored(FilteredFrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx)));

			const int32 SequenceIdx = FilteredFrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);

			if (UAnimSequence* AnimSequence = Database->GetAnimSequence(SequenceIdx))
			{
				FScopedTransaction Transaction(LOCTEXT("RemoveAnimNotifiesTransaction", "Removing Anim Notifies"), bShouldTransact);
				AnimSequence->Modify();

				int32 AnimNotifyTrack = AnimSequence->AnimNotifyTracks.IndexOfByPredicate(
					[&](const FAnimNotifyTrack& Track)
					{
						return Track.TrackName == AnimNotifyTrackNames[TrackIdx];
					});

				if (AnimNotifyTrack == INDEX_NONE) { continue; }

				AnimSequence->Notifies.RemoveAll([SequenceIdx, AnimNotifyTrack, &FilteredFrameRanges, Database](const FAnimNotifyEvent& AnimNotify) {

					return (
						AnimNotify.TrackIndex == AnimNotifyTrack &&
						AnimNotify.Notify != nullptr &&
						AnimNotify.NotifyStateClass == nullptr &&
						FilteredFrameRanges.FrameRangeSet->ContainsTime(
							SequenceIdx, AnimNotify.GetTime(), 1.0f / FMath::Max(Database->GetFrameRate().AsDecimal(), UE_SMALL_NUMBER)));
					});

				AnimSequence->RefreshCacheData();
				AnimSequence->MarkPackageDirty();
			}
		}
	}
#endif
}

void UAnimDatabaseLibrary::DatabaseAddSyncMarker(UAnimDatabase* Database, const FName SyncMarkerTrackName, const FName SyncMarkerName, const FAnimDatabaseFrames& Frames, const FLinearColor Color, const bool bShouldTransact)
{
	DatabaseAddSyncMarkersFromArrayViews(Database, { SyncMarkerTrackName }, { SyncMarkerName }, { Frames }, { Color }, bShouldTransact);
}

void UAnimDatabaseLibrary::DatabaseAddSyncMarkers(UAnimDatabase* Database, const TArray<FName>& SyncMarkerTrackNames, const TArray<FName>& SyncMarkerNames, const TArray<FAnimDatabaseFrames>& Frames, const TArray<FLinearColor>& Colors, const bool bShouldTransact)
{
	DatabaseAddSyncMarkersFromArrayViews(Database, SyncMarkerTrackNames, SyncMarkerNames, Frames, Colors, bShouldTransact);
}

void UAnimDatabaseLibrary::DatabaseAddSyncMarkersFromArrayViews(UAnimDatabase* Database, const TArrayView<const FName> SyncMarkerTrackNames, const TArrayView<const FName> SyncMarkerNames, const TArrayView<const FAnimDatabaseFrames> Frames, const TArrayView<const FLinearColor> Colors, const bool bShouldTransact)
{
#if WITH_EDITOR

	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseLibrary::DatabaseAddSyncMarkersFromArrayViews);

	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "DatabaseAddSyncMarkersFromArrayViews: Database is nullptr.");
		return;
	}

	if (SyncMarkerTrackNames.Num() != SyncMarkerNames.Num() ||
		SyncMarkerTrackNames.Num() != Frames.Num())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "DatabaseAddSyncMarkersFromArrayViews: Number of Track Names, Sync Marker names, and Frames must match. Got {Tracks}, {SyncMarkers}, and {Frames}.", SyncMarkerTrackNames.Num(), SyncMarkerNames.Num(), Frames.Num());
		return;
	}

	const int32 TrackNum = SyncMarkerTrackNames.Num();

	for (int32 TrackIdx = 0; TrackIdx < TrackNum; TrackIdx++)
	{
		const FAnimDatabaseFrames FilteredFrames = UAnimDatabaseFrameRangesLibrary::MakeFramesAtFrameRangesStarts(
			UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromNotMirrored(Database, UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFrames(Frames[TrackIdx])));

		const int32 EntryNum = FilteredFrames.FrameSet->GetEntryNum();

		for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
		{
			check(!Database->GetIsMirrored(FilteredFrames.FrameSet->GetEntrySequence(EntryIdx)));

			if (UAnimSequence* AnimSequence = Database->GetAnimSequence(FilteredFrames.FrameSet->GetEntrySequence(EntryIdx)))
			{
				FScopedTransaction Transaction(LOCTEXT("AddSyncMarkersTransaction", "Adding Sync Markers"), bShouldTransact);
				AnimSequence->Modify();

				int32 AnimNotifyTrack = AnimSequence->AnimNotifyTracks.IndexOfByPredicate(
					[&](const FAnimNotifyTrack& Track)
					{
						return Track.TrackName == SyncMarkerTrackNames[TrackIdx];
					});

				if (AnimNotifyTrack == INDEX_NONE)
				{
					AnimNotifyTrack = AnimSequence->AnimNotifyTracks.Add(FAnimNotifyTrack(SyncMarkerTrackNames[TrackIdx], FLinearColor::Red));
				}

				if (Colors.IsValidIndex(TrackIdx))
				{
					AnimSequence->AnimNotifyTracks[AnimNotifyTrack].TrackColor = Colors[TrackIdx];
				}

				const int32 EntryFrameNum = FilteredFrames.FrameSet->GetEntryFrameNum(EntryIdx);

				for (int32 FrameIdx = 0; FrameIdx < EntryFrameNum; FrameIdx++)
				{
					const float StartTime = FilteredFrames.FrameSet->GetEntryFrame(EntryIdx, FrameIdx) / FMath::Max(Database->GetFrameRate().AsDecimal(), UE_SMALL_NUMBER);

					FAnimSyncMarker& Marker = AnimSequence->AuthoredSyncMarkers.AddDefaulted_GetRef();
					Marker.MarkerName = SyncMarkerNames[TrackIdx];
					Marker.Time = StartTime;
					Marker.TrackIndex = AnimNotifyTrack;
					Marker.Guid = FGuid::NewGuid();
				}

				AnimSequence->RefreshCacheData();
				AnimSequence->MarkPackageDirty();
			}
		}
	}
#endif
}

void UAnimDatabaseLibrary::DatabaseRemoveSyncMarkerFromTrack(UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName SyncMarkerTrackName, const bool bShouldTransact)
{
	DatabaseRemoveSyncMarkerFromTracksArrayView(Database, FrameRanges, { SyncMarkerTrackName }, bShouldTransact);
}

void UAnimDatabaseLibrary::DatabaseRemoveSyncMarkerFromTracks(UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArray<FName>& SyncMarkerTrackNames, const bool bShouldTransact)
{
	DatabaseRemoveSyncMarkerFromTracksArrayView(Database, FrameRanges, SyncMarkerTrackNames, bShouldTransact);
}

void UAnimDatabaseLibrary::DatabaseRemoveSyncMarkerFromTracksArrayView(UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const FName> SyncMarkerTrackNames, const bool bShouldTransact)
{
#if WITH_EDITOR

	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseLibrary::DatabaseRemoveSyncMarkerFromTracksArrayView);

	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "DatabaseRemoveSyncMarkerFromTracksArrayView: Database is nullptr.");
		return;
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "DatabaseRemoveSyncMarkerFromTracksArrayView: Invalid FrameRanges.");
		return;
	}

	const int32 TrackNum = SyncMarkerTrackNames.Num();

	for (int32 TrackIdx = 0; TrackIdx < TrackNum; TrackIdx++)
	{
		const FAnimDatabaseFrameRanges FilteredFrameRanges = UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromNotMirrored(Database, FrameRanges);

		const int32 EntryNum = FilteredFrameRanges.FrameRangeSet->GetEntryNum();

		for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
		{
			check(!Database->GetIsMirrored(FilteredFrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx)));

			const int32 SequenceIdx = FilteredFrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);

			if (UAnimSequence* AnimSequence = Database->GetAnimSequence(SequenceIdx))
			{
				FScopedTransaction Transaction(LOCTEXT("RemoveSyncMarkersTransaction", "Removing Sync Markers"), bShouldTransact);
				AnimSequence->Modify();

				int32 AnimNotifyTrack = AnimSequence->AnimNotifyTracks.IndexOfByPredicate(
					[&](const FAnimNotifyTrack& Track)
					{
						return Track.TrackName == SyncMarkerTrackNames[TrackIdx];
					});

				if (AnimNotifyTrack == INDEX_NONE) { continue; }

				AnimSequence->AuthoredSyncMarkers.RemoveAll([SequenceIdx, AnimNotifyTrack, &FilteredFrameRanges, Database](const FAnimSyncMarker& SyncMarker) {
					
					return (
						SyncMarker.TrackIndex == AnimNotifyTrack && 
						FilteredFrameRanges.FrameRangeSet->ContainsTime(
							SequenceIdx, SyncMarker.Time, 1.0f / FMath::Max(Database->GetFrameRate().AsDecimal(), UE_SMALL_NUMBER)));
					});

				AnimSequence->RefreshCacheData();
				AnimSequence->MarkPackageDirty();
			}
		}
	}
#endif
}

void UAnimDatabaseLibrary::DatabaseSetLocalBoneTransform(UAnimDatabase* Database, const FName BoneName, const FAnimDatabaseFrameAttribute& TransformFrameAttribute, const bool bShouldTransact)
{
	DatabaseSetLocalBoneTransformsFromArrayViews(Database, { BoneName }, { TransformFrameAttribute }, bShouldTransact);
}

void UAnimDatabaseLibrary::DatabaseSetLocalBoneTransforms(UAnimDatabase* Database, const TArray<FName>& BoneNames, const TArray<FAnimDatabaseFrameAttribute>& TransformFrameAttributes, const bool bShouldTransact)
{
	DatabaseSetLocalBoneTransformsFromArrayViews(Database, BoneNames, TransformFrameAttributes, bShouldTransact);
}

void UAnimDatabaseLibrary::DatabaseSetLocalBoneTransformsFromArrayViews(UAnimDatabase* Database, const TArrayView<const FName> BoneNames, const TArrayView<const FAnimDatabaseFrameAttribute> TransformFrameAttributes, const bool bShouldTransact)
{
#if WITH_EDITOR
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseLibrary::DatabaseSetLocalBoneTransformsFromArrayViews);

	if (BoneNames.Num() != TransformFrameAttributes.Num())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "DatabaseSetLocalBoneTransformsFromArrayViews: Number of Bone Names and FrameAttributes must match. Got {Names} and {Attributes}.", BoneNames.Num(), TransformFrameAttributes.Num());
		return;
	}

	const int32 BoneNum = BoneNames.Num();

	for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
	{
		if (TransformFrameAttributes[BoneIdx].Type != EAnimDatabaseAttributeType::Transform)
		{
			UE_LOGFMT(LogAnimDatabase, Error, "DatabaseSetLocalBoneTransformsFromArrayViews: Incorrect FrameAttribute Type. Got {Type}. Expected Transform.", *UEnum::GetValueAsString(TransformFrameAttributes[BoneIdx].Type));
			return;
		}
	}

	const FFrameRate DatabaseFrameRate = Database->GetFrameRate();

	// To avoid triggering recompression during modification we need to open brackets on all the controllers for all the to-be-modified animation 
	// sequences once, and then close them all at the end.

	TArray<TScriptInterface<IAnimationDataController>> Controllers;
	TMap<int32, int32> SequenceControllerIndices;

	UE::AnimDatabase::Private::OpenControllers(
		Controllers, SequenceControllerIndices, Database, TransformFrameAttributes,
		LOCTEXT("SetBoneTransformScript", "Setting Bone Transform"), bShouldTransact);

	FScopedSlowTask SlowTask(BoneNum, LOCTEXT("SettingBoneLocalTransforms", "Setting Bone Transforms"));
	SlowTask.MakeDialog(true);

	for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
	{
		if (SlowTask.ShouldCancel()) { break; }
		SlowTask.EnterProgressFrame();

		const FAnimDatabaseFrameAttribute FilteredFrameAttribute = UAnimDatabaseFrameAttributeLibrary::FrameAttributeIntersection(TransformFrameAttributes[BoneIdx],
			UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromNotMirrored(Database, UAnimDatabaseFrameAttributeLibrary::FrameAttributeFrameRanges(TransformFrameAttributes[BoneIdx])));

		const int32 TotalRangeNum = FilteredFrameAttribute.FrameAttribute->FrameRangeSet.GetTotalRangeNum();

		TLearningArray<1, int32> TagRangeSequenceIndices;
		TagRangeSequenceIndices.SetNumUninitialized({ TotalRangeNum });
		UE::Learning::FrameRangeSet::AllRangeSequences(TagRangeSequenceIndices, FilteredFrameAttribute.FrameAttribute->FrameRangeSet);

		TArray<FVector3f> LocationKeys;
		TArray<FQuat4f> RotationKeys;
		TArray<FVector3f> ScaleKeys;

		for (int32 RangeIdx = 0; RangeIdx < TotalRangeNum; RangeIdx++)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseLibrary::DatabaseSetBoneTransform::Range);

			const int32 RangeSequenceIdx = TagRangeSequenceIndices[RangeIdx];

			check(!Database->GetIsMirrored(RangeSequenceIdx));

			TScriptInterface<IAnimationDataController> Controller = Controllers[SequenceControllerIndices[RangeSequenceIdx]];

			// Add bone track if it does not exist
			if (!Controller->GetModel()->IsValidBoneTrackName(BoneNames[BoneIdx]))
			{
				if (!Controller->AddBoneCurve(BoneNames[BoneIdx], bShouldTransact))
				{
					UE_LOGFMT(LogAnimDatabase, Error, "DatabaseSetLocalBoneTransformsFromArrayViews: Could not add track for bone '{Name}'.", *BoneNames[BoneIdx].ToString());
					continue;
				}
			}

			// Set Transform Keys
			const int32 RangeFrameNum = FilteredFrameAttribute.FrameAttribute->FrameRangeSet.GetAllRangeLengths()[RangeIdx];
			const int32 FrameAttributeRangeOffset = FilteredFrameAttribute.FrameAttribute->FrameRangeSet.GetAllRangeOffsets()[RangeIdx];
			const int32 StartFrame = FilteredFrameAttribute.FrameAttribute->FrameRangeSet.GetAllRangeStarts()[RangeIdx];

			const UAnimSequence* RangeAnimSequence = Database->GetAnimSequence(RangeSequenceIdx);
			const FFrameRate SequenceFrameRate = RangeAnimSequence->GetSamplingFrameRate();
			const int32 SequenceFrameNum = (SequenceFrameRate.Numerator * RangeFrameNum * DatabaseFrameRate.Denominator) /
				(SequenceFrameRate.Denominator * DatabaseFrameRate.Numerator);
			const int32 SequenceStartFrame = (SequenceFrameRate.Numerator * StartFrame * DatabaseFrameRate.Denominator) /
				(SequenceFrameRate.Denominator * DatabaseFrameRate.Numerator);
			const int32 SequenceTotalFrameNum = RangeAnimSequence->GetNumberOfSampledKeys();

			LocationKeys.Reset(SequenceFrameNum);
			RotationKeys.Reset(SequenceFrameNum);
			ScaleKeys.Reset(SequenceFrameNum);

			for (int32 FrameIdx = 0; FrameIdx < SequenceFrameNum; FrameIdx++)
			{
				FTransform3f FrameTransform = FTransform3f::Identity;

				if (SequenceFrameRate != DatabaseFrameRate)
				{
					const float Time = (StartFrame + FrameIdx) / FMath::Max(SequenceFrameRate.AsDecimal(), UE_SMALL_NUMBER);
					UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleTransformFloat(FrameTransform, FilteredFrameAttribute, RangeSequenceIdx, Time, DatabaseFrameRate);
				}
				else
				{
					check(SequenceFrameNum == RangeFrameNum);
					check(SequenceStartFrame == StartFrame);
					FrameTransform = FilteredFrameAttribute.GetAsTransform(FrameAttributeRangeOffset + FrameIdx);
				}

				LocationKeys.Add(FrameTransform.GetLocation());
				RotationKeys.Add(FrameTransform.GetRotation());
				ScaleKeys.Add(FrameTransform.GetScale3D());
			}

			FInt32Range KeyRange(SequenceStartFrame, FMath::Min(SequenceStartFrame + SequenceFrameNum, SequenceTotalFrameNum));
			if (!Controller->UpdateBoneTrackKeys(BoneNames[BoneIdx], KeyRange, LocationKeys, RotationKeys, ScaleKeys, bShouldTransact))
			{
				UE_LOGFMT(LogAnimDatabase, Error, "DatabaseSetLocalBoneTransformsFromArrayViews: Updating Transform Keys Failed.");
			}
		}
	}

	UE::AnimDatabase::Private::CloseControllers(Controllers, bShouldTransact);
#endif
}

void UAnimDatabaseLibrary::DatabaseSetGlobalBoneTransform(UAnimDatabase* Database, const FName BoneName, const FAnimDatabaseFrameAttribute& TransformFrameAttribute, const bool bShouldTransact)
{
	DatabaseSetGlobalBoneTransformsFromArrayViews(Database, { BoneName }, { TransformFrameAttribute }, bShouldTransact);
}

void UAnimDatabaseLibrary::DatabaseSetGlobalBoneTransforms(UAnimDatabase* Database, const TArray<FName>& BoneNames, const TArray<FAnimDatabaseFrameAttribute>& TransformFrameAttributes, const bool bShouldTransact)
{
	DatabaseSetGlobalBoneTransformsFromArrayViews(Database, BoneNames, TransformFrameAttributes, bShouldTransact);
}

void UAnimDatabaseLibrary::DatabaseSetGlobalBoneTransformsFromArrayViews(UAnimDatabase* Database, const TArrayView<const FName> BoneNames, const TArrayView<const FAnimDatabaseFrameAttribute> TransformFrameAttributes, const bool bShouldTransact)
{
#if WITH_EDITOR

	const int32 BoneNum = BoneNames.Num();

	if (BoneNames.Num() != TransformFrameAttributes.Num())
	{
		return;
	}

	TArray<FAnimDatabaseFrameAttribute> LocalTransforms;
	LocalTransforms.SetNum(BoneNum);

	for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
	{
		const int32 ParentBoneIdx = Database->GetBoneParent(Database->FindBoneIndex(BoneNames[BoneIdx]));
		if (ParentBoneIdx != INDEX_NONE)
		{
			LocalTransforms[BoneIdx] = UAnimDatabaseFrameAttributeLibrary::FrameAttributeDivide(TransformFrameAttributes[BoneIdx],
				UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalTransformFrameAttribute(Database, UAnimDatabaseFrameAttributeLibrary::FrameAttributeFrameRanges(TransformFrameAttributes[BoneIdx]), ParentBoneIdx));
		}
		else
		{
			LocalTransforms[BoneIdx] = TransformFrameAttributes[BoneIdx];
		}
	}

	DatabaseSetLocalBoneTransformsFromArrayViews(Database, BoneNames, LocalTransforms, bShouldTransact);
#endif
}

void UAnimDatabaseLibrary::DatabaseResetBoneTransform(UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName BoneName, const bool bShouldTransact)
{
	DatabaseResetBoneTransformsFromArrayViews(Database, FrameRanges, { BoneName }, bShouldTransact);
}

void UAnimDatabaseLibrary::DatabaseResetBoneTransforms(UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArray<FName>& BoneNames, const bool bShouldTransact)
{
	DatabaseResetBoneTransformsFromArrayViews(Database, FrameRanges, BoneNames, bShouldTransact);
}

void UAnimDatabaseLibrary::DatabaseResetBoneTransformsFromArrayViews(UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const FName> BoneNames, const bool bShouldTransact)
{
#if WITH_EDITOR
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseLibrary::DatabaseResetBoneTransformsFromArrayViews);

	const int32 BoneNum = BoneNames.Num();

	const FFrameRate DatabaseFrameRate = Database->GetFrameRate();

	const FAnimDatabaseFrameRanges FilteredFrameRanges = UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromNotMirrored(Database, FrameRanges);

	// To avoid triggering recompression during modification we need to open brackets on all the controllers for all the to-be-modified animation 
	// sequences once, and then close them all at the end.

	TArray<TScriptInterface<IAnimationDataController>> Controllers;
	TMap<int32, int32> SequenceControllerIndices;

	UE::AnimDatabase::Private::OpenControllers(
		Controllers, SequenceControllerIndices, Database, FilteredFrameRanges,
		LOCTEXT("ResetBoneTransformScript", "Resetting Bone Transform"), bShouldTransact);

	FScopedSlowTask SlowTask(BoneNum, LOCTEXT("ResettingBoneLocalTransforms", "Resetting Bone Transforms"));
	SlowTask.MakeDialog(true);

	for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
	{
		if (SlowTask.ShouldCancel()) { break; }
		SlowTask.EnterProgressFrame();

		const int32 BoneIndex = Database->FindBoneIndex(BoneNames[BoneIdx]);

		if (BoneIndex == INDEX_NONE)
		{
			UE_LOGFMT(LogAnimDatabase, Error, "DatabaseResetBoneTransformsFromArrayViews: Could not find bone {BoneName}.", *BoneNames[BoneIdx].ToString());
			continue;
		}

		const int32 TotalRangeNum = FilteredFrameRanges.FrameRangeSet->GetTotalRangeNum();

		TLearningArray<1, int32> TagRangeSequenceIndices;
		TagRangeSequenceIndices.SetNumUninitialized({ TotalRangeNum });
		UE::Learning::FrameRangeSet::AllRangeSequences(TagRangeSequenceIndices, *FilteredFrameRanges.FrameRangeSet);

		TArray<FVector3f> LocationKeys;
		TArray<FQuat4f> RotationKeys;
		TArray<FVector3f> ScaleKeys;

		for (int32 RangeIdx = 0; RangeIdx < TotalRangeNum; RangeIdx++)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseLibrary::DatabaseResetBoneTransformsFromArrayViews::Range);

			const int32 RangeSequenceIdx = TagRangeSequenceIndices[RangeIdx];

			check(!Database->GetIsMirrored(RangeSequenceIdx));

			TScriptInterface<IAnimationDataController> Controller = Controllers[SequenceControllerIndices[RangeSequenceIdx]];

			// Skip bone track if it does not exist
			if (!Controller->GetModel()->IsValidBoneTrackName(BoneNames[BoneIdx]))
			{
				continue;
			}

			// Set Transform Keys
			const int32 RangeFrameNum = FilteredFrameRanges.FrameRangeSet->GetAllRangeLengths()[RangeIdx];
			const int32 FrameAttributeRangeOffset = FilteredFrameRanges.FrameRangeSet->GetAllRangeOffsets()[RangeIdx];
			const int32 StartFrame = FilteredFrameRanges.FrameRangeSet->GetAllRangeStarts()[RangeIdx];

			const UAnimSequence* RangeAnimSequence = Database->GetAnimSequence(RangeSequenceIdx);
			const FFrameRate SequenceFrameRate = RangeAnimSequence->GetSamplingFrameRate();
			const int32 SequenceFrameNum = (SequenceFrameRate.Numerator * RangeFrameNum * DatabaseFrameRate.Denominator) /
				(SequenceFrameRate.Denominator * DatabaseFrameRate.Numerator);
			const int32 SequenceStartFrame = (SequenceFrameRate.Numerator * StartFrame * DatabaseFrameRate.Denominator) /
				(SequenceFrameRate.Denominator * DatabaseFrameRate.Numerator);
			const int32 SequenceTotalFrameNum = RangeAnimSequence->GetNumberOfSampledKeys();

			LocationKeys.Reset(SequenceFrameNum);
			RotationKeys.Reset(SequenceFrameNum);
			ScaleKeys.Reset(SequenceFrameNum);

			for (int32 FrameIdx = 0; FrameIdx < SequenceFrameNum; FrameIdx++)
			{
				const FTransform FrameTransform = Database->GetSkeleton()->GetReferenceSkeleton().GetRefBonePose()[BoneIndex];
				LocationKeys.Add((FVector3f)FrameTransform.GetLocation());
				RotationKeys.Add((FQuat4f)FrameTransform.GetRotation());
				ScaleKeys.Add((FVector3f)FrameTransform.GetScale3D());
			}

			FInt32Range KeyRange(SequenceStartFrame, FMath::Min(SequenceStartFrame + SequenceFrameNum, SequenceTotalFrameNum));
			if (!Controller->UpdateBoneTrackKeys(BoneNames[BoneIdx], KeyRange, LocationKeys, RotationKeys, ScaleKeys, bShouldTransact))
			{
				UE_LOGFMT(LogAnimDatabase, Error, "DatabaseResetBoneTransformsFromArrayViews: Updating Transform Keys Failed.");
			}
		}
	}

	UE::AnimDatabase::Private::CloseControllers(Controllers, bShouldTransact);
#endif
}

void UAnimDatabaseLibrary::DatabaseStripUnusedBoneTracks(UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const bool bShouldTransact)
{
#if WITH_EDITOR
	const FAnimDatabaseFrameRanges FilteredFrameRanges = UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromNotMirrored(Database, FrameRanges);

	// Sample pose data

	const int32 TotalFrameNum = FilteredFrameRanges.FrameRangeSet->GetTotalFrameNum();
	const int32 BoneNum = Database->GetBoneNum();

	UE::AnimDatabase::FPoseData PoseData;
	PoseData.Resize(TotalFrameNum, BoneNum, {}, {});

	Database->WaitForCompressionOnAnimSequencesFromArrayView(FilteredFrameRanges.FrameRangeSet->GetEntrySequences());

	UE::Learning::FrameRangeSet::ParallelForEachRange(*FilteredFrameRanges.FrameRangeSet, [&FilteredFrameRanges, &PoseData, &Database](
		const int32 TotalRangeIdx,
		const int32 EntryIdx,
		const int32 RangeIdx) {

			TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseLibrary::DatabaseStripUnusedBoneTracks::Sample);

			const int32 FrameOffset = FilteredFrameRanges.FrameRangeSet->GetEntryRangeOffset(EntryIdx, RangeIdx);
			const int32 FrameNum = FilteredFrameRanges.FrameRangeSet->GetEntryRangeLength(EntryIdx, RangeIdx);
			const int32 SequenceIdx = FilteredFrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);
			const int32 StartFrame = FilteredFrameRanges.FrameRangeSet->GetEntryRangeStart(EntryIdx, RangeIdx);

			Database->GetPoseData(PoseData.Slice(FrameOffset, FrameNum), SequenceIdx, StartFrame);
		});

	const FReferenceSkeleton ReferenceSkeleton = Database->Skeleton->GetReferenceSkeleton();
	check(ReferenceSkeleton.GetRefBonePose().Num() == BoneNum);

	TArray<int32> UsedBones;
	UE::AnimDatabase::PoseData::ComputeUsedBones(
		UsedBones,
		PoseData.LocalBoneData.ConstView(),
		ReferenceSkeleton.GetRefBonePose());

	const int32 EntryNum = FilteredFrameRanges.FrameRangeSet->GetEntryNum();

	for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
	{
		const int32 SequenceIdx = FilteredFrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);

		if (UAnimSequence* AnimSequence = Database->Entries[SequenceIdx])
		{
			TScriptInterface<IAnimationDataController> Controller = AnimSequence->GetDataModel()->GetController();
			Controller->OpenBracket(LOCTEXT("StrippingKeysScript", "Stripping Unused Keys"), bShouldTransact);

			for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
			{
				if (!UsedBones.Contains(BoneIdx))
				{
					const FName BoneName = ReferenceSkeleton.GetBoneName(BoneIdx);
					check(BoneName != NAME_None);

					if (Controller->GetModel()->IsValidBoneTrackName(BoneName))
					{
						Controller->RemoveBoneTrack(BoneName, bShouldTransact);
					}
				}
			}

			Controller->CloseBracket();
		}
	}
#endif
}

void UAnimDatabaseLibrary::DatabaseModifyPoseData(UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TFunctionRef<void(const UE::AnimDatabase::FPoseDataView& InOutPoseData)> Function, const bool bShouldTransact)
{
#if WITH_EDITOR
	const FAnimDatabaseFrameRanges FilteredFrameRanges = UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromNotMirrored(Database, FrameRanges);
	
	// Sample and modify pose data

	const int32 TotalFrameNum = FilteredFrameRanges.FrameRangeSet->GetTotalFrameNum();
	const int32 BoneNum = Database->GetBoneNum();

	UE::AnimDatabase::FPoseData PoseData;
	PoseData.Resize(TotalFrameNum, BoneNum, {}, {});

	Database->WaitForCompressionOnAnimSequencesFromArrayView(FilteredFrameRanges.FrameRangeSet->GetEntrySequences());

	UE::Learning::FrameRangeSet::ParallelForEachRange(*FilteredFrameRanges.FrameRangeSet, [&FilteredFrameRanges, &PoseData, &Database, &Function](
		const int32 TotalRangeIdx,
		const int32 EntryIdx,
		const int32 RangeIdx) {

			TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseLibrary::DatabaseModifyPoseData::SampleAndModify);

			const int32 FrameOffset = FilteredFrameRanges.FrameRangeSet->GetEntryRangeOffset(EntryIdx, RangeIdx);
			const int32 FrameNum = FilteredFrameRanges.FrameRangeSet->GetEntryRangeLength(EntryIdx, RangeIdx);
			const int32 SequenceIdx = FilteredFrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);
			const int32 StartFrame = FilteredFrameRanges.FrameRangeSet->GetEntryRangeStart(EntryIdx, RangeIdx);

			Database->GetPoseData(PoseData.Slice(FrameOffset, FrameNum), SequenceIdx, StartFrame);

			Function(PoseData.Slice(FrameOffset, FrameNum));
		});

	// Compute the required used bones

	TArray<int32> UsedBones;
	UE::AnimDatabase::PoseData::ComputeUsedBones(
		UsedBones,
		PoseData.LocalBoneData.ConstView(),
		Database->Skeleton->GetReferenceSkeleton().GetRefBonePose());

	// Write back modified pose data

	const int32 TotalRangeNum = FilteredFrameRanges.FrameRangeSet->GetTotalRangeNum();

	TArray<TScriptInterface<IAnimationDataController>> Controllers;
	TMap<int32, int32> SequenceControllerIndices;

	UE::AnimDatabase::Private::OpenControllers(
		Controllers, SequenceControllerIndices, Database, FilteredFrameRanges,
		LOCTEXT("ModifyPoseDataScript", "Modifying Pose Data"), bShouldTransact);

	TLearningArray<1, int32> TagRangeSequenceIndices;
	TagRangeSequenceIndices.SetNumUninitialized({ TotalRangeNum });
	UE::Learning::FrameRangeSet::AllRangeSequences(TagRangeSequenceIndices, *FilteredFrameRanges.FrameRangeSet);

	FScopedSlowTask SlowTask(TotalRangeNum, LOCTEXT("ModifyingRanges", "Modifying Sequence Ranges"));
	SlowTask.MakeDialog(true);

	for (int32 RangeIdx = 0; RangeIdx < TotalRangeNum; RangeIdx++)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseLibrary::DatabaseModifyPoseData::WriteBack);

		if (SlowTask.ShouldCancel()) { break; }
		SlowTask.EnterProgressFrame();

		const int32 RangeSequenceIdx = TagRangeSequenceIndices[RangeIdx];
		check(!Database->GetIsMirrored(RangeSequenceIdx));

		const int32 FrameOffset = FilteredFrameRanges.FrameRangeSet->GetAllRangeOffsets()[RangeIdx];
		const int32 FrameNum = FilteredFrameRanges.FrameRangeSet->GetAllRangeLengths()[RangeIdx];
		const int32 StartFrame = FilteredFrameRanges.FrameRangeSet->GetAllRangeStarts()[RangeIdx];

		SetAnimSequencePoseData(
			Database->GetAnimSequence(RangeSequenceIdx),
			Controllers[SequenceControllerIndices[RangeSequenceIdx]],
			StartFrame,
			PoseData.ConstSlice(FrameOffset, FrameNum),
			UsedBones,
			bShouldTransact);
	}

	UE::AnimDatabase::Private::CloseControllers(Controllers, bShouldTransact);
#endif
}

void UAnimDatabaseLibrary::DatabaseMakeLooped(UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const float StartEndRatio, const float BlendInTime, const float BlendOutTime, const bool bShouldTransact)
{
#if WITH_EDITOR
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseLibrary::DatabaseMakeLooped);

	DatabaseModifyPoseData(Database, FrameRanges, [Database, BlendInTime, BlendOutTime, StartEndRatio](const UE::AnimDatabase::FPoseDataView& InOutPoseData)
		{
			UE::AnimDatabase::PoseData::MakeLooped(InOutPoseData, Database->GetFrameRate(), StartEndRatio, BlendInTime, BlendOutTime);

		}, bShouldTransact);
#endif
}

void UAnimDatabaseLibrary::DatabasePatchRanges(UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const bool bApplyToRoot, const TArray<int32>& BoneIndices, const bool bShouldTransact)
{
#if WITH_EDITOR
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseLibrary::DatabasePatchRanges);

	if (!Database) { return; }

	DatabaseModifyPoseData(Database, FrameRanges, [Database, bApplyToRoot , &BoneIndices](const UE::AnimDatabase::FPoseDataView& InOutPoseData)
		{
			UE::AnimDatabase::PoseData::PatchInterpolate(InOutPoseData, Database->GetFrameRate(), bApplyToRoot, BoneIndices);

		}, bShouldTransact);
#endif
}

void UAnimDatabaseLibrary::DatabaseSetToReferencePose(UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const bool bApplyToRoot, const TArray<int32>& IgnoreBoneIndices, const bool bShouldTransact)
{
#if WITH_EDITOR
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseLibrary::DatabasePatchRanges);

	if (!Database || !Database->GetSkeleton()) { return; }

	DatabaseModifyPoseData(Database, FrameRanges, [Database, bApplyToRoot, &IgnoreBoneIndices](const UE::AnimDatabase::FPoseDataView& InOutPoseData)
		{
			UE::AnimDatabase::PoseData::Reset(InOutPoseData.AttributeData);
			UE::AnimDatabase::PoseData::Reset(InOutPoseData.LocalBoneData, Database->GetSkeleton()->GetReferenceSkeleton().GetRefBonePose(), IgnoreBoneIndices);

			if (bApplyToRoot)
			{
				UE::AnimDatabase::PoseData::Reset(InOutPoseData.RootData);
			}

		}, bShouldTransact);
#endif
}

void UAnimDatabaseLibrary::DatabaseRemoveFootGroundPenetration(UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName LeftToeBone, const FName RightToeBone, const float ToeBoneLength, const float PelvisHeightAdjustment, const FVector LeftKneeSideVector, const FVector RightKneeSideVector, const FVector LeftToeForwardVector, const FVector RightToeForwardVector, const bool bShouldTransact)
{
#if WITH_EDITOR
	if (!Database || !Database->GetSkeleton()) { return; }

	const int32 LeftToeBoneIndex = Database->FindBoneIndex(LeftToeBone);
	const int32 LeftHeelBoneIndex = Database->GetBoneParent(LeftToeBoneIndex);
	const int32 LeftKneeBoneIndex = Database->GetBoneParent(LeftHeelBoneIndex);
	const int32 LeftHipBoneIndex = Database->GetBoneParent(LeftKneeBoneIndex);
	const int32 LeftPelvisBoneIndex = Database->GetBoneParent(LeftHipBoneIndex);

	const int32 RightToeBoneIndex = Database->FindBoneIndex(RightToeBone);
	const int32 RightHeelBoneIndex = Database->GetBoneParent(RightToeBoneIndex);
	const int32 RightKneeBoneIndex = Database->GetBoneParent(RightHeelBoneIndex);
	const int32 RightHipBoneIndex = Database->GetBoneParent(RightKneeBoneIndex);
	const int32 RightPelvisBoneIndex = Database->GetBoneParent(RightHipBoneIndex);

	if (LeftToeBoneIndex == INDEX_NONE || RightToeBoneIndex == INDEX_NONE ||
		LeftHeelBoneIndex == INDEX_NONE || RightHeelBoneIndex == INDEX_NONE ||
		LeftKneeBoneIndex == INDEX_NONE || RightKneeBoneIndex == INDEX_NONE ||
		LeftHipBoneIndex == INDEX_NONE || RightHipBoneIndex == INDEX_NONE ||
		LeftPelvisBoneIndex == INDEX_NONE || RightPelvisBoneIndex == INDEX_NONE)
	{
		return; 
	}

	UE::AnimDatabase::PoseData::ContactPinning::FContactPinningSettings ContactSettings;
	ContactSettings.ContactHeight = Database->GetSkeleton()->GetReferenceSkeleton().GetBoneAbsoluteTransform(LeftToeBoneIndex).GetLocation().Z;
	ContactSettings.MinToeHeight = Database->GetSkeleton()->GetReferenceSkeleton().GetBoneAbsoluteTransform(LeftToeBoneIndex).GetLocation().Z;
	ContactSettings.MinAnkleHeight = Database->GetSkeleton()->GetReferenceSkeleton().GetBoneAbsoluteTransform(LeftHeelBoneIndex).GetLocation().Z;
	ContactSettings.ToeLength = ToeBoneLength;
	ContactSettings.HyperExtensionLimit = 0.0f;

	TLearningArray<1, int32> BoneParents;
	BoneParents.SetNumUninitialized({ Database->GetBoneNum() });
	Database->GetBoneParentsToArrayView(BoneParents);

	DatabaseModifyPoseData(Database, FrameRanges, [
		Database, 
		PelvisHeightAdjustment,
		LeftPelvisBoneIndex,
		RightPelvisBoneIndex,
		LeftHipBoneIndex,
		RightHipBoneIndex,
		LeftKneeBoneIndex,
		RightKneeBoneIndex,
		LeftHeelBoneIndex,
		RightHeelBoneIndex,
		LeftToeBoneIndex,
		RightToeBoneIndex,
		LeftKneeSideVector,
		RightKneeSideVector,
		LeftToeForwardVector,
		RightToeForwardVector,
		&BoneParents,
		&ContactSettings
	](const UE::AnimDatabase::FPoseDataView& InOutPoseData)
		{
			const int32 FrameNum = InOutPoseData.GetFrameNum();
			const int32 BoneNum = InOutPoseData.GetBoneNum();

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				InOutPoseData.LocalBoneData.BoneLocations[FrameIdx][LeftPelvisBoneIndex] += FVector3f(0.0f, 0.0f, PelvisHeightAdjustment);
			}
		
			UE::AnimDatabase::FPoseGlobalBoneData GlobalData;
			GlobalData.Resize(FrameNum, BoneNum);

			UE::AnimDatabase::PoseData::ForwardKinematics(
				GlobalData.View(), 
				InOutPoseData.LocalBoneData.ConstView(), 
				InOutPoseData.RootData.ConstView(),
				BoneParents);

			UE::AnimDatabase::PoseData::ContactPinning::RemoveFootGroundPenetration(
				GlobalData.View(),
				InOutPoseData.LocalBoneData,
				InOutPoseData.RootData.ConstView(),
				BoneParents,
				{ LeftPelvisBoneIndex, RightPelvisBoneIndex },
				{ LeftHipBoneIndex, RightHipBoneIndex },
				{ LeftKneeBoneIndex, RightKneeBoneIndex },
				{ LeftHeelBoneIndex, RightHeelBoneIndex },
				{ LeftToeBoneIndex, RightToeBoneIndex },
				{ (FVector3f)(LeftKneeSideVector), (FVector3f)RightKneeSideVector },
				{ (FVector3f)(LeftToeForwardVector), (FVector3f)RightToeForwardVector },
				ContactSettings);

		}, bShouldTransact);
#endif
}

#if WITH_EDITOR

void UAnimDatabaseFunction_StripUnusedBoneTracks::Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact)
{
	UAnimDatabaseLibrary::DatabaseStripUnusedBoneTracks(InDatabase, InFrameRanges, bShouldTransact);
}

void UAnimDatabaseFunction_ExportBVH::Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact)
{
	if (!ExportDirectory.Path.IsEmpty())
	{
		UAnimDatabaseLibrary::DatabaseExportAsBVH(InDatabase, InFrameRanges, ExportDirectory, bAddRootAxesChangeRotation, RotationOrder);
	}
}

void UAnimDatabaseFunction_ImportBVH::Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact)
{
	if (!ImportDirectory.Path.IsEmpty() && !AssetDirectory.Path.IsEmpty())
	{
		UAnimDatabaseLibrary::DatabaseImportFromBVH(InDatabase, ImportDirectory, AssetDirectory, bRemoveRootAxesChangeRotation, bAddToDatabase, bIgnoreMirrored);
	}
}

void UAnimDatabaseFunction_ExportNotifies::Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact)
{
	if (!ExportDirectory.Path.IsEmpty())
	{
		FFilePath ExportFile;
		ExportFile.FilePath = ExportDirectory.Path + TEXT("/") + FileName;
		UAnimDatabaseLibrary::DatabaseExportNotifies(InDatabase, InFrameRanges, ExportFile);
	}
}

void UAnimDatabaseFunction_ImportNotifies::Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact)
{
	if (!ImportPath.FilePath.IsEmpty())
	{
		UAnimDatabaseLibrary::DatabaseImportNotifies(InDatabase, InFrameRanges, ImportPath);
	}
}

void UAnimDatabaseFunction_RemoveNotifies::Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact)
{
	UAnimDatabaseLibrary::DatabaseRemoveNotifies(InDatabase, InFrameRanges);
}

void UAnimDatabaseFunction_UpdateIKBones::Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact)
{
	TArray<FAnimDatabaseFrameAttribute> Attributes;
	Attributes.SetNum(5);
	UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalTransformFrameAttributesFromNamesArrayView(
		Attributes, InDatabase, InFrameRanges, { RootBone, FootBoneL, FootBoneR, HandBoneL, HandBoneR });

	UAnimDatabaseLibrary::DatabaseSetLocalBoneTransformsFromArrayViews(
		InDatabase,
		{ IKFootBoneL, IKFootBoneR, IKHandBoneGun, IKHandBoneL, IKHandBoneR },
		{
			UAnimDatabaseFrameAttributeLibrary::FrameAttributeMultiplyInverse(Attributes[1], Attributes[0]),
			UAnimDatabaseFrameAttributeLibrary::FrameAttributeMultiplyInverse(Attributes[2], Attributes[0]),
			UAnimDatabaseFrameAttributeLibrary::FrameAttributeMultiplyInverse(Attributes[4], Attributes[0]),
			UAnimDatabaseFrameAttributeLibrary::FrameAttributeMultiplyInverse(Attributes[3], Attributes[4]),
			UAnimDatabaseFrameAttributeLibrary::MakeTransformFrameAttributeFromIdentity(InFrameRanges),
		}, bShouldTransact);
}

void UAnimDatabaseFunction_SetRootFromLookAt::Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact)
{
	const FAnimDatabaseFrameAttribute SmoothedRootTransform = bUseSavGolFilter ?
		
		UAnimDatabaseFrameAttributeLibrary::FrameAttributeSavGolSmoothedRootTransform(
			InDatabase,
			InFrameRanges,
			RootLocationBone,
			RootDirectionBone,
			RootDirectionBoneDirection,
			RootDirection,
			bApplySmoothing ? FMath::RoundToInt(LocationSavGolFilterWidth * InDatabase->GetFrameRate().AsDecimal()) : 0,
			LocationSavGolPolynomialDegree,
			bApplySmoothing ? FMath::RoundToInt(DirectionSavGolFilterWidth * InDatabase->GetFrameRate().AsDecimal()) : 0,
			DirectionSavGolPolynomialDegree,
			bUseGaussianWindowedSavGolFilter) :
		
		UAnimDatabaseFrameAttributeLibrary::FrameAttributeGaussianSmoothedRootTransform(
			InDatabase,
			InFrameRanges,
			RootLocationBone,
			RootDirectionBone,
			RootDirectionBoneDirection,
			RootDirection,
			bApplySmoothing ? LocationSmoothingTime * InDatabase->GetFrameRate().AsDecimal() : 0.0f,
			bApplySmoothing ? DirectionSmoothingTime * InDatabase->GetFrameRate().AsDecimal() : 0.0f);

	const FAnimDatabaseFrameAttribute PelvisTransform = UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalTransformFrameAttributeFromName(InDatabase, InFrameRanges, PelvisBone);

	UAnimDatabaseLibrary::DatabaseSetLocalBoneTransformsFromArrayViews(
		InDatabase,
		{ RootBone, PelvisBone },
		{
			SmoothedRootTransform,
			UAnimDatabaseFrameAttributeLibrary::FrameAttributeMultiplyInverse(PelvisTransform, SmoothedRootTransform),
		}, bShouldTransact);
}

void UAnimDatabaseFunction_SetRootFromPelvis::Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact)
{
	const FAnimDatabaseFrameAttribute PelvisTransform = UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalTransformFrameAttributeFromName(InDatabase, InFrameRanges, PelvisBone);
	const FAnimDatabaseFrameAttribute RootTransform = bApplyInLocalSpace ?
		UAnimDatabaseFrameAttributeLibrary::FrameAttributeTransformMultiply(Offset, PelvisTransform) :
		UAnimDatabaseFrameAttributeLibrary::FrameAttributeMultiplyTransform(PelvisTransform, Offset);

	UAnimDatabaseLibrary::DatabaseSetLocalBoneTransformsFromArrayViews(
		InDatabase,
		{ RootBone, PelvisBone },
		{
			RootTransform,
			UAnimDatabaseFrameAttributeLibrary::FrameAttributeMultiplyInverse(PelvisTransform, RootTransform),
		}, bShouldTransact);
}

void UAnimDatabaseFunction_BakeRootMotionIntoPelvis::Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact)
{
	const FAnimDatabaseFrameAttribute InitialRootTransform = UAnimDatabaseFrameAttributeLibrary::MakeRootTransformAtSequenceStartFrameAttribute(InDatabase, InFrameRanges);
	const FAnimDatabaseFrameAttribute PelvisTransform = UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalTransformFrameAttributeFromName(InDatabase, InFrameRanges, PelvisBone);

	UAnimDatabaseLibrary::DatabaseSetLocalBoneTransformsFromArrayViews(
		InDatabase,
		{ RootBone, PelvisBone },
		{
			InitialRootTransform,
			UAnimDatabaseFrameAttributeLibrary::FrameAttributeMultiplyInverse(PelvisTransform, InitialRootTransform),
		}, bShouldTransact);
}

void UAnimDatabaseFunction_MakeRootStartAtOrigin::Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact)
{
	const FAnimDatabaseFrameAttribute RootTransform = UAnimDatabaseFrameAttributeLibrary::MakeRootTransformFrameAttribute(InDatabase, InFrameRanges);
	const FAnimDatabaseFrameAttribute InitialRootTransform = UAnimDatabaseFrameAttributeLibrary::FrameAttributeRepeatFirstInRange(RootTransform);

	UAnimDatabaseLibrary::DatabaseSetLocalBoneTransformsFromArrayViews(
		InDatabase,
		{ RootBone },
		{
			UAnimDatabaseFrameAttributeLibrary::FrameAttributeMultiplyInverse(RootTransform, InitialRootTransform),
		}, bShouldTransact);
}

void UAnimDatabaseFunction_ExportToAnimSequences::Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact)
{
	if (!ExportDirectory.Path.IsEmpty())
	{
		UAnimDatabaseLibrary::DatabaseExportAsAnimSequences(InDatabase, InFrameRanges, ExportDirectory, AssetNameFormatString, bExportMirrored, bShouldTransact);
	}
}

void UAnimDatabaseFunction_ExportRootMotion::Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact)
{
	if (!ExportDirectory.Path.IsEmpty())
	{
		FFilePath ExportFile;
		ExportFile.FilePath = ExportDirectory.Path + TEXT("/") + FileName;
		UAnimDatabaseLibrary::DatabaseExportRootMotion(InDatabase, InFrameRanges, ExportFile);
	}
}

void UAnimDatabaseFunction_MakeLooped::Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact)
{
	UAnimDatabaseLibrary::DatabaseMakeLooped(InDatabase, InFrameRanges, StartEndRatio, BlendInTime, BlendOutTime, bShouldTransact);
}

void UAnimDatabaseFunction_PatchBones::Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact)
{
	TArray<int32> BoneIndices;
	if (bEntirePose)
	{
		const int32 BoneNum = InDatabase->GetBoneNum();
		for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
		{
			BoneIndices.Add(BoneIdx);
		}
	}
	else
	{
		InDatabase->FindBoneIndices(BoneIndices, BoneNames);
	}

	UAnimDatabaseLibrary::DatabasePatchRanges(InDatabase, InFrameRanges, bApplyToRoot, BoneIndices, bShouldTransact);
}

void UAnimDatabaseFunction_SetToReferencePose::Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact)
{
	TArray<int32> IgnoreBoneIndices;
	InDatabase->FindBoneIndices(IgnoreBoneIndices, IgnoreBoneNames);

	UAnimDatabaseLibrary::DatabaseSetToReferencePose(InDatabase, InFrameRanges, bApplyToRoot, IgnoreBoneIndices, bShouldTransact);
}

void UAnimDatabaseFunction_BoneApplyTransform::Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact)
{
	if (Mode == EAnimDatabaseTransformMode::Replace)
	{
		UAnimDatabaseLibrary::DatabaseSetLocalBoneTransform(InDatabase, BoneName, UAnimDatabaseFrameAttributeLibrary::MakeTransformFrameAttributeFromConstant(InFrameRanges, Transform), bShouldTransact);
	}
	else if (Mode == EAnimDatabaseTransformMode::PreMultiply)
	{
		UAnimDatabaseLibrary::DatabaseSetLocalBoneTransform(InDatabase, BoneName, 
			UAnimDatabaseFrameAttributeLibrary::FrameAttributeTransformMultiply(Transform, UAnimDatabaseFrameAttributeLibrary::MakeBoneLocalTransformFrameAttributeFromName(InDatabase, InFrameRanges, BoneName)), bShouldTransact);
	}
	else if (Mode == EAnimDatabaseTransformMode::PostMultiply)
	{
		UAnimDatabaseLibrary::DatabaseSetLocalBoneTransform(InDatabase, BoneName,
			UAnimDatabaseFrameAttributeLibrary::FrameAttributeMultiplyTransform(UAnimDatabaseFrameAttributeLibrary::MakeBoneLocalTransformFrameAttributeFromName(InDatabase, InFrameRanges, BoneName), Transform), bShouldTransact);
	}
}

void UAnimDatabaseFunction_AddAnimNotifyState::Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact)
{
	UAnimDatabaseLibrary::DatabaseAddAnimNotifyStateObject(InDatabase, TrackName, AnimNotifyState, InFrameRanges, Color, bShouldTransact);
}

void UAnimDatabaseFunction_Sequence::Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact)
{
	for (TObjectPtr<UAnimDatabaseFunction> Function : Functions)
	{
		if (Function)
		{
			Function->Run(InDatabase, InFrameRanges, bShouldTransact);
		}
	}
}

void UAnimDatabaseFunction_AddAnimNotifyAtFrames::Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact)
{
	UAnimDatabaseLibrary::DatabaseAddAnimNotifyObject(
		InDatabase,
		TrackName,
		AnimNotify, 
		UAnimDatabaseFramesLibrary::MakeFramesFromFunction(InDatabase, InFrameRanges, Frames),
		Color,
		bShouldTransact);
}

void UAnimDatabaseFunction_AddAnimNotifyStateAtFrameRanges::Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact)
{
	UAnimDatabaseLibrary::DatabaseAddAnimNotifyStateObject(
		InDatabase,
		TrackName,
		AnimNotifyState,
		UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFunction(InDatabase, InFrameRanges, FrameRanges),
		Color,
		bShouldTransact);
}

void UAnimDatabaseFunction_AddCurveFromFrameAttribute::Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact)
{
	UAnimDatabaseLibrary::DatabaseAddCurve(
		InDatabase, 
		CurveName, 
		UAnimDatabaseFrameAttributeLibrary::MakeFrameAttributeFromFunction(InDatabase, InFrameRanges, FrameAttribute),
		Color,
		bSparseKeys, 
		bShouldTransact);
}

void UAnimDatabaseFunction_ProcessContacts::Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact)
{
	if (!InDatabase) { return; }
	if (!InFrameRanges.IsValid()) { return; }

	InDatabase->WaitForCompressionOnAnimSequencesFromArrayView(InFrameRanges.FrameRangeSet->GetEntrySequences());

	UE::Tasks::TTask<FAnimDatabaseFrameAttribute> LeftFootSpeedCurve, RightFootSpeedCurve;

	if (bAddFootSpeedCurves)
	{
		LeftFootSpeedCurve = UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, InDatabase, InFrameRanges]() {

			return UAnimDatabaseFrameAttributeLibrary::FrameAttributeBoneSpeed(
				InDatabase,
				InFrameRanges,
				LeftFootSpeedBoneName);
			});

		RightFootSpeedCurve = UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, InDatabase, InFrameRanges]() {

			return UAnimDatabaseFrameAttributeLibrary::FrameAttributeBoneSpeed(
				InDatabase,
				InFrameRanges,
				RightFootSpeedBoneName);
			});

		UAnimDatabaseLibrary::DatabaseAddCurvesFromArrayViews(
			InDatabase,
			{ LeftFootSpeedCurveName, RightFootSpeedCurveName },
			{ LeftFootSpeedCurve.GetResult(), RightFootSpeedCurve.GetResult() },
			{ LeftFootSpeedCurveColor, RightFootSpeedCurveColor },
			bSparseKeys,
			bShouldTransact);
	}

	UE::Tasks::TTask<FAnimDatabaseFrameRanges> LeftContactRanges, RightContactRanges;

	if (bAddContactCurves || bAddAnimNotifies || bAddSyncMarkers)
	{
		const int32 MajorityVoteFilterWidth = FMath::Max(FMath::RoundToInt(2.0f * ContactFilterTime * InDatabase->GetFrameRate().AsDecimal()), 0);

		LeftContactRanges = UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, MajorityVoteFilterWidth, InDatabase, InFrameRanges]() {

			return UAnimDatabaseFrameAttributeLibrary::FrameAttributeContactRanges(
				InDatabase,
				InFrameRanges,
				LeftContactBoneName,
				HeightThreshold,
				VelocityThreshold,
				bFilterContactCurves,
				MajorityVoteFilterWidth);
			});

		RightContactRanges = UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, MajorityVoteFilterWidth, InDatabase, InFrameRanges]() {

			return UAnimDatabaseFrameAttributeLibrary::FrameAttributeContactRanges(
				InDatabase,
				InFrameRanges,
				RightContactBoneName,
				HeightThreshold,
				VelocityThreshold,
				bFilterContactCurves,
				MajorityVoteFilterWidth);
			});
	}

	if (bAddContactCurves)
	{
		UE::Tasks::TTask<FAnimDatabaseFrameAttribute> LeftContactCurve = UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, &LeftContactRanges, InDatabase, InFrameRanges]() {

				const FAnimDatabaseFrameAttribute LeftActive = UAnimDatabaseFrameAttributeLibrary::MakeBoolFrameAttributeFromActiveRanges(LeftContactRanges.GetResult(), InFrameRanges);
				return bSmoothContactCurves ? UAnimDatabaseFrameAttributeLibrary::FrameAttributeFilterGaussian(LeftActive, SmoothingAmount) : LeftActive;

			});

		UE::Tasks::TTask<FAnimDatabaseFrameAttribute> RightContactCurve = UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, &RightContactRanges, InDatabase, InFrameRanges]() {

				const FAnimDatabaseFrameAttribute RightActive = UAnimDatabaseFrameAttributeLibrary::MakeBoolFrameAttributeFromActiveRanges(RightContactRanges.GetResult(), InFrameRanges);
				return bSmoothContactCurves ? UAnimDatabaseFrameAttributeLibrary::FrameAttributeFilterGaussian(RightActive, SmoothingAmount) : RightActive;

			});

		UAnimDatabaseLibrary::DatabaseAddCurvesFromArrayViews(
			InDatabase,
			{ LeftContactCurveName, RightContactCurveName },
			{ LeftContactCurve.GetResult(), RightContactCurve.GetResult() },
			{ LeftContactCurveColor, RightContactCurveColor },
			bSparseKeys,
			bShouldTransact);
	}

	if (bAddAnimNotifies || bAddSyncMarkers)
	{
		if (bAddAnimNotifies)
		{
			FAnimDatabaseFrames LeftNotifyFrames = UAnimDatabaseFramesLibrary::FramesBefore(InDatabase, UAnimDatabaseFrameRangesLibrary::MakeFramesAtFrameRangesStarts(LeftContactRanges.GetResult()),
				FMath::RoundToInt(InDatabase->GetFrameRate().AsDecimal() * AnimNotifyOffset), EAnimDatabaseFrameShiftBehavior::Clamp);

			FAnimDatabaseFrames RightNotifyFrames = UAnimDatabaseFramesLibrary::FramesBefore(InDatabase, UAnimDatabaseFrameRangesLibrary::MakeFramesAtFrameRangesStarts(RightContactRanges.GetResult()),
				FMath::RoundToInt(InDatabase->GetFrameRate().AsDecimal() * AnimNotifyOffset), EAnimDatabaseFrameShiftBehavior::Clamp);

			if (bRemoveFirstFrameContactEvent)
			{
				const FAnimDatabaseFrames FrameRangeStartFrames = UAnimDatabaseFrameRangesLibrary::MakeFramesAtFrameRangesStarts(InFrameRanges);
				LeftNotifyFrames = UAnimDatabaseFramesLibrary::FramesDifference(LeftNotifyFrames, FrameRangeStartFrames);
				RightNotifyFrames = UAnimDatabaseFramesLibrary::FramesDifference(RightNotifyFrames, FrameRangeStartFrames);
			}

			if (bClearAnimNotifyTracks)
			{
				UAnimDatabaseLibrary::DatabaseRemoveAnimNotifiesFromTracksArrayView(
					InDatabase, InFrameRanges,
					{ LeftContactAnimNotifyTrackName, RightContactAnimNotifyTrackName },
					bShouldTransact);
			}

			UAnimDatabaseLibrary::DatabaseAddAnimNotifyObjectsFromArrayViews(
				InDatabase,
				{ LeftContactAnimNotifyTrackName, RightContactAnimNotifyTrackName },
				{ LeftContactAnimNotify, RightContactAnimNotify },
				{ LeftNotifyFrames, RightNotifyFrames },
				{ LeftContactAnimNotifyColor, RightContactAnimNotifyColor },
				bShouldTransact);
		}

		if (bAddSyncMarkers)
		{
			FAnimDatabaseFrames LeftSyncMarkerFrames = UAnimDatabaseFramesLibrary::FramesBefore(InDatabase, UAnimDatabaseFrameRangesLibrary::MakeFramesAtFrameRangesStarts(LeftContactRanges.GetResult()),
				FMath::RoundToInt(InDatabase->GetFrameRate().AsDecimal() * SyncMarkerOffset), EAnimDatabaseFrameShiftBehavior::Clamp);

			FAnimDatabaseFrames RightSyncMarkerFrames = UAnimDatabaseFramesLibrary::FramesBefore(InDatabase, UAnimDatabaseFrameRangesLibrary::MakeFramesAtFrameRangesStarts(RightContactRanges.GetResult()),
				FMath::RoundToInt(InDatabase->GetFrameRate().AsDecimal() * SyncMarkerOffset), EAnimDatabaseFrameShiftBehavior::Clamp);

			if (bRemoveFirstFrameContactEvent)
			{
				const FAnimDatabaseFrames FrameRangeStartFrames = UAnimDatabaseFrameRangesLibrary::MakeFramesAtFrameRangesStarts(InFrameRanges);
				LeftSyncMarkerFrames = UAnimDatabaseFramesLibrary::FramesDifference(LeftSyncMarkerFrames, FrameRangeStartFrames);
				RightSyncMarkerFrames = UAnimDatabaseFramesLibrary::FramesDifference(RightSyncMarkerFrames, FrameRangeStartFrames);
			}

			if (bClearSyncMarkerTracks)
			{
				UAnimDatabaseLibrary::DatabaseRemoveSyncMarkerFromTracksArrayView(
					InDatabase, InFrameRanges,
					{ LeftContactSyncMarkerTrackName, RightContactSyncMarkerTrackName },
					bShouldTransact);
			}

			UAnimDatabaseLibrary::DatabaseAddSyncMarkersFromArrayViews(
				InDatabase,
				{ LeftContactSyncMarkerTrackName, RightContactSyncMarkerTrackName },
				{ LeftContactSyncMarker, RightContactSyncMarker },
				{ LeftSyncMarkerFrames, RightSyncMarkerFrames },
				{ LeftContactSyncMarkerColor, RightContactSyncMarkerColor },
				bShouldTransact);
		}
	}
}

void UAnimDatabaseFunction_ProcessPhaseCurves::Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact)
{
	const FAnimDatabaseFrameAttribute PhaseAngle = UAnimDatabaseFrameAttributeLibrary::FrameAttributeExtractPhase(
		InFrameRanges, 
		UAnimDatabaseFramesLibrary::MakeFramesFromFunction(InDatabase, InFrameRanges, ZeroPhaseFrames),
		UAnimDatabaseFramesLibrary::MakeFramesFromFunction(InDatabase, InFrameRanges, HalfPhaseFrames),
		ExtrapolationMode);

	if (bAddPhaseAngle)
	{
		FAnimDatabaseFrameAttribute PhaseAngleValue = UAnimDatabaseFrameAttributeLibrary::FrameAttributeAngleToFloatDegrees(PhaseAngle);

		if (bRescalePhaseAngle)
		{
			PhaseAngleValue = UAnimDatabaseFrameAttributeLibrary::FrameAttributeAddFloat(UAnimDatabaseFrameAttributeLibrary::FrameAttributeDivideFloat(PhaseAngleValue, 360.0f), 0.5f);
		}

		UAnimDatabaseLibrary::DatabaseAddCurves(
			InDatabase,
			{ PhaseAngleCurveName },
			{ PhaseAngleValue },
			{ PhaseAngleCurveColor },
			bSparseKeys,
			bShouldTransact);
	}

	if (bAddPhaseDirection)
	{
		const FAnimDatabaseFrameAttribute PhaseDirectionX = UAnimDatabaseFrameAttributeLibrary::FrameAttributeCos(PhaseAngle);
		const FAnimDatabaseFrameAttribute PhaseDirectionY = UAnimDatabaseFrameAttributeLibrary::FrameAttributeSin(PhaseAngle);

		UAnimDatabaseLibrary::DatabaseAddCurves(
			InDatabase,
			{ PhaseDirectionXCurveName, PhaseDirectionYCurveName },
			{ PhaseDirectionX, PhaseDirectionY },
			{ PhaseDirectionXCurveColor, PhaseDirectionYCurveColor },
			bSparseKeys,
			bShouldTransact);
	}

	if (bAddPhaseRamp)
	{
		const FAnimDatabaseFrameAttribute PhaseRamp = UAnimDatabaseFrameAttributeLibrary::FrameAttributeSubtractFloat(
			UAnimDatabaseFrameAttributeLibrary::FrameAttributeMultiplyFloat(
				UAnimDatabaseFrameAttributeLibrary::FrameAttributeAbs(
					UAnimDatabaseFrameAttributeLibrary::FrameAttributeDivideFloat(
						UAnimDatabaseFrameAttributeLibrary::FrameAttributeAngleToFloatDegrees(PhaseAngle), 180.0f)), 2.0f), 1.0f);

		UAnimDatabaseLibrary::DatabaseAddCurves(
			InDatabase,
			{ PhaseRampCurveName },
			{ PhaseRamp },
			{ PhaseRampCurveColor },
			bSparseKeys,
			bShouldTransact);
	}
}

void UAnimDatabaseFunction_SetBoneLocalTransformFromFrameAttribute::Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact)
{
	UAnimDatabaseLibrary::DatabaseSetLocalBoneTransform(InDatabase, BoneName, UAnimDatabaseFrameAttributeLibrary::MakeFrameAttributeFromFunction(InDatabase, InFrameRanges, FrameAttribute), bShouldTransact);
}

void UAnimDatabaseFunction_SetBoneGlobalTransformFromFrameAttribute::Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact)
{
	const int32 BoneIndex = InDatabase->FindBoneIndex(BoneName);
	if (BoneIndex == INDEX_NONE) { return; }
	const int32 ParentIndex = InDatabase->GetBoneParent(BoneIndex);
	if (ParentIndex == INDEX_NONE) { return; }

	UAnimDatabaseLibrary::DatabaseSetLocalBoneTransform(InDatabase, BoneName, 
		UAnimDatabaseFrameAttributeLibrary::FrameAttributeDivide(
			UAnimDatabaseFrameAttributeLibrary::MakeFrameAttributeFromFunction(InDatabase, InFrameRanges, FrameAttribute),
			UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalTransformFrameAttribute(InDatabase, InFrameRanges, ParentIndex)), bShouldTransact);
}

void UAnimDatabaseFunction_CopyBones::Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact)
{
	const int32 BoneNum = BoneMap.Num();

	TArray<FAnimDatabaseFrameAttribute, TInlineAllocator<16>> FrameAttributes;
	FrameAttributes.SetNum(BoneNum);

	TArray<FName, TInlineAllocator<16>> SrcBones, DstBones;
	SrcBones.SetNum(BoneNum);
	DstBones.SetNum(BoneNum);

	BoneMap.GetKeys(SrcBones);
	for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
	{
		DstBones[BoneIdx] = BoneMap[SrcBones[BoneIdx]];
	}

	switch (Space)
	{
	case EAnimDatabaseBoneSpace::Global:
	{
		UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalTransformFrameAttributesFromNamesArrayView(FrameAttributes, InDatabase, InFrameRanges, SrcBones);
		UAnimDatabaseLibrary::DatabaseSetGlobalBoneTransformsFromArrayViews(InDatabase, DstBones, FrameAttributes, bShouldTransact);
		return;
	}
	
	case EAnimDatabaseBoneSpace::Local:
	{
		UAnimDatabaseFrameAttributeLibrary::MakeBoneLocalTransformFrameAttributesFromNamesArrayView(FrameAttributes, InDatabase, InFrameRanges, SrcBones);
		UAnimDatabaseLibrary::DatabaseSetLocalBoneTransformsFromArrayViews(InDatabase, DstBones, FrameAttributes, bShouldTransact);
		return;
	}
	}
}

void UAnimDatabaseFunction_ResetBones::Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact)
{
	UAnimDatabaseLibrary::DatabaseResetBoneTransformsFromArrayViews(InDatabase, InFrameRanges, Bones, bShouldTransact);
}

void UAnimDatabaseFunction_SetBoneToRootTransformAtFrames::Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact)
{
	const int32 BoneIndex = InDatabase->FindBoneIndex(BoneName);
	const int32 ParentIndex = InDatabase->GetBoneParent(BoneIndex);

	if (BoneIndex == INDEX_NONE || ParentIndex == INDEX_NONE) { return; }

	const FAnimDatabaseFrames SingleFrames = UAnimDatabaseFramesLibrary::MakeFramesFromFunction(InDatabase, InFrameRanges, Frames);

	if (!SingleFrames.IsValid()) { return; }

	FAnimDatabaseFrameAttribute RootLocation, RootDirection;
	UAnimDatabaseFrameAttributeLibrary::MakeRootLocationAndDirectionFrameAttribute(RootLocation, RootDirection, InDatabase, UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFrames(SingleFrames));

	FVector AvgRootLocation = FVector::ZeroVector;
	FVector AvgRootDirection = FVector::ForwardVector;
	UAnimDatabaseFrameAttributeLibrary::FrameAttributeLocationAverage(AvgRootLocation, RootLocation);
	UAnimDatabaseFrameAttributeLibrary::FrameAttributeDirectionAverage(AvgRootDirection, RootDirection);

	UAnimDatabaseLibrary::DatabaseSetLocalBoneTransform(
		InDatabase,
		BoneName,
		UAnimDatabaseFrameAttributeLibrary::FrameAttributeMultiplyInverse(
			UAnimDatabaseFrameAttributeLibrary::MakeTransformFrameAttributeNoScale(
				UAnimDatabaseFrameAttributeLibrary::MakeLocationFrameAttributeFromConstant(InFrameRanges, AvgRootLocation),
				UAnimDatabaseFrameAttributeLibrary::MakeRotationFrameAttributeFromConstant(InFrameRanges, AvgRootDirection.ToOrientationRotator())),
			UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalTransformFrameAttribute(InDatabase, InFrameRanges, ParentIndex)),
		bShouldTransact);

}

void UAnimDatabaseFunction_AdjustNeckRotation::Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact)
{
	const FAnimDatabaseFrameAttribute Neck0Transform = UAnimDatabaseFrameAttributeLibrary::MakeBoneLocalTransformFrameAttributeFromName(InDatabase, InFrameRanges, Neck0Bone);
	const FAnimDatabaseFrameAttribute Neck1Transform = UAnimDatabaseFrameAttributeLibrary::MakeBoneLocalTransformFrameAttributeFromName(InDatabase, InFrameRanges, Neck1Bone);
	const FAnimDatabaseFrameAttribute HeadTransform = UAnimDatabaseFrameAttributeLibrary::MakeBoneLocalTransformFrameAttributeFromName(InDatabase, InFrameRanges, HeadBone);

	UAnimDatabaseLibrary::DatabaseSetLocalBoneTransforms(
		InDatabase,
		{ Neck0Bone, Neck1Bone, HeadBone },
		{
			UAnimDatabaseFrameAttributeLibrary::FrameAttributeMultiply(UAnimDatabaseFrameAttributeLibrary::MakeTransformFrameAttributeNoScale(
				UAnimDatabaseFrameAttributeLibrary::MakeLocationFrameAttributeFromConstant(InFrameRanges, FVector::ZeroVector),
				UAnimDatabaseFrameAttributeLibrary::MakeRotationFrameAttributeFromConstant(InFrameRanges, FQuat::MakeFromRotationVector(FMath::DegreesToRadians(Neck0Rotation) * LocalAxis).Rotator())), Neck0Transform),
			UAnimDatabaseFrameAttributeLibrary::FrameAttributeMultiply(UAnimDatabaseFrameAttributeLibrary::MakeTransformFrameAttributeNoScale(
				UAnimDatabaseFrameAttributeLibrary::MakeLocationFrameAttributeFromConstant(InFrameRanges, FVector::ZeroVector),
				UAnimDatabaseFrameAttributeLibrary::MakeRotationFrameAttributeFromConstant(InFrameRanges, FQuat::MakeFromRotationVector(FMath::DegreesToRadians(Neck1Rotation) * LocalAxis).Rotator())), Neck1Transform),
			UAnimDatabaseFrameAttributeLibrary::FrameAttributeMultiply(UAnimDatabaseFrameAttributeLibrary::MakeTransformFrameAttributeNoScale(
				UAnimDatabaseFrameAttributeLibrary::MakeLocationFrameAttributeFromConstant(InFrameRanges, FVector::ZeroVector),
				UAnimDatabaseFrameAttributeLibrary::MakeRotationFrameAttributeFromConstant(InFrameRanges, FQuat::MakeFromRotationVector(FMath::DegreesToRadians(HeadRotation) * LocalAxis).Rotator())), HeadTransform)
		},
		bShouldTransact);
}

void UAnimDatabaseFunction_RemoveFootGroundPenetration::Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact)
{
	UAnimDatabaseLibrary::DatabaseRemoveFootGroundPenetration(InDatabase, InFrameRanges, LeftToeBone, RightToeBone, ToeBoneLength, PelvisHeightAdjustment, LeftKneeSideVector, RightKneeSideVector, LeftToeForwardVector, RightToeForwardVector, bShouldTransact);
}

void UAnimDatabaseFunction_Statistics::Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact)
{
	UAnimDatabaseFrameRangesLibrary::FrameRangesStatistics(Statistics, InDatabase, InFrameRanges);
}

void UAnimDatabaseFunction_SpeedStatistics::Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact)
{
	const FAnimDatabaseFrameRanges TransitionRanges = FrameRanges ? UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFunction(InDatabase, InFrameRanges, FrameRanges) : InFrameRanges;

	const FAnimDatabaseFrameAttribute RootLinearVelocityMagnitude = UAnimDatabaseFrameAttributeLibrary::FrameAttributeLength(UAnimDatabaseFrameAttributeLibrary::MakeRootLinearVelocityFrameAttribute(InDatabase, TransitionRanges));
	const FAnimDatabaseFrameAttribute RootAngularVelocityMagnitude = UAnimDatabaseFrameAttributeLibrary::FrameAttributeLength(UAnimDatabaseFrameAttributeLibrary::MakeRootAngularVelocityFrameAttribute(InDatabase, TransitionRanges));

	UAnimDatabaseFrameAttributeLibrary::FrameAttributeFloatAverage(AverageLinearVelocity, RootLinearVelocityMagnitude);
	UAnimDatabaseFrameAttributeLibrary::FrameAttributeFloatMinAndMax(MinimumLinearVelocity, MaximumLinearVelocity, RootLinearVelocityMagnitude);

	UAnimDatabaseFrameAttributeLibrary::FrameAttributeFloatAverage(AverageAngularVelocity, RootAngularVelocityMagnitude);
	UAnimDatabaseFrameAttributeLibrary::FrameAttributeFloatMinAndMax(MinimumAngularVelocity, MaximumAngularVelocity, RootAngularVelocityMagnitude);

	AverageAngularVelocity = FMath::RadiansToDegrees(AverageAngularVelocity);
	MinimumAngularVelocity = FMath::RadiansToDegrees(MinimumAngularVelocity);
	MaximumAngularVelocity = FMath::RadiansToDegrees(MaximumAngularVelocity);
}

void UAnimDatabaseFunction_TransitionStatistics::Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact)
{
	const FAnimDatabaseFrameRanges TransitionRanges = FrameRanges ? UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFunction(InDatabase, InFrameRanges, FrameRanges) : InFrameRanges;

	AverageTime = UAnimDatabaseFrameRangesLibrary::FrameRangesAverageFrameNum(TransitionRanges) / FMath::Max(InDatabase->GetFrameRate().AsDecimal(), UE_SMALL_NUMBER);
	MinimumTime = UAnimDatabaseFrameRangesLibrary::FrameRangesMinFrameNum(TransitionRanges) / FMath::Max(InDatabase->GetFrameRate().AsDecimal(), UE_SMALL_NUMBER);
	MaximumTime = UAnimDatabaseFrameRangesLibrary::FrameRangesMaxFrameNum(TransitionRanges) / FMath::Max(InDatabase->GetFrameRate().AsDecimal(), UE_SMALL_NUMBER);

	float StdDistance = 0.0f;
	UAnimDatabaseFrameAttributeLibrary::FrameAttributeLocationDistanceTraveled(
		AverageDistance, 
		StdDistance, 
		MinimumDistance, 
		MaximumDistance, 
		UAnimDatabaseFrameAttributeLibrary::MakeRootLocationFrameAttribute(InDatabase, TransitionRanges));

	float StdAngle = 0.0f;
	UAnimDatabaseFrameAttributeLibrary::FrameAttributeRotationAngleTraveled(
		AverageAngle,
		StdAngle,
		MinimumAngle,
		MaximumAngle,
		UAnimDatabaseFrameAttributeLibrary::MakeRootRotationFrameAttribute(InDatabase, TransitionRanges));
}

void UAnimDatabaseDebugDraw_Attributes::DrawDebug_Implementation(
	const FDebugDrawer& Drawer,
	const FDebugDrawer& CanvasDrawer,
	const FAnimDatabasePoseState& InPoseState,
	const UAnimDatabase* InDatabase,
	const FAnimDatabaseFrameRanges& InFrameRanges,
	const FTransform& RangeViewportTransform,
	const int32 CharacterIdx,
	const int32 SequenceIdx,
	const float SequenceTime,
	const int32 RangeStart,
	const int32 RangeLength,
	const FLinearColor& IdentifierColor)
{
	const int32 AttributeNum = UAnimDatabasePoseStateLibrary::PoseStateAttributeNum(InPoseState);

	const float Offset = VerticalSpacing + CharacterIdx * AttributeNum * VerticalSpacing;

	FDrawDebugCanvasTextSettings Settings;
	Settings.bShadow = true;

	for (int32 AttributeIdx = 0; AttributeIdx < AttributeNum; AttributeIdx++)
	{
		UDrawDebugLibrary::DrawDebugCanvasStringView(
			CanvasDrawer,
			FVector2D(VerticalSpacing, Offset + AttributeIdx * VerticalSpacing),
			FString::Printf(TEXT("%s (%s)"),
				*UAnimDatabasePoseStateLibrary::PoseStateAttributeName(InPoseState, AttributeIdx).ToString(),
				UAnimDatabaseFrameAttributeLibrary::AttributeTypeNameInternal(UAnimDatabasePoseStateLibrary::PoseStateAttributeType(InPoseState, AttributeIdx))),
			IdentifierColor,
			Settings);
	}

	for (int32 AttributeIdx = 0; AttributeIdx < AttributeNum; AttributeIdx++)
	{
		UDrawDebugLibrary::DrawDebugCanvasStringView(
			CanvasDrawer,
			FVector2D(VerticalSpacing + PropertyHorizontalOffset, Offset + AttributeIdx * VerticalSpacing),
			UAnimDatabasePoseStateLibrary::PoseStateIsAttributeActive(InPoseState, AttributeIdx) ?
			*UAnimDatabasePoseStateLibrary::PoseStateAttributeString(InPoseState, AttributeIdx) : TEXT("Inactive"),
			IdentifierColor,
			Settings);
	}
}

void UAnimDatabaseDebugDraw_ContactCurves::InitializeDrawDebug_Implementation(const UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges)
{
	LeftContactCurve = UAnimDatabaseFrameAttributeLibrary::MakeFloatFrameAttributeFromCurveWhenActive(InDatabase, InFrameRanges, LeftContactCurveName);
	RightContactCurve = UAnimDatabaseFrameAttributeLibrary::MakeFloatFrameAttributeFromCurveWhenActive(InDatabase, InFrameRanges, RightContactCurveName);
}

void UAnimDatabaseDebugDraw_ContactCurves::DrawDebug_Implementation(
	const FDebugDrawer& Drawer,
	const FDebugDrawer& CanvasDrawer,
	const FAnimDatabasePoseState& InPoseState,
	const UAnimDatabase* InDatabase,
	const FAnimDatabaseFrameRanges& InFrameRanges,
	const FTransform& RangeViewportTransform,
	const int32 CharacterIdx,
	const int32 SequenceIdx,
	const float SequenceTime,
	const int32 RangeStart,
	const int32 RangeLength,
	const FLinearColor& IdentifierColor)
{
	const int32 LeftToeBoneIndex = InDatabase->FindBoneIndex(LeftToeBoneName);

	float LeftBoneContactCurveValue = 0.0f;

	if (LeftToeBoneIndex != INDEX_NONE && UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleFloat(LeftBoneContactCurveValue, LeftContactCurve, SequenceIdx, SequenceTime, InDatabase->GetFrameRate()))
	{
		UDrawDebugLibrary::DrawDebugCircle(
			Drawer,
			UAnimDatabasePoseStateLibrary::PoseStateBoneWorldLocation(InPoseState, LeftToeBoneIndex),
			FRotator::ZeroRotator,
			UDrawDebugLibrary::MakeDrawDebugLineStyleFromColor(IdentifierColor),
			true,
			ContactCircleRadius * LeftBoneContactCurveValue);

		UDrawDebugLibrary::DrawDebugStringView(
			Drawer,
			FString::Printf(TEXT("Vel: %5.1f"), UAnimDatabasePoseStateLibrary::PoseStateBoneWorldLinearVelocity(InPoseState, LeftToeBoneIndex).Length()),
			UAnimDatabasePoseStateLibrary::PoseStateBoneWorldLocation(InPoseState, LeftToeBoneIndex) + FVector(20.0f, 0.0f, 20.0f), 
			FRotator(0.0f, -90.0f, 0.0f),
			UDrawDebugLibrary::MakeDrawDebugLineStyleFromColor(IdentifierColor));
	}

	const int32 RightToeBoneIndex = InDatabase->FindBoneIndex(RightToeBoneName);

	float RightBoneContactCurveValue = 0.0f;

	if (RightToeBoneIndex != INDEX_NONE && UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleFloat(RightBoneContactCurveValue, RightContactCurve, SequenceIdx, SequenceTime, InDatabase->GetFrameRate()))
	{
		UDrawDebugLibrary::DrawDebugCircle(
			Drawer,
			UAnimDatabasePoseStateLibrary::PoseStateBoneWorldLocation(InPoseState, RightToeBoneIndex),
			FRotator::ZeroRotator,
			UDrawDebugLibrary::MakeDrawDebugLineStyleFromColor(IdentifierColor),
			true,
			ContactCircleRadius * RightBoneContactCurveValue);

		UDrawDebugLibrary::DrawDebugStringView(
			Drawer,
			FString::Printf(TEXT("Vel: %5.1f"), UAnimDatabasePoseStateLibrary::PoseStateBoneWorldLinearVelocity(InPoseState, RightToeBoneIndex).Length()),
			UAnimDatabasePoseStateLibrary::PoseStateBoneWorldLocation(InPoseState, RightToeBoneIndex) + FVector(20.0f, 0.0f, 20.0f), 
			FRotator(0.0f, -90.0f, 0.0f),
			UDrawDebugLibrary::MakeDrawDebugLineStyleFromColor(IdentifierColor));
	}
}

void UAnimDatabaseDebugDraw_ContactThresholds::InitializeDrawDebug_Implementation(const UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges)
{
	const int32 LeftContactIndex = InDatabase->FindBoneIndex(LeftToeBoneName);
	const int32 RightContactIndex = InDatabase->FindBoneIndex(RightToeBoneName);

	if (LeftContactIndex == INDEX_NONE || RightContactIndex == INDEX_NONE) { return; }

	TArray<FAnimDatabaseFrameAttribute, TInlineAllocator<2>> Locations, Velocities;
	Locations.SetNum(2); Velocities.SetNum(2);

	UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalFrameAttributesFromArrayViews(
		Locations, {}, {},
		Velocities, {}, {},
		InDatabase, InFrameRanges,
		{ LeftContactIndex, RightContactIndex }, {}, {},
		{ LeftContactIndex, RightContactIndex }, {}, {});

	LeftContactVelocity = UAnimDatabaseFrameAttributeLibrary::FrameAttributeLength(Velocities[0]);
	RightContactVelocity = UAnimDatabaseFrameAttributeLibrary::FrameAttributeLength(Velocities[1]);
	LeftContactHeight = UAnimDatabaseFrameAttributeLibrary::FrameAttributeZ(Locations[0]);
	RightContactHeight = UAnimDatabaseFrameAttributeLibrary::FrameAttributeZ(Locations[1]);
}

void UAnimDatabaseDebugDraw_ContactThresholds::DrawDebug_Implementation(
	const FDebugDrawer& Drawer,
	const FDebugDrawer& CanvasDrawer,
	const FAnimDatabasePoseState& InPoseState,
	const UAnimDatabase* InDatabase,
	const FAnimDatabaseFrameRanges& InFrameRanges,
	const FTransform& RangeViewportTransform,
	const int32 CharacterIdx,
	const int32 SequenceIdx,
	const float SequenceTime,
	const int32 RangeStart,
	const int32 RangeLength,
	const FLinearColor& IdentifierColor)
{
	const int32 LeftContactIndex = InDatabase->FindBoneIndex(LeftToeBoneName);
	const int32 RightContactIndex = InDatabase->FindBoneIndex(RightToeBoneName);

	if (LeftContactIndex == INDEX_NONE || RightContactIndex == INDEX_NONE) { return; }

	if (!LeftContactVelocity.IsValid() || 
		!RightContactVelocity.IsValid() || 
		!LeftContactHeight.IsValid() || 
		!RightContactHeight.IsValid()) { return; }

	const FDrawDebugLineStyle GraphTextStyle = UDrawDebugLibrary::MakeDrawDebugLineStyleFromColor(IdentifierColor);
	const FDrawDebugLineStyle GraphAxisStyle = UDrawDebugLibrary::MakeDrawDebugLineStyleFromColor(IdentifierColor);
	const FDrawDebugLineStyle GraphPlotStyle = UDrawDebugLibrary::MakeDrawDebugLineStyleFromColor(IdentifierColor);
	
	FDrawDebugCanvasTextSettings CanvasTextSettings;
	CanvasTextSettings.bShadow = true;
	CanvasTextSettings.bCenterVertically = true;
	CanvasTextSettings.bCenterHorizontally = true;

	FDrawDebugLineStyle ThresholdLineStyle;
	ThresholdLineStyle.Color = FLinearColor::Red;
	ThresholdLineStyle.LineType = EDrawDebugLineType::Dashed;
	ThresholdLineStyle.DashSpacing = 2.0f;

	FDrawDebugLineStyle VerticalLineStyle;
	VerticalLineStyle.Color = IdentifierColor;
	VerticalLineStyle.LineType = EDrawDebugLineType::Dotted;
	VerticalLineStyle.DotSpacing = 4.0f;

	const float StartTime = SequenceTime - (0.5f * GraphFrameNum) / InDatabase->GetFrameRate().AsDecimal();
	const float EndTime = SequenceTime + (0.5f * GraphFrameNum) / InDatabase->GetFrameRate().AsDecimal();

	TimeValues.SetNumUninitialized(GraphFrameNum);
	LeftVelocityValues.SetNumUninitialized(GraphFrameNum);
	LeftHeightValues.SetNumUninitialized(GraphFrameNum);
	RightVelocityValues.SetNumUninitialized(GraphFrameNum);
	RightHeightValues.SetNumUninitialized(GraphFrameNum);
	UDrawDebugLibrary::MakeLinearlySpacedFloatArrayView(TimeValues);
	UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleFloatRangeToArrayView(LeftVelocityValues, LeftContactVelocity, SequenceIdx, StartTime, EndTime, GraphFrameNum, InDatabase->GetFrameRate());
	UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleFloatRangeToArrayView(LeftHeightValues, LeftContactHeight, SequenceIdx, StartTime, EndTime, GraphFrameNum, InDatabase->GetFrameRate());
	UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleFloatRangeToArrayView(RightVelocityValues, RightContactVelocity, SequenceIdx, StartTime, EndTime, GraphFrameNum, InDatabase->GetFrameRate());
	UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleFloatRangeToArrayView(RightHeightValues, RightContactHeight, SequenceIdx, StartTime, EndTime, GraphFrameNum, InDatabase->GetFrameRate());

	for (int32 Idx = 0; Idx < GraphFrameNum; Idx++)
	{
		LeftVelocityValues[Idx] = FMath::Min(LeftVelocityValues[Idx], GraphMaxVelocity);
		RightVelocityValues[Idx] = FMath::Min(RightVelocityValues[Idx], GraphMaxVelocity);
		LeftHeightValues[Idx] = FMath::Min(LeftHeightValues[Idx], GraphMaxHeight);
		RightHeightValues[Idx] = FMath::Min(RightHeightValues[Idx], GraphMaxHeight);
	}

	float CurrentLeftVelocity, CurrentRightVelocity, CurrentLeftHeight, CurrentRightHeight;
	UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleFloat(CurrentLeftVelocity, LeftContactVelocity, SequenceIdx, SequenceTime, InDatabase->GetFrameRate());
	UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleFloat(CurrentRightVelocity, RightContactVelocity, SequenceIdx, SequenceTime, InDatabase->GetFrameRate());
	UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleFloat(CurrentLeftHeight, LeftContactHeight, SequenceIdx, SequenceTime, InDatabase->GetFrameRate());
	UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleFloat(CurrentRightHeight, RightContactHeight, SequenceIdx, SequenceTime, InDatabase->GetFrameRate());

	const FVector2D ScreenOffset = FVector2D(30.0f, 60.0f) + FVector2D(0.0f, CharacterIdx * (GraphHeight + 50.0f));


	FVector CanvasLocation; FRotator CanvasRotation;
	FDrawDebugGraphAxesSettings AxisSettings;
	AxisSettings.bDrawAxesBox = true;

	UDrawDebugLibrary::DrawDebugCanvasStringView(
		CanvasDrawer,
		ScreenOffset + FVector2D(GraphWidth / 2.0f, -20.0f),
		FString::Printf(TEXT("%s velocity: %3.2f cm/s"), *LeftToeBoneName.ToString(), CurrentLeftVelocity),
		CurrentLeftVelocity < VelocityThreshold ? IdentifierColor : FLinearColor::Red,
		CanvasTextSettings);

	UDrawDebugLibrary::DrawDebugOrientUprightToCanvas(CanvasLocation, CanvasRotation, ScreenOffset + FVector2D(0.0f, GraphHeight));
	UDrawDebugLibrary::DrawDebugGraphArrayView(
		CanvasDrawer,
		CanvasLocation,
		CanvasRotation,
		TimeValues,
		LeftVelocityValues,
		0.0f, 1.0f,
		0.0f, GraphMaxVelocity,
		GraphWidth, GraphHeight,
		GraphTextStyle,
		GraphAxisStyle,
		GraphPlotStyle,
		false,
		AxisSettings);

	UDrawDebugLibrary::DrawDebugLine(CanvasDrawer,
		FVector(ScreenOffset.X + GraphWidth / 2.0f, ScreenOffset.Y + GraphHeight, 0.0f),
		FVector(ScreenOffset.X + GraphWidth / 2.0f, ScreenOffset.Y + GraphHeight - GraphHeight, 0.0f),
		VerticalLineStyle,
		false);

	UDrawDebugLibrary::DrawDebugLine(CanvasDrawer,
		FVector(ScreenOffset.X, ScreenOffset.Y + GraphHeight - GraphHeight * FMath::Min(VelocityThreshold / FMath::Max(GraphMaxVelocity, UE_SMALL_NUMBER), 1.0f), 0.0f),
		FVector(ScreenOffset.X + GraphWidth, ScreenOffset.Y + GraphHeight - GraphHeight * FMath::Min(VelocityThreshold / FMath::Max(GraphMaxVelocity, UE_SMALL_NUMBER), 1.0f), 0.0f),
		ThresholdLineStyle,
		false);

	UDrawDebugLibrary::DrawDebugCanvasStringView(
		CanvasDrawer,
		ScreenOffset + FVector2D(GraphWidth / 2.0f + GraphWidth + 30.0f, -20.0f),
		FString::Printf(TEXT("%s height: %3.2f cm"), *LeftToeBoneName.ToString(), CurrentLeftHeight),
		CurrentLeftHeight < HeightThreshold ? IdentifierColor : FLinearColor::Red,
		CanvasTextSettings);

	UDrawDebugLibrary::DrawDebugOrientUprightToCanvas(CanvasLocation, CanvasRotation, ScreenOffset + FVector2D(GraphWidth + 30.0f, GraphHeight));
	UDrawDebugLibrary::DrawDebugGraphArrayView(
		CanvasDrawer,
		CanvasLocation,
		CanvasRotation,
		TimeValues,
		LeftHeightValues,
		0.0f, 1.0f,
		0.0f, GraphMaxHeight,
		GraphWidth, GraphHeight,
		GraphTextStyle,
		GraphAxisStyle,
		GraphPlotStyle,
		false,
		AxisSettings);

	UDrawDebugLibrary::DrawDebugLine(CanvasDrawer,
		FVector(ScreenOffset.X + GraphWidth + 30.0f + GraphWidth / 2.0f, ScreenOffset.Y + GraphHeight, 0.0f),
		FVector(ScreenOffset.X + GraphWidth + 30.0f + GraphWidth / 2.0f, ScreenOffset.Y + GraphHeight - GraphHeight, 0.0f),
		VerticalLineStyle,
		false);

	UDrawDebugLibrary::DrawDebugLine(CanvasDrawer,
		FVector(ScreenOffset.X + GraphWidth + 30.0f, ScreenOffset.Y + GraphHeight - GraphHeight * FMath::Min(HeightThreshold / FMath::Max(GraphMaxHeight, UE_SMALL_NUMBER), 1.0f), 0.0f),
		FVector(ScreenOffset.X + GraphWidth + 30.0f + GraphWidth, ScreenOffset.Y + GraphHeight - GraphHeight * FMath::Min(HeightThreshold / FMath::Max(GraphMaxHeight, UE_SMALL_NUMBER), 1.0f), 0.0f),
		ThresholdLineStyle,
		false);

	UDrawDebugLibrary::DrawDebugCanvasStringView(
		CanvasDrawer,
		ScreenOffset + FVector2D(GraphWidth / 2.0f + 2.0f * (GraphWidth + 30.0f), -20.0f),
		FString::Printf(TEXT("%s velocity: %3.2f cm/s"), *RightToeBoneName.ToString(), CurrentRightVelocity),
		CurrentRightVelocity < VelocityThreshold ? IdentifierColor : FLinearColor::Red,
		CanvasTextSettings);

	UDrawDebugLibrary::DrawDebugOrientUprightToCanvas(CanvasLocation, CanvasRotation, ScreenOffset + FVector2D(2.0f * (GraphWidth + 30.0f), GraphHeight));
	UDrawDebugLibrary::DrawDebugGraphArrayView(
		CanvasDrawer,
		CanvasLocation,
		CanvasRotation,
		TimeValues,
		RightVelocityValues,
		0.0f, 1.0f,
		0.0f, GraphMaxVelocity,
		GraphWidth, GraphHeight,
		GraphTextStyle,
		GraphAxisStyle,
		GraphPlotStyle,
		false,
		AxisSettings);

	UDrawDebugLibrary::DrawDebugLine(CanvasDrawer,
		FVector(ScreenOffset.X + 2.0f * (GraphWidth + 30.0f) + GraphWidth / 2.0f, ScreenOffset.Y + GraphHeight, 0.0f),
		FVector(ScreenOffset.X + 2.0f * (GraphWidth + 30.0f) + GraphWidth / 2.0f, ScreenOffset.Y + GraphHeight - GraphHeight, 0.0f),
		VerticalLineStyle,
		false);

	UDrawDebugLibrary::DrawDebugLine(CanvasDrawer,
		FVector(ScreenOffset.X + 2.0f * (GraphWidth + 30.0f), ScreenOffset.Y + GraphHeight - GraphHeight * FMath::Min(VelocityThreshold / FMath::Max(GraphMaxVelocity, UE_SMALL_NUMBER), 1.0f), 0.0f),
		FVector(ScreenOffset.X + 2.0f * (GraphWidth + 30.0f) + GraphWidth, ScreenOffset.Y + GraphHeight - GraphHeight * FMath::Min(VelocityThreshold / FMath::Max(GraphMaxVelocity, UE_SMALL_NUMBER), 1.0f), 0.0f),
		ThresholdLineStyle,
		false);

	UDrawDebugLibrary::DrawDebugCanvasStringView(
		CanvasDrawer,
		ScreenOffset  + FVector2D(GraphWidth / 2.0f + 3.0f * (GraphWidth + 30.0f), -20.0f),
		FString::Printf(TEXT("%s height: %3.2f cm"), *RightToeBoneName.ToString(), CurrentRightHeight),
		CurrentRightHeight < HeightThreshold ? IdentifierColor : FLinearColor::Red,
		CanvasTextSettings);

	UDrawDebugLibrary::DrawDebugOrientUprightToCanvas(CanvasLocation, CanvasRotation, ScreenOffset + FVector2D(3.0f * (GraphWidth + 30.0f), GraphHeight));
	UDrawDebugLibrary::DrawDebugGraphArrayView(
		CanvasDrawer,
		CanvasLocation,
		CanvasRotation,
		TimeValues,
		RightHeightValues,
		0.0f, 1.0f,
		0.0f, GraphMaxHeight,
		GraphWidth, GraphHeight,
		GraphTextStyle,
		GraphAxisStyle,
		GraphPlotStyle,
		false,
		AxisSettings);

	UDrawDebugLibrary::DrawDebugLine(CanvasDrawer,
		FVector(ScreenOffset.X + 3.0f * (GraphWidth + 30.0f) + GraphWidth / 2.0f, ScreenOffset.Y + GraphHeight, 0.0f),
		FVector(ScreenOffset.X + 3.0f * (GraphWidth + 30.0f) + GraphWidth / 2.0f, ScreenOffset.Y + GraphHeight - GraphHeight, 0.0f),
		VerticalLineStyle,
		false);

	UDrawDebugLibrary::DrawDebugLine(CanvasDrawer,
		FVector(ScreenOffset.X + 3.0f * (GraphWidth + 30.0f), ScreenOffset.Y + GraphHeight - GraphHeight * FMath::Min(HeightThreshold / FMath::Max(GraphMaxHeight, UE_SMALL_NUMBER), 1.0f), 0.0f),
		FVector(ScreenOffset.X + 3.0f * (GraphWidth + 30.0f) + GraphWidth, ScreenOffset.Y + GraphHeight - GraphHeight * FMath::Min(HeightThreshold / FMath::Max(GraphMaxHeight, UE_SMALL_NUMBER), 1.0f), 0.0f),
		ThresholdLineStyle,
		false);

	const FVector LeftToeLocation = UAnimDatabasePoseStateLibrary::PoseStateBoneWorldLocation(InPoseState, LeftContactIndex);
	const FVector RightToeLocation = UAnimDatabasePoseStateLibrary::PoseStateBoneWorldLocation(InPoseState, RightContactIndex);
	const FVector LeftToeVelocity = UAnimDatabasePoseStateLibrary::PoseStateBoneWorldLinearVelocity(InPoseState, LeftContactIndex);
	const FVector RightToeVelocity = UAnimDatabasePoseStateLibrary::PoseStateBoneWorldLinearVelocity(InPoseState, RightContactIndex);

	UDrawDebugLibrary::DrawDebugVelocity(
		Drawer,
		LeftToeLocation,
		LeftToeVelocity,
		UDrawDebugLibrary::MakeDrawDebugLineStyleFromColorAndThickness(IdentifierColor, VelocityLineThickness),
		false,
		VelocityLineScale);

	UDrawDebugLibrary::DrawDebugVelocity(
		Drawer,
		RightToeLocation,
		RightToeVelocity,
		UDrawDebugLibrary::MakeDrawDebugLineStyleFromColorAndThickness(IdentifierColor, VelocityLineThickness),
		false,
		VelocityLineScale);

	FDrawDebugStringSettings WorldStringSettings;
	WorldStringSettings.bCenterHorizontally = true;
	WorldStringSettings.bCenterVertically = true;

	UDrawDebugLibrary::DrawDebugStringView(
		Drawer,
		FString::Printf(TEXT("velocity: %3.2f cm/s"), CurrentLeftVelocity),
		LeftToeLocation + FVector(0.0f, 0.0f, 20.0f),
		FRotator(0.0f, -90.0f, 0.0f),
		CurrentLeftVelocity < VelocityThreshold ?
		UDrawDebugLibrary::MakeDrawDebugLineStyleFromColor(IdentifierColor) :
		UDrawDebugLibrary::MakeDrawDebugLineStyleFromColor(FLinearColor::Red),
		false,
		WorldStringSettings);

	UDrawDebugLibrary::DrawDebugStringView(
		Drawer,
		FString::Printf(TEXT("height: %3.2f cm"), CurrentLeftHeight),
		LeftToeLocation + FVector(0.0f, 0.0f, 10.0f),
		FRotator(0.0f, -90.0f, 0.0f),
		CurrentLeftHeight < HeightThreshold ?
		UDrawDebugLibrary::MakeDrawDebugLineStyleFromColor(IdentifierColor) :
		UDrawDebugLibrary::MakeDrawDebugLineStyleFromColor(FLinearColor::Red),
		false,
		WorldStringSettings);

	UDrawDebugLibrary::DrawDebugStringView(
		Drawer,
		FString::Printf(TEXT("velocity: %3.2f cm/s"), CurrentRightVelocity),
		RightToeLocation + FVector(0.0f, 0.0f, 20.0f),
		FRotator(0.0f, -90.0f, 0.0f),
		CurrentRightVelocity < VelocityThreshold ?
		UDrawDebugLibrary::MakeDrawDebugLineStyleFromColor(IdentifierColor) :
		UDrawDebugLibrary::MakeDrawDebugLineStyleFromColor(FLinearColor::Red),
		false,
		WorldStringSettings);

	UDrawDebugLibrary::DrawDebugStringView(
		Drawer,
		FString::Printf(TEXT("height: %3.2f cm"), CurrentRightHeight),
		RightToeLocation + FVector(0.0f, 0.0f, 10.0f),
		FRotator(0.0f, -90.0f, 0.0f),
		CurrentRightHeight < HeightThreshold ?
		UDrawDebugLibrary::MakeDrawDebugLineStyleFromColor(IdentifierColor) :
		UDrawDebugLibrary::MakeDrawDebugLineStyleFromColor(FLinearColor::Red),
		false,
		WorldStringSettings);

	if (CurrentLeftVelocity < VelocityThreshold && CurrentLeftHeight < HeightThreshold)
	{
		const FVector LeftFloorLocation = FVector(LeftToeLocation.X, LeftToeLocation.Y, ContactCircleThickness);

		UDrawDebugLibrary::DrawDebugCircle(
			Drawer,
			LeftFloorLocation,
			FRotator::ZeroRotator,
			UDrawDebugLibrary::MakeDrawDebugLineStyleFromColorAndThickness(IdentifierColor, ContactCircleThickness),
			true,
			ContactCircleRadius);
	}

	if (CurrentRightVelocity < VelocityThreshold && CurrentRightHeight < HeightThreshold)
	{
		const FVector RightFloorLocation = FVector(RightToeLocation.X, RightToeLocation.Y, ContactCircleThickness);

		UDrawDebugLibrary::DrawDebugCircle(
			Drawer,
			RightFloorLocation,
			FRotator::ZeroRotator,
			UDrawDebugLibrary::MakeDrawDebugLineStyleFromColorAndThickness(IdentifierColor, ContactCircleThickness),
			true,
			ContactCircleRadius);
	}
}

void UAnimDatabaseDebugDraw_Multi::InitializeDrawDebug_Implementation(const UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges)
{
	for (TObjectPtr<UAnimDatabaseDebugDraw>& DrawerObject : Drawers)
	{
		if (DrawerObject) { DrawerObject->InitializeDrawDebug(InDatabase, InFrameRanges); }
	}
}

void UAnimDatabaseDebugDraw_Multi::DrawDebug_Implementation(
	const FDebugDrawer& Drawer,
	const FDebugDrawer& CanvasDrawer,
	const FAnimDatabasePoseState& InPoseState,
	const UAnimDatabase* InDatabase,
	const FAnimDatabaseFrameRanges& InFrameRanges,
	const FTransform& RangeViewportTransform,
	const int32 CharacterIdx,
	const int32 SequenceIdx,
	const float SequenceTime,
	const int32 RangeStart,
	const int32 RangeLength,
	const FLinearColor& IdentifierColor)
{
	for (TObjectPtr<UAnimDatabaseDebugDraw>& DrawerObject : Drawers)
	{
		if (DrawerObject)
		{
			DrawerObject->DrawDebug(
				Drawer,
				CanvasDrawer,
				InPoseState,
				InDatabase, 
				InFrameRanges,
				RangeViewportTransform,
				CharacterIdx,
				SequenceIdx,
				SequenceTime,
				RangeStart,
				RangeLength,
				IdentifierColor);
		}
	}
}

void UAnimDatabaseDebugDraw_RootOrientation::DrawDebug_Implementation(
	const FDebugDrawer& Drawer,
	const FDebugDrawer& CanvasDrawer,
	const FAnimDatabasePoseState& InPoseState,
	const UAnimDatabase* InDatabase,
	const FAnimDatabaseFrameRanges& InFrameRanges,
	const FTransform& RangeViewportTransform,
	const int32 CharacterIdx,
	const int32 SequenceIdx,
	const float SequenceTime,
	const int32 RangeStart,
	const int32 RangeLength,
	const FLinearColor& IdentifierColor)
{
	UDrawDebugLibrary::DrawDebugMoverOrientation(
		Drawer,
		UAnimDatabasePoseStateLibrary::PoseStateRootLocation(InPoseState) + FVector(0.0, 0.0, FloorOffset),
		UAnimDatabasePoseStateLibrary::PoseStateRootRotation(InPoseState),
		UDrawDebugLibrary::MakeDrawDebugLineStyleFromColorAndThickness(IdentifierColor, 1.0f),
		true,
		ForwardVector,
		Radius);
}

void UAnimDatabaseDebugDraw_MovementDirection::DrawDebug_Implementation(
	const FDebugDrawer& Drawer,
	const FDebugDrawer& CanvasDrawer,
	const FAnimDatabasePoseState& InPoseState,
	const UAnimDatabase* InDatabase,
	const FAnimDatabaseFrameRanges& InFrameRanges,
	const FTransform& RangeViewportTransform,
	const int32 CharacterIdx,
	const int32 SequenceIdx,
	const float SequenceTime,
	const int32 RangeStart,
	const int32 RangeLength,
	const FLinearColor& IdentifierColor)
{
	const FVector RootLocation = UAnimDatabasePoseStateLibrary::PoseStateRootLocation(InPoseState) + FVector(0.0, 0.0, FloorOffset);
	const FVector RootDirection = UAnimDatabasePoseStateLibrary::PoseStateRootDirection(InPoseState, Direction);
	const FVector RootLinearVelocity = UAnimDatabasePoseStateLibrary::PoseStateRootLinearVelocity(InPoseState);

	UDrawDebugLibrary::DrawDebugLine(
		Drawer,
		RootLocation,
		RootLocation + VelocityScale * RootLinearVelocity,
		UDrawDebugLibrary::MakeDrawDebugLineStyleFromColorAndThickness(IdentifierColor, Thickness),
		true);

	UDrawDebugLibrary::DrawDebugArrow(
		Drawer,
		RootLocation,
		RootLocation + DirectionArrowLength * RootDirection,
		UDrawDebugLibrary::MakeDrawDebugLineStyleFromColorAndThickness(IdentifierColor, Thickness),
		true);

	if (RootLinearVelocity.SquaredLength() > FMath::Square(VelocityThreshold))
	{
		const float Angle = FMath::Acos(RootDirection.Dot(RootLinearVelocity.GetSafeNormal())) * (RootDirection.Cross(RootLinearVelocity).Z > 0.0f ? 1.0f : -1.0f);

		UDrawDebugLibrary::DrawDebugArc(
			Drawer,
			RootLocation,
			RootDirection.Rotation(),
			FMath::RadiansToDegrees(Angle),
			UDrawDebugLibrary::MakeDrawDebugLineStyleFromColorAndThickness(IdentifierColor, Thickness));

		FDrawDebugStringSettings StringSettings;
		StringSettings.bCenterHorizontally = true;
		StringSettings.bCenterVertically = true;

		UDrawDebugLibrary::DrawDebugStringView(
			Drawer,
			FString::Printf(TEXT("Angle: % 5.1f deg"), FMath::RadiansToDegrees(Angle)),
			RootLocation + DirectionArrowLength * RootDirection + FVector(0.0f, 0.0f, 20.0f),
			(-RootDirection).Rotation(),
			UDrawDebugLibrary::MakeDrawDebugLineStyleFromColorAndThickness(IdentifierColor),
			true,
			StringSettings);
	}
}

void UAnimDatabaseDebugDraw_BoneTransform::DrawDebug_Implementation(
	const FDebugDrawer& Drawer,
	const FDebugDrawer& CanvasDrawer,
	const FAnimDatabasePoseState& InPoseState,
	const UAnimDatabase* InDatabase,
	const FAnimDatabaseFrameRanges& InFrameRanges,
	const FTransform& RangeViewportTransform,
	const int32 CharacterIdx,
	const int32 SequenceIdx,
	const float SequenceTime,
	const int32 RangeStart,
	const int32 RangeLength,
	const FLinearColor& IdentifierColor)
{
	const int32 BoneIndex = InDatabase->FindBoneIndex(BoneName);
	if (BoneIndex == INDEX_NONE) { return; }

	UDrawDebugLibrary::DrawDebugTransform(
		Drawer,
		UAnimDatabasePoseStateLibrary::PoseStateBoneWorldTransform(InPoseState, BoneIndex),
		UDrawDebugLibrary::MakeDrawDebugLineStyleFromColorAndThickness(IdentifierColor, Thickness),
		bDepthTest,
		Radius);
}


void UAnimDatabaseDebugDraw_BoneAttachment::DrawDebug_Implementation(
	const FDebugDrawer& Drawer,
	const FDebugDrawer& CanvasDrawer,
	const FAnimDatabasePoseState& InPoseState,
	const UAnimDatabase* InDatabase,
	const FAnimDatabaseFrameRanges& InFrameRanges,
	const FTransform& RangeViewportTransform,
	const int32 CharacterIdx,
	const int32 SequenceIdx,
	const float SequenceTime,
	const int32 RangeStart,
	const int32 RangeLength,
	const FLinearColor& IdentifierColor)
{
	const int32 BoneIndex = InDatabase->FindBoneIndex(BoneName);
	if (BoneIndex == INDEX_NONE) { return; }

	const FTransform BoneTransform = UAnimDatabasePoseStateLibrary::PoseStateBoneWorldTransform(InPoseState, BoneIndex);
	const FTransform AttachmentTransform = RelativeTransform * BoneTransform;

	if (bDrawTransform)
	{
		UDrawDebugLibrary::DrawDebugTransform(
			Drawer,
			AttachmentTransform,
			UDrawDebugLibrary::MakeDrawDebugLineStyleFromColorAndThickness(IdentifierColor, TransformThickness),
			bDepthTest,
			TransformRadius);
	}

	if (StaticMesh)
	{
		UDrawDebugLibrary::DrawDebugStaticMeshBoundingBox(
			Drawer, 
			StaticMesh, 
			AttachmentTransform.GetLocation(), 
			AttachmentTransform.GetRotation().Rotator(),
			UDrawDebugLibrary::MakeDrawDebugLineStyleFromColorAndThickness(IdentifierColor, BoundingBoxThickness),
			bDepthTest);
	}
}

void UAnimDatabaseDebugDraw_RootVelocity::InitializeDrawDebug_Implementation(const UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges)
{
	RootLinearVelocity = UAnimDatabaseFrameAttributeLibrary::FrameAttributeLength(UAnimDatabaseFrameAttributeLibrary::MakeRootLinearVelocityFrameAttribute(InDatabase, InFrameRanges));
	RootAngularVelocity = UAnimDatabaseFrameAttributeLibrary::FrameAttributeLength(UAnimDatabaseFrameAttributeLibrary::MakeRootAngularVelocityFrameAttribute(InDatabase, InFrameRanges));
}

void UAnimDatabaseDebugDraw_RootVelocity::DrawDebug_Implementation(
	const FDebugDrawer& Drawer,
	const FDebugDrawer& CanvasDrawer,
	const FAnimDatabasePoseState& InPoseState,
	const UAnimDatabase* InDatabase,
	const FAnimDatabaseFrameRanges& InFrameRanges,
	const FTransform& RangeViewportTransform,
	const int32 CharacterIdx,
	const int32 SequenceIdx,
	const float SequenceTime,
	const int32 RangeStart,
	const int32 RangeLength,
	const FLinearColor& IdentifierColor)
{
	const int32 SampleNum = 60;

	TArray<float, TInlineAllocator<60>> XValues;
	TArray<float, TInlineAllocator<60>> YValuesLinearVelocity;
	TArray<float, TInlineAllocator<60>> YValuesAngularVelocity;
	XValues.SetNumUninitialized(SampleNum);
	YValuesLinearVelocity.SetNumUninitialized(SampleNum);
	YValuesAngularVelocity.SetNumUninitialized(SampleNum);

	UDrawDebugLibrary::MakeLinearlySpacedFloatArrayView(XValues);

	UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleFloatRangeToArrayView(
		YValuesLinearVelocity, 
		RootLinearVelocity, 
		SequenceIdx, 
		SequenceTime - SampleNum / FMath::Max(InDatabase->GetFrameRate().AsDecimal(), UE_SMALL_NUMBER),
		SequenceTime, 
		SampleNum, 
		InDatabase->GetFrameRate());

	UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleFloatRangeToArrayView(
		YValuesAngularVelocity,
		RootAngularVelocity,
		SequenceIdx,
		SequenceTime - SampleNum / FMath::Max(InDatabase->GetFrameRate().AsDecimal(), UE_SMALL_NUMBER),
		SequenceTime,
		SampleNum,
		InDatabase->GetFrameRate());

	const FDrawDebugLineStyle TextLineStyle = UDrawDebugLibrary::MakeDrawDebugLineStyleFromColor(IdentifierColor);
	const FDrawDebugLineStyle GraphLineStyle = UDrawDebugLibrary::MakeDrawDebugLineStyleFromColorAndThickness(IdentifierColor, 1.0f);

	FDrawDebugGraphAxesSettings LinearVelocityAxisSettings;
	LinearVelocityAxisSettings.Title = FString::Printf(TEXT("Linear Velocity: %5.1f cm/s"), UAnimDatabasePoseStateLibrary::PoseStateRootLinearVelocity(InPoseState).Length());
	LinearVelocityAxisSettings.YaxisLabel = TEXT("cm/s");

	UDrawDebugLibrary::DrawDebugGraphArrayView(
		Drawer,
		UAnimDatabasePoseStateLibrary::PoseStateRootLocation(InPoseState) + FVector(50.0f, 0.0f, 100.0f),
		FRotator(0.0f, -90.0f, 0.0f),
		XValues,
		YValuesLinearVelocity,
		0.0f, 1.0f,
		0.0f, MaxLinearVelocity,
		100.0f, 100.0f,
		TextLineStyle,
		GraphLineStyle,
		GraphLineStyle,
		true,
		LinearVelocityAxisSettings);

	FDrawDebugGraphAxesSettings AngularVelocityAxisSettings;
	AngularVelocityAxisSettings.Title = FString::Printf(TEXT("Angular Velocity: %5.1f deg/s"), FMath::RadiansToDegrees(UAnimDatabasePoseStateLibrary::PoseStateRootAngularVelocity(InPoseState).Length()));
	AngularVelocityAxisSettings.YaxisLabel = TEXT("deg/s");

	UDrawDebugLibrary::DrawDebugGraphArrayView(
		Drawer,
		UAnimDatabasePoseStateLibrary::PoseStateRootLocation(InPoseState) + FVector(50.0f, 0.0f, 220.0f),
		FRotator(0.0f, -90.0f, 0.0f),
		XValues,
		YValuesAngularVelocity,
		0.0f, 1.0f,
		0.0f, FMath::DegreesToRadians(MaxAngularVelocity),
		100.0f, 100.0f,
		TextLineStyle,
		GraphLineStyle,
		GraphLineStyle,
		true,
		AngularVelocityAxisSettings);
}

void UAnimDatabaseDebugDraw_ChairOnAttach::InitializeDrawDebug_Implementation(const UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges)
{
	ValidAttachRanges = AttachRanges ? UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFunction(InDatabase, InFrameRanges, AttachRanges) : InFrameRanges;
}

void UAnimDatabaseDebugDraw_ChairOnAttach::DrawDebug_Implementation(
	const FDebugDrawer& Drawer,
	const FDebugDrawer& CanvasDrawer,
	const FAnimDatabasePoseState& InPoseState,
	const UAnimDatabase* InDatabase,
	const FAnimDatabaseFrameRanges& InFrameRanges,
	const FTransform& RangeViewportTransform,
	const int32 CharacterIdx,
	const int32 SequenceIdx,
	const float SequenceTime,
	const int32 RangeStart,
	const int32 RangeLength,
	const FLinearColor& IdentifierColor)
{
	const int32 AttachBoneIndex = InDatabase->FindBoneIndex(AttachBoneName);

	if (AttachBoneIndex != INDEX_NONE && UAnimDatabaseFrameRangesLibrary::FrameRangesContainsTime(ValidAttachRanges, SequenceIdx, SequenceTime, InDatabase->GetFrameRate()))
	{
		UDrawDebugLibrary::DrawDebugChair(
			Drawer,
			UAnimDatabasePoseStateLibrary::PoseStateBoneWorldLocation(InPoseState, AttachBoneIndex),
			UAnimDatabasePoseStateLibrary::PoseStateBoneWorldRotation(InPoseState, AttachBoneIndex),
			UDrawDebugLibrary::MakeDrawDebugLineStyleFromColorAndThickness(IdentifierColor, 2.0f));
	}
}

#endif

#undef LOCTEXT_NAMESPACE