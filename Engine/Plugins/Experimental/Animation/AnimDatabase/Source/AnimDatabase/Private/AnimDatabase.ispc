// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef UE_MAX_FLT
#define UE_MAX_FLT			(3.402823466e+38F)
#endif

#ifndef INDEX_NONE
#define INDEX_NONE		(-1)
#endif

#ifndef UE_SMALL_NUMBER
#define UE_SMALL_NUMBER (1.e-8f)
#endif

#ifndef FLOAT_PI
#define FLOAT_PI		(3.1415926535897932384626433832795f)
#endif

static inline uniform float Square(uniform const float X)
{
	return X * X;
}

static inline float Square(const float X)
{
	return X * X;
}

static inline uniform float Lerp(uniform const float X, uniform const float Y, uniform const float A)
{
	return X * (1.0f - A) + Y * A;
}

static inline float Lerp(const float X, const float Y, uniform const float A)
{
	return X * (1.0f - A) + Y * A;
}

static inline float Lerp(const float X, const float Y, const float A)
{
	return X * (1.0f - A) + Y * A;
}

static inline uniform float InvExpApprox(uniform float X)
{
	uniform const float A = 1.00746054f; // 1 / 1! in Taylor series
	uniform const float B = 0.45053901f; // 1 / 2! in Taylor series
	uniform const float C = 0.25724632f; // 1 / 3! in Taylor series
	return 1 / (1 + A * X + B * X * X + C * X * X * X);
}

static inline float InvExpApprox(float X)
{
	uniform const float A = 1.00746054f; // 1 / 1! in Taylor series
	uniform const float B = 0.45053901f; // 1 / 2! in Taylor series
	uniform const float C = 0.25724632f; // 1 / 3! in Taylor series
	return 1 / (1 + A * X + B * X * X + C * X * X * X);
}

typedef float<3> Vector3f;
typedef float<4> Vector4f;
typedef float<4> Quat4f;
typedef float<16> Matrix44f;

static inline Vector3f Vector3fLoad(uniform const float Data[], const int Index)
{
	const Vector3f X = {
#pragma ignore warning(perf)
		Data[Index * 3 + 0],
#pragma ignore warning(perf)
		Data[Index * 3 + 1],
#pragma ignore warning(perf)
		Data[Index * 3 + 2],
	};

	return X;
}

static inline Vector4f Vector4fLoad(uniform const float Data[], const int Index)
{
	const Vector4f X = {
#pragma ignore warning(perf)
		Data[Index * 4 + 0],
#pragma ignore warning(perf)
		Data[Index * 4 + 1],
#pragma ignore warning(perf)
		Data[Index * 4 + 2],
#pragma ignore warning(perf)
		Data[Index * 4 + 3],
	};

	return X;
}

static inline uniform Quat4f Quat4fLoad(uniform const float Data[], uniform const int Index)
{
	uniform const Quat4f X = {
		Data[Index * 4 + 0],
		Data[Index * 4 + 1],
		Data[Index * 4 + 2],
		Data[Index * 4 + 3],
	};

	return X;
}

static inline Quat4f Quat4fLoad(uniform const float Data[], const int Index)
{
	const Quat4f X = {
#pragma ignore warning(perf)
		Data[Index * 4 + 0],
#pragma ignore warning(perf)
		Data[Index * 4 + 1],
#pragma ignore warning(perf)
		Data[Index * 4 + 2],
#pragma ignore warning(perf)
		Data[Index * 4 + 3],
	};

	return X;
}

static inline Matrix44f Matrix44fLoad(uniform const float Data[], const int Index)
{
	const Matrix44f X = {
#pragma ignore warning(perf)
		Data[Index * 16 +  0],
#pragma ignore warning(perf)
		Data[Index * 16 +  1],
#pragma ignore warning(perf)
		Data[Index * 16 +  2],
#pragma ignore warning(perf)
		Data[Index * 16 +  3],
#pragma ignore warning(perf)
		Data[Index * 16 +  4],
#pragma ignore warning(perf)
		Data[Index * 16 +  5],
#pragma ignore warning(perf)
		Data[Index * 16 +  6],
#pragma ignore warning(perf)
		Data[Index * 16 +  7],
#pragma ignore warning(perf)
		Data[Index * 16 +  8],
#pragma ignore warning(perf)
		Data[Index * 16 +  9],
#pragma ignore warning(perf)
		Data[Index * 16 + 10],
#pragma ignore warning(perf)
		Data[Index * 16 + 11],
#pragma ignore warning(perf)
		Data[Index * 16 + 12],
#pragma ignore warning(perf)
		Data[Index * 16 + 13],
#pragma ignore warning(perf)
		Data[Index * 16 + 14],
#pragma ignore warning(perf)
		Data[Index * 16 + 15],
	};

	return X;
}

static inline void Vector3fSave(uniform float Data[], uniform const int Index, uniform const Vector3f X)
{
	Data[Index * 3 + 0] = X[0];
	Data[Index * 3 + 1] = X[1];
	Data[Index * 3 + 2] = X[2];
}

static inline void Vector3fSave(uniform float Data[], const int Index, const Vector3f X)
{
#pragma ignore warning(perf)
	Data[Index * 3 + 0] = X[0];
#pragma ignore warning(perf)
	Data[Index * 3 + 1] = X[1];
#pragma ignore warning(perf)
	Data[Index * 3 + 2] = X[2];
}

static inline void Vector4fSave(uniform float Data[], const int Index, const Vector4f X)
{
#pragma ignore warning(perf)
	Data[Index * 4 + 0] = X[0];
#pragma ignore warning(perf)
	Data[Index * 4 + 1] = X[1];
#pragma ignore warning(perf)
	Data[Index * 4 + 2] = X[2];
#pragma ignore warning(perf)
	Data[Index * 4 + 3] = X[3];
}

static inline void Quat4fSave(uniform float Data[], uniform const int Index, uniform const Quat4f X)
{
	Data[Index * 4 + 0] = X[0];
	Data[Index * 4 + 1] = X[1];
	Data[Index * 4 + 2] = X[2];
	Data[Index * 4 + 3] = X[3];
}

static inline void Quat4fSave(uniform float Data[], const int Index, const Quat4f X)
{
#pragma ignore warning(perf)
	Data[Index * 4 + 0] = X[0];
#pragma ignore warning(perf)
	Data[Index * 4 + 1] = X[1];
#pragma ignore warning(perf)
	Data[Index * 4 + 2] = X[2];
#pragma ignore warning(perf)
	Data[Index * 4 + 3] = X[3];
}

static inline void Matrix44fSave(uniform float Data[], const int Index, const Matrix44f X)
{
#pragma ignore warning(perf)
	Data[Index * 16 +  0] = X[0];
#pragma ignore warning(perf)
	Data[Index * 16 +  1] = X[1];
#pragma ignore warning(perf)
	Data[Index * 16 +  2] = X[2];
#pragma ignore warning(perf)
	Data[Index * 16 +  3] = X[3];
#pragma ignore warning(perf)
	Data[Index * 16 +  4] = X[4];
#pragma ignore warning(perf)
	Data[Index * 16 +  5] = X[5];
#pragma ignore warning(perf)
	Data[Index * 16 +  6] = X[6];
#pragma ignore warning(perf)
	Data[Index * 16 +  7] = X[7];
#pragma ignore warning(perf)
	Data[Index * 16 +  8] = X[8];
#pragma ignore warning(perf)
	Data[Index * 16 +  9] = X[9];
#pragma ignore warning(perf)
	Data[Index * 16 + 10] = X[10];
#pragma ignore warning(perf)
	Data[Index * 16 + 11] = X[11];
#pragma ignore warning(perf)
	Data[Index * 16 + 12] = X[12];
#pragma ignore warning(perf)
	Data[Index * 16 + 13] = X[13];
#pragma ignore warning(perf)
	Data[Index * 16 + 14] = X[14];
#pragma ignore warning(perf)
	Data[Index * 16 + 15] = X[15];
}

static inline uniform float Vector3fDot(uniform const Vector3f V, uniform const Vector3f W)
{
	return V.x * W.x + V.y * W.y + V.z * W.z;
}

static inline float Vector3fDot(const Vector3f V, const Vector3f W)
{
	return V.x * W.x + V.y * W.y + V.z * W.z;
}

static inline uniform Vector3f Vector3fCross(uniform const Vector3f V, uniform const Vector3f W)
{
	uniform Vector3f O = {
        V.y * W.z - V.z * W.y,
        V.z * W.x - V.x * W.z,
        V.x * W.y - V.y * W.x
	};

	return O;
}

static inline Vector3f Vector3fCross(const Vector3f V, const Vector3f W)
{
	Vector3f O = {
        V.y * W.z - V.z * W.y,
        V.z * W.x - V.x * W.z,
        V.x * W.y - V.y * W.x
	};

	return O;
}

static inline uniform float Vector3fLengthSquared(uniform const Vector3f V)
{
	return Vector3fDot(V, V);
}

static inline float Vector3fLengthSquared(const Vector3f V)
{
	return Vector3fDot(V, V);
}

static inline uniform float Vector3fLength(uniform const Vector3f V)
{
	return sqrt(Vector3fLengthSquared(V));
}

static inline float Vector3fLength(const Vector3f V)
{
	return sqrt(Vector3fLengthSquared(V));
}

static inline uniform Vector3f Vector3fNormalize(uniform const Vector3f V)
{
	return V / Vector3fLength(V);
}

static inline Vector3f Vector3fNormalize(const Vector3f V)
{
	return V / Vector3fLength(V);
}

static inline uniform Vector3f Vector3fNormalizeSafe(uniform const Vector3f V, uniform const float Eps, uniform const Vector3f Default)
{
	uniform float LengthSq = Vector3fLengthSquared(V);

	if (LengthSq < Eps)
	{
		return Default;
	}
	else
	{
		return V / sqrt(LengthSq);
	}
}

static inline Vector3f Vector3fNormalizeSafe(const Vector3f V, uniform const float Eps, uniform const Vector3f Default)
{
	float LengthSq = Vector3fLengthSquared(V);

	if (LengthSq < Eps)
	{
		return Default;
	}
	else
	{
		return V / sqrt(LengthSq);
	}
}

static inline uniform Vector3f Vector3fSqrt(uniform const Vector3f V)
{
	uniform const Vector3f O = {
		sqrt(V.x),
		sqrt(V.y),
		sqrt(V.z),
	};

	return O;
}

static inline Vector3f Vector3fSqrt(const Vector3f V)
{
	const Vector3f O = {
		sqrt(V.x),
		sqrt(V.y),
		sqrt(V.z),
	};

	return O;
}

static inline uniform Vector3f Vector3fSquare(uniform const Vector3f V)
{
	uniform const Vector3f O = {
		Square(V.x),
		Square(V.y),
		Square(V.z),
	};

	return O;
}

static inline Vector3f Vector3fSquare(const Vector3f V)
{
	const Vector3f O = {
		Square(V.x),
		Square(V.y),
		Square(V.z),
	};

	return O;
}


static inline Vector3f Vector3fExpSafe(const Vector3f V, uniform const float Max = 10.0f)
{
	const Vector3f O = {
		exp(min(V.x, Max)),
		exp(min(V.y, Max)),
		exp(min(V.z, Max)),
	};

	return O;
}

static inline Vector3f Vector3fEerp(const Vector3f V, const Vector3f W, uniform const float Alpha, uniform const float Epsilon = UE_SMALL_NUMBER)
{
	if (Vector3fLengthSquared(V - W) < Epsilon)
	{
		const Vector3f O = {
			Lerp(max(V.x, Epsilon), max(W.x, Epsilon), Alpha),
			Lerp(max(V.y, Epsilon), max(W.y, Epsilon), Alpha),
			Lerp(max(V.z, Epsilon), max(W.z, Epsilon), Alpha),
		};

		return O;
	}
	else
	{
		const Vector3f O = {
			pow(max(V.x, Epsilon), (1.0f - Alpha)) * pow(max(W.x, Epsilon), Alpha),
			pow(max(V.y, Epsilon), (1.0f - Alpha)) * pow(max(W.y, Epsilon), Alpha),
			pow(max(V.z, Epsilon), (1.0f - Alpha)) * pow(max(W.z, Epsilon), Alpha),
		};

		return O;
	}
}

static inline Vector3f Vector3fEerp(const Vector3f V, const Vector3f W, const float Alpha, uniform const float Epsilon = UE_SMALL_NUMBER)
{
	if (Vector3fLengthSquared(V - W) < Epsilon)
	{
		const Vector3f O = {
			Lerp(max(V.x, Epsilon), max(W.x, Epsilon), Alpha),
			Lerp(max(V.y, Epsilon), max(W.y, Epsilon), Alpha),
			Lerp(max(V.z, Epsilon), max(W.z, Epsilon), Alpha),
		};

		return O;
	}
	else
	{
		const Vector3f O = {
			pow(max(V.x, Epsilon), (1.0f - Alpha)) * pow(max(W.x, Epsilon), Alpha),
			pow(max(V.y, Epsilon), (1.0f - Alpha)) * pow(max(W.y, Epsilon), Alpha),
			pow(max(V.z, Epsilon), (1.0f - Alpha)) * pow(max(W.z, Epsilon), Alpha),
		};

		return O;
	}
}

static inline Vector3f Vector3fLerp(const Vector3f X, const Vector3f Y, uniform const float A)
{
	return X * (1.0f - A) + Y * A;
}

static inline Vector3f Vector3fLerp(const Vector3f X, const Vector3f Y, const float A)
{
	return X * (1.0f - A) + Y * A;
}

static inline uniform float Vector4fDot(uniform const Vector4f V, uniform const Vector4f W)
{
	return V.x * W.x + V.y * W.y + V.z * W.z + V.w * W.w;
}

static inline float Vector4fDot(const Vector4f V, const Vector4f W)
{
	return V.x * W.x + V.y * W.y + V.z * W.z + V.w * W.w;
}

static inline uniform float Vector4fLengthSquared(uniform const Vector4f V)
{
	return Vector4fDot(V, V);
}

static inline float Vector4fLengthSquared(const Vector4f V)
{
	return Vector4fDot(V, V);
}

static inline uniform float Vector4fLength(uniform const Vector4f V)
{
	return sqrt(Vector4fLengthSquared(V));
}

static inline float Vector4fLength(const Vector4f V)
{
	return sqrt(Vector4fLengthSquared(V));
}

static inline uniform Vector4f Vector4fNormalize(uniform const Vector4f V)
{
	return V / Vector4fLength(V);
}

static inline Vector4f Vector4fNormalize(const Vector4f V)
{
	return V / Vector4fLength(V);
}

static inline uniform Quat4f Quat4fAbs(uniform const Quat4f Q)
{
	uniform const Quat4f O = {
		select(Q.w > 0.0f, Q.x, -Q.x),
		select(Q.w > 0.0f, Q.y, -Q.y),
		select(Q.w > 0.0f, Q.z, -Q.z),
		select(Q.w > 0.0f, Q.w, -Q.w),
	};

	return O;
}

static inline Quat4f Quat4fAbs(const Quat4f Q)
{
	const Quat4f O = {
		select(Q.w > 0.0f, Q.x, -Q.x),
		select(Q.w > 0.0f, Q.y, -Q.y),
		select(Q.w > 0.0f, Q.z, -Q.z),
		select(Q.w > 0.0f, Q.w, -Q.w),
	};

	return O;
}

static inline uniform Quat4f Quat4fMul(uniform const Quat4f Q, uniform const Quat4f P)
{
	uniform const Quat4f O = {
		P.w * Q.x + P.x * Q.w - P.y * Q.z + P.z * Q.y,
		P.w * Q.y + P.x * Q.z + P.y * Q.w - P.z * Q.x,
		P.w * Q.z - P.x * Q.y + P.y * Q.x + P.z * Q.w,
		P.w * Q.w - P.x * Q.x - P.y * Q.y - P.z * Q.z,
	};

	return O;
}

static inline Quat4f Quat4fMul(const Quat4f Q, const Quat4f P)
{
	const Quat4f O = {
		P.w * Q.x + P.x * Q.w - P.y * Q.z + P.z * Q.y,
		P.w * Q.y + P.x * Q.z + P.y * Q.w - P.z * Q.x,
		P.w * Q.z - P.x * Q.y + P.y * Q.x + P.z * Q.w,
		P.w * Q.w - P.x * Q.x - P.y * Q.y - P.z * Q.z,
	};

	return O;
}

static inline uniform Quat4f Quat4fInv(uniform const Quat4f Q)
{
	uniform const Quat4f O = { -Q.x, -Q.y, -Q.z, Q.w };
	return O;
}

static inline Quat4f Quat4fInv(const Quat4f Q)
{
	const Quat4f O = { -Q.x, -Q.y, -Q.z, Q.w };
	return O;
}

static inline float Quat4fDot(const Quat4f Q, const Quat4f P)
{
	return Q.x * P.x + Q.y * P.y + Q.z * P.z + Q.w * P.w;
}

static inline float Quat4fLengthSquared(const Quat4f Q)
{
	return Quat4fDot(Q, Q);
}

static inline float Quat4fLength(const Quat4f Q)
{
	return sqrt(Quat4fLengthSquared(Q));
}

static inline Quat4f Quat4fNormalized(const Quat4f Q)
{
	return Q / Quat4fLength(Q);
}


static inline Quat4f Quat4fExp(const Vector3f V, const uniform float Epsilon = UE_SMALL_NUMBER)
{
	const float HalfAngleSq = Vector3fLengthSquared(V);

	cif (HalfAngleSq < Epsilon)
	{
		const Quat4f O = { V.x, V.y, V.z, 1.0f };
		return Quat4fNormalized(O);
	}
	else
	{
		const float HalfAngle = sqrt(HalfAngleSq);
		const float C = cos(HalfAngle);
		const float S = sin(HalfAngle) / HalfAngle;

		const Quat4f O = { S * V.x, S * V.y, S * V.z, C };
		return O;
	}
}

static inline uniform Vector3f Quat4fLog(uniform const Quat4f Q, uniform const float Epsilon = UE_SMALL_NUMBER)
{
	uniform const Vector3f V = { Q.x, Q.y, Q.z };
	uniform const float LengthSq = Vector3fLengthSquared(V);

	if(LengthSq < Epsilon)
	{
		return V;
	}
	else
	{
		uniform const float Length = sqrt(LengthSq);
		uniform const float HalfAngle = acos(clamp(Q.w, -1.0f, 1.0f));
		return HalfAngle * (V / Length);
	}
}

static inline Vector3f Quat4fLog(const Quat4f Q, const uniform float Epsilon = UE_SMALL_NUMBER)
{
	const Vector3f V = { Q.x, Q.y, Q.z };
	const float LengthSq = Vector3fLengthSquared(V);

	cif (LengthSq < Epsilon)
	{
		return V;
	}
	else
	{
		const float Length = sqrt(LengthSq);
		const float HalfAngle = acos(clamp(Q.w, -1.0f, 1.0f));
		return HalfAngle * (V / Length);
	}
}

static inline uniform Vector3f Quat4fToRotationVector(uniform const Quat4f Q, uniform const float Epsilon = UE_SMALL_NUMBER)
{
	return 2.0f * Quat4fLog(Q, Epsilon);
}

static inline Vector3f Quat4fToRotationVector(const Quat4f Q, uniform const float Epsilon = UE_SMALL_NUMBER)
{
	return 2.0f * Quat4fLog(Q, Epsilon);
}

static inline Quat4f Quat4fFromRotationVector(const Vector3f V, const uniform float Epsilon = UE_SMALL_NUMBER)
{
	return Quat4fExp(V * 0.5f, Epsilon);
}

static inline Quat4f Quat4fSlerp(const Quat4f X, const Quat4f Y, const float Alpha, const uniform float Epsilon = UE_SMALL_NUMBER)
{
	float RawCosom =
		X.x * Y.x +
		X.y * Y.y +
		X.z * Y.z +
		X.w * Y.w;

	const float Sign = select(RawCosom > 0.0f, 1.0f, -1.0f);
	RawCosom *= Sign;
		
	float Scale0 = 1.0f - Alpha;
	float Scale1 = Alpha * Sign;
		
	if (RawCosom < 0.9999f)
	{
		const float Omega = acos(RawCosom);
		const float InvSin = 1.0f / sin(Omega);
		Scale0 = sin(Scale0 * Omega) * InvSin;
		Scale1 = sin(Scale1 * Omega) * InvSin;
	}
	
	const Quat4f O = {
		Scale0 * X.x + Scale1 * Y.x,
		Scale0 * X.y + Scale1 * Y.y,
		Scale0 * X.z + Scale1 * Y.z,
		Scale0 * X.w + Scale1 * Y.w
	};

	return O;
}

static inline Quat4f Quat4fFromBasis(const Vector3f X, const Vector3f Y, const Vector3f Z)
{
	if (Z.z < 0.0f)
	{
		if (X.x > Y.y)
		{
			const Quat4f Q = {
                1.0f + X.x - Y.y - Z.z, 
                X.y + Y.x, 
                Z.x + X.z,
                Y.z - Z.y, 
			};

			return Quat4fNormalized(Q);
		}
		else
		{
			const Quat4f Q = {
                X.y + Y.x, 
                1.0f - X.x + Y.y - Z.z, 
                Y.z + Z.y,
                Z.x - X.z, 
			};

			return Quat4fNormalized(Q);
		}
	}
	else
	{
		if (X.x < -Y.y)
		{
			const Quat4f Q = {
				Z.x + X.z, 
				Y.z + Z.y, 
				1.0f - X.x - Y.y + Z.z,
				X.y - Y.x, 
			};

			return Quat4fNormalized(Q);
		}
		else
		{
			const Quat4f Q = {
                    Y.z - Z.y, 
                    Z.x - X.z, 
                    X.y - Y.x,
                    1.0f + X.x + Y.y + Z.z, 
			};

			return Quat4fNormalized(Q);
		}
	}

}

static inline uniform Vector4f Matrix44fTransformVector4f(uniform const Matrix44f M, uniform const Vector4f V)
{
	uniform const Vector4f O = {
		M[0] * V[0] + M[4] * V[1] + M[8] * V[2] + M[12] * V[3],
		M[1] * V[0] + M[5] * V[1] + M[9] * V[2] + M[13] * V[3],
		M[2] * V[0] + M[6] * V[1] + M[10] * V[2] + M[14] * V[3],
		M[3] * V[0] + M[7] * V[1] + M[11] * V[2] + M[15] * V[3],
	};

	return O;
}

static inline Vector4f Matrix44fTransformVector4f(const Matrix44f M, const Vector4f V)
{
	const Vector4f O = {
		M[0] * V[0] + M[4] * V[1] + M[8] * V[2] + M[12] * V[3],
		M[1] * V[0] + M[5] * V[1] + M[9] * V[2] + M[13] * V[3],
		M[2] * V[0] + M[6] * V[1] + M[10] * V[2] + M[14] * V[3],
		M[3] * V[0] + M[7] * V[1] + M[11] * V[2] + M[15] * V[3],
	};

	return O;
}

static inline uniform Vector4f DominantEigenVector(
	uniform const Matrix44f A,
	uniform const Vector4f V0,
	uniform const int Iterations)
{
	// Initial Guess at Eigen Vector & Value
	uniform Vector4f V = V0;
	uniform float EV = (Matrix44fTransformVector4f(A, V) / V).x;

	for (uniform int Iteration = 0; Iteration < Iterations; Iteration++)
	{
		// Power Iteration
		uniform const Vector4f Av = Matrix44fTransformVector4f(A, V);

		// Next Guess at Eigen Vector & Value
		uniform const Vector4f VNew = Vector4fNormalize(Av);
		uniform const float EVNew = (Matrix44fTransformVector4f(A, VNew) / VNew).x;

		// Update best guess
		V = VNew;
		EV = EVNew;
	}

	return V;
}

static inline Vector4f DominantEigenVector(
	const Matrix44f A,
	const Vector4f V0,
	uniform const int Iterations)
{
	// Initial Guess at Eigen Vector & Value
	Vector4f V = V0;
	float EV = (Matrix44fTransformVector4f(A, V) / V).x;

	for (uniform int Iteration = 0; Iteration < Iterations; Iteration++)
	{
		// Power Iteration
		const Vector4f Av = Matrix44fTransformVector4f(A, V);

		// Next Guess at Eigen Vector & Value
		const Vector4f VNew = Vector4fNormalize(Av);
		const float EVNew = (Matrix44fTransformVector4f(A, VNew) / VNew).x;

		// Update best guess
		V = VNew;
		EV = EVNew;
	}

	return V;
}

export void AnimDatabaseComputeMinMax(
	uniform float OutMin[],
	uniform float OutMax[],
	uniform const float Data[],
	uniform const int SampleNum,
	uniform const int DimNum,
	uniform const int32 ColSliceStart,
	uniform const int32 ColSliceNum)
{
	foreach(DimIdx = 0 ... ColSliceNum)
	{
		OutMin[ColSliceStart + DimIdx] = +UE_MAX_FLT;
		OutMax[ColSliceStart + DimIdx] = -UE_MAX_FLT;
	}

	for (uniform int SampleIdx = 0; SampleIdx < SampleNum; SampleIdx++)
	{
		foreach(DimIdx = 0 ... ColSliceNum)
		{
			OutMin[ColSliceStart + DimIdx] = min(OutMin[ColSliceStart + DimIdx], Data[SampleIdx * DimNum + ColSliceStart + DimIdx]);
			OutMax[ColSliceStart + DimIdx] = max(OutMax[ColSliceStart + DimIdx], Data[SampleIdx * DimNum + ColSliceStart + DimIdx]);
		}
	}
}

export void AnimDatabaseComputeMean(
	uniform float OutMeans[],
	uniform const float Vectors[],
	uniform const int RowNum,
	uniform const int ColNum)
{
	foreach(ColIdx = 0 ... ColNum)
	{
		OutMeans[ColIdx] = 0.0f;
	}

	for (uniform int RowIdx = 0; RowIdx < RowNum; RowIdx++)
	{
		uniform float MeanUpdateScale = 1.0f / (RowIdx + 1);

		foreach(ColIdx = 0 ... ColNum)
		{
			OutMeans[ColIdx] += MeanUpdateScale * (Vectors[RowIdx * ColNum + ColIdx] - OutMeans[ColIdx]);
		}
	}
}

export void AnimDatabaseComputeMeanStd(
	uniform float OutMeans[],
	uniform float OutStds[],
	uniform const float Vectors[],
	uniform const int RowNum,
	uniform const int ColNum,
	uniform const int32 ColSliceStart,
	uniform const int32 ColSliceNum)
{
	foreach(ColIdx = 0 ... ColSliceNum)
	{
		OutMeans[ColSliceStart + ColIdx] = 0.0f;
		OutStds[ColSliceStart + ColIdx] = 0.0f;
	}

	for (uniform int RowIdx = 0; RowIdx < RowNum; RowIdx++)
	{
		uniform float StdUpdateScale = (((uniform float)RowIdx / RowNum) / (RowIdx + 1));

		foreach(ColIdx = 0 ... ColSliceNum)
		{
			OutStds[ColSliceStart + ColIdx] += StdUpdateScale * Square(Vectors[RowIdx * ColNum + ColSliceStart + ColIdx] - OutMeans[ColSliceStart + ColIdx]);
		}

		uniform float MeanUpdateScale = 1.0f / (RowIdx + 1);

		foreach(ColIdx = 0 ... ColSliceNum)
		{
			OutMeans[ColSliceStart + ColIdx] += MeanUpdateScale * (Vectors[RowIdx * ColNum + ColSliceStart + ColIdx] - OutMeans[ColSliceStart + ColIdx]);
		}
	}

	foreach(ColIdx = 0 ... ColSliceNum)
	{
		OutStds[ColSliceStart + ColIdx] = sqrt(OutStds[ColSliceStart + ColIdx]);
	}
}

export void AnimDatabaseComputeMeanStdOfLog(
	uniform float OutMeanLogs[],
	uniform float OutStdLogs[],
	uniform const float Vectors[],
	uniform const int RowNum,
	uniform const int ColNum)
{
	foreach(ColIdx = 0 ... ColNum)
	{
		OutMeanLogs[ColIdx] = 0.0f;
		OutStdLogs[ColIdx] = 0.0f;
	}

	for (uniform int RowIdx = 0; RowIdx < RowNum; RowIdx++)
	{
		uniform float StdUpdateScale = (((uniform float)RowIdx / RowNum) / (RowIdx + 1));

		foreach(ColIdx = 0 ... ColNum)
		{
			OutStdLogs[ColIdx] += StdUpdateScale * Square(log(max(Vectors[RowIdx * ColNum + ColIdx], UE_SMALL_NUMBER)) - OutMeanLogs[ColIdx]);
		}

		uniform float MeanUpdateScale = 1.0f / (RowIdx + 1);

		foreach(ColIdx = 0 ... ColNum)
		{
			OutMeanLogs[ColIdx] += MeanUpdateScale * (log(max(Vectors[RowIdx * ColNum + ColIdx], UE_SMALL_NUMBER)) - OutMeanLogs[ColIdx]);
		}
	}

	foreach(ColIdx = 0 ... ColNum)
	{
		OutMeanLogs[ColIdx] = exp(OutMeanLogs[ColIdx]);
		OutStdLogs[ColIdx] = sqrt(OutStdLogs[ColIdx]);
	}
}

export void AnimDatabaseComputeMeanStdWithReference1D(
	uniform float OutMean[],
	uniform float OutStd[],
	uniform const float Rotations[],
	uniform const float Reference[],
	uniform const int Num)
{
	uniform Matrix44f Accum = {
		0.0f, 0.0f, 0.0f, 0.0f,
		0.0f, 0.0f, 0.0f, 0.0f,
		0.0f, 0.0f, 0.0f, 0.0f,
		0.0f, 0.0f, 0.0f, 0.0f
	};

	for (uniform int Idx = 0; Idx < Num; Idx++)
	{
		uniform const Quat4f Q = Quat4fLoad(Rotations, Idx);

		uniform const Matrix44f Outer = {
			((Q.x * Q.x) - Accum[0]),
			((Q.x * Q.y) - Accum[1]),
			((Q.x * Q.z) - Accum[2]),
			((Q.x * Q.w) - Accum[3]),

			((Q.y * Q.x) - Accum[4]),
			((Q.y * Q.y) - Accum[5]),
			((Q.y * Q.z) - Accum[6]),
			((Q.y * Q.w) - Accum[7]),

			((Q.z * Q.x) - Accum[8]),
			((Q.z * Q.y) - Accum[9]),
			((Q.z * Q.z) - Accum[10]),
			((Q.z * Q.w) - Accum[11]),

			((Q.w * Q.x) - Accum[12]),
			((Q.w * Q.y) - Accum[13]),
			((Q.w * Q.z) - Accum[14]),
			((Q.w * Q.w) - Accum[15]),
		};

		Accum += Outer / (Idx + 1);
	}

	uniform const Quat4f R = Quat4fLoad(Reference, 0);
	uniform const Vector4f RVec = { R.x, R.y, R.z, R.w };
	uniform const Vector4f AverageVec = DominantEigenVector(Accum, RVec, 128);
	uniform const Quat4f Mean = { AverageVec.x, AverageVec.y, AverageVec.z, AverageVec.w };

	uniform Vector3f Var = { 0.0f, 0.0f, 0.0f };

	for (uniform int Idx = 0; Idx < Num; Idx++)
	{
		uniform const Quat4f Diff = Quat4fAbs(Quat4fMul(Quat4fLoad(Rotations, Idx), Quat4fInv(Mean)));
		Var += Vector3fSquare(Quat4fToRotationVector(Diff)) / Num;
	}

	Quat4fSave(OutMean, 0, Mean);
	Vector3fSave(OutStd, 0, Vector3fSqrt(Var));
}

export void AnimDatabaseComputeMeanStdWithReference(
	uniform float OutMeans[],
	uniform float OutAccum[],
	uniform float OutStds[],
	uniform const float Rotations[],
	uniform const float Reference[],
	uniform const int RowNum,
	uniform const int ColNum)
{		
	foreach(Idx = 0 ... ColNum * 16)
	{
		OutAccum[Idx] = 0.0f;
	}

	for (uniform int RowIdx = 0; RowIdx < RowNum; RowIdx++)
	{
		foreach (ColIdx = 0 ... ColNum)
		{
			const Quat4f Q = Quat4fLoad(Rotations, RowIdx * ColNum + ColIdx);

			const Matrix44f Curr = Matrix44fLoad(OutAccum, ColIdx);

			const Matrix44f Outer = {
				((Q.x * Q.x) - Curr[0]),
				((Q.x * Q.y) - Curr[1]),
				((Q.x * Q.z) - Curr[2]),
				((Q.x * Q.w) - Curr[3]),

				((Q.y * Q.x) - Curr[4]),
				((Q.y * Q.y) - Curr[5]),
				((Q.y * Q.z) - Curr[6]),
				((Q.y * Q.w) - Curr[7]),

				((Q.z * Q.x) - Curr[8]),
				((Q.z * Q.y) - Curr[9]),
				((Q.z * Q.z) - Curr[10]),
				((Q.z * Q.w) - Curr[11]),

				((Q.w * Q.x) - Curr[12]),
				((Q.w * Q.y) - Curr[13]),
				((Q.w * Q.z) - Curr[14]),
				((Q.w * Q.w) - Curr[15]),
			};

			Matrix44fSave(OutAccum, ColIdx, Curr + Outer / (RowIdx + 1));
		}
	}

	foreach(ColIdx = 0 ... ColNum)
	{
		const Quat4f R = Quat4fLoad(Reference, ColIdx);
		const Vector4f RVec = { R.x, R.y, R.z, R.w };
		const Vector4f AverageVec = DominantEigenVector(Matrix44fLoad(OutAccum, ColIdx), RVec, 128);
		const Quat4f AverageQuat = { AverageVec.x, AverageVec.y, AverageVec.z, AverageVec.w };
		Quat4fSave(OutMeans, ColIdx, AverageQuat);
	}

	foreach(Idx = 0 ... ColNum * 3)
	{
		OutStds[Idx] = 0.0f;
	}

	for (uniform int RowIdx = 0; RowIdx < RowNum; RowIdx++)
	{
		foreach(ColIdx = 0 ... ColNum)
		{
			const Quat4f Diff = Quat4fAbs(Quat4fMul(Quat4fLoad(Rotations, RowIdx * ColNum + ColIdx), Quat4fInv(Quat4fLoad(OutMeans, ColIdx))));
			const Vector3f Curr = Vector3fLoad(OutStds, ColIdx);

			Vector3fSave(OutStds, ColIdx, Curr + Vector3fSquare(Quat4fToRotationVector(Diff)) / RowNum);
		}
	}

	foreach(Idx = 0 ... ColNum * 3)
	{
		OutStds[Idx] = sqrt(OutStds[Idx]);
	}
}

export void AnimDatabaseScaleInplace(
	uniform float InOutData[],
	uniform const float Scale[],
	uniform const int RowNum,
	uniform const int ColNum)
{
	foreach(RowIdx = 0 ... RowNum, ColIdx = 0 ... ColNum)
	{
		InOutData[RowIdx * ColNum + ColIdx] *= Scale[ColIdx];
	}
}

export void AnimDatabaseClampInplace(
	uniform float InOutData[],
	uniform const float Min[],
	uniform const float Max[],
	uniform const int RowNum,
	uniform const int ColNum)
{
	foreach(RowIdx = 0 ... RowNum, ColIdx = 0 ... ColNum)
	{
		InOutData[RowIdx * ColNum + ColIdx] = clamp(InOutData[RowIdx * ColNum + ColIdx], Min[ColIdx], Max[ColIdx]);
	}
}

export void AnimDatabaseNormalizeInplace(
	uniform float InOutData[],
	uniform const float Mean[],
	uniform const float Std[],
	uniform const int RowNum,
	uniform const int ColNum,
	uniform const float Eps)
{
	foreach(RowIdx = 0 ... RowNum, ColIdx = 0 ... ColNum)
	{
		InOutData[RowIdx * ColNum + ColIdx] = (InOutData[RowIdx * ColNum + ColIdx] - Mean[ColIdx]) / max(Std[ColIdx], Eps);
	}
}

export void AnimDatabaseNormalizeSingleScaleInplace(
	uniform float InOutData[],
	uniform const float Mean[],
	uniform const float Std,
	uniform const int RowNum,
	uniform const int ColNum,
	uniform const float Eps)
{
	foreach(RowIdx = 0 ... RowNum, ColIdx = 0 ... ColNum)
	{
		InOutData[RowIdx * ColNum + ColIdx] = (InOutData[RowIdx * ColNum + ColIdx] - Mean[ColIdx]) / max(Std, Eps);
	}
}

export void AnimDatabaseDenormalizeInplace(
	uniform float InOutData[],
	uniform const float Mean[],
	uniform const float Std[],
	uniform const int RowNum,
	uniform const int ColNum)
{
	foreach(RowIdx = 0 ... RowNum, ColIdx = 0 ... ColNum)
	{
		InOutData[RowIdx * ColNum + ColIdx] = InOutData[RowIdx * ColNum + ColIdx] * Std[ColIdx] + Mean[ColIdx];
	}
}

export void AnimDatabaseDenormalizeSingleScaleInplace(
	uniform float InOutData[],
	uniform const float Mean[],
	uniform const float Std,
	uniform const int RowNum,
	uniform const int ColNum)
{
	foreach(RowIdx = 0 ... RowNum, ColIdx = 0 ... ColNum)
	{
		InOutData[RowIdx * ColNum + ColIdx] = InOutData[RowIdx * ColNum + ColIdx] * Std + Mean[ColIdx];
	}
}

export void AnimDatabaseClampNormalizedSingleScaleInplace(
	uniform float InOutData[],
	uniform const float Mean[],
	uniform const float Std,
	uniform const float Min[],
	uniform const float Max[],
	uniform const int RowNum,
	uniform const int ColNum)
{
	foreach(RowIdx = 0 ... RowNum, ColIdx = 0 ... ColNum)
	{
		InOutData[RowIdx * ColNum + ColIdx] = clamp(
			InOutData[RowIdx * ColNum + ColIdx],
			(Min[ColIdx] - Mean[ColIdx]) / max(Std, UE_SMALL_NUMBER),
			(Max[ColIdx] - Mean[ColIdx]) / max(Std, UE_SMALL_NUMBER));
	}
}

export void AnimDatabaseLerpToTargetInplace(
	uniform float InOut[],
	uniform const float Target[],
	uniform const float Alpha,
	uniform const int RowNum,
	uniform const int ColNum)
{
	foreach(RowIdx = 0 ... RowNum, ColIdx = 0 ... ColNum)
	{
		InOut[RowIdx * ColNum + ColIdx] = Lerp(InOut[RowIdx * ColNum + ColIdx], Target[RowIdx * ColNum + ColIdx], Alpha);
	}
}

export void AnimDatabaseArrayInterpolateLinear(
	uniform float OutValues[],
	uniform const float P0[],
	uniform const float P1[],
	uniform const float Alpha,
	uniform const int Num)
{
	foreach(Idx = 0 ... Num)
	{
		OutValues[Idx] = Lerp(P0[Idx], P1[Idx], Alpha);
	}
}

export void AnimDatabasePoseDampInplaceUsingVelocityIntegrationMix(
	uniform float InOutBoneLocations[],
	uniform float InOutBoneRotations[],
	uniform float InOutBoneScales[],
	uniform float InOutBoneLinearVelocities[],
	uniform float InOutBoneAngularVelocities[],
	uniform float InOutBoneScalarVelocities[],
	uniform const float TargetBoneLocations[],
	uniform const float TargetBoneRotations[],
	uniform const float TargetBoneScales[],
	uniform const float TargetBoneLinearVelocities[],
	uniform const float TargetBoneAngularVelocities[],
	uniform const float TargetBoneScalarVelocities[],
	uniform const float DeltaTime,
	uniform const float PoseSmoothingTime,
	uniform const int FrameNum,
	uniform const int BoneNum)
{
	uniform const float Factor = 1.0f - InvExpApprox(DeltaTime / max(PoseSmoothingTime, UE_SMALL_NUMBER));

	foreach (FrameIdx = 0 ... FrameNum, BoneIdx = 0 ... BoneNum)
	{
		// Doing the integration with a 50/50 velocity mix between start and end pose is more "implicit"
		const Vector3f LinearVelocity = Vector3fLerp(Vector3fLoad(TargetBoneLinearVelocities, FrameIdx * BoneNum + BoneIdx), Vector3fLoad(InOutBoneLinearVelocities, FrameIdx * BoneNum + BoneIdx), 0.5f);
		const Vector3f AngularVelocity = Vector3fLerp(Vector3fLoad(TargetBoneAngularVelocities, FrameIdx * BoneNum + BoneIdx), Vector3fLoad(InOutBoneAngularVelocities, FrameIdx * BoneNum + BoneIdx), 0.5f);
		const Vector3f ScalarVelocity = Vector3fLerp(Vector3fLoad(TargetBoneScalarVelocities, FrameIdx * BoneNum + BoneIdx), Vector3fLoad(InOutBoneScalarVelocities, FrameIdx * BoneNum + BoneIdx), 0.5f);

		const Vector3f BoneLocationIntegrated = DeltaTime * LinearVelocity + Vector3fLoad(InOutBoneLocations, FrameIdx * BoneNum + BoneIdx);
		const Quat4f BoneRotationIntegrated = Quat4fMul(Quat4fFromRotationVector(DeltaTime * AngularVelocity), Quat4fLoad(InOutBoneRotations, FrameIdx * BoneNum + BoneIdx));
		const Vector3f BoneScaleIntegrated = Vector3fExpSafe(DeltaTime * ScalarVelocity) * Vector3fLoad(InOutBoneScales, FrameIdx * BoneNum + BoneIdx);

		Vector3fSave(InOutBoneLocations, FrameIdx * BoneNum + BoneIdx, Vector3fLerp(BoneLocationIntegrated, Vector3fLoad(TargetBoneLocations, FrameIdx * BoneNum + BoneIdx), Factor));
		Quat4fSave(InOutBoneRotations, FrameIdx * BoneNum + BoneIdx, Quat4fSlerp(BoneRotationIntegrated, Quat4fLoad(TargetBoneRotations, FrameIdx * BoneNum + BoneIdx), Factor));
		Vector3fSave(InOutBoneScales, FrameIdx * BoneNum + BoneIdx, Vector3fEerp(BoneScaleIntegrated, Vector3fLoad(TargetBoneScales, FrameIdx * BoneNum + BoneIdx), Factor));
		Vector3fSave(InOutBoneLinearVelocities, FrameIdx * BoneNum + BoneIdx, Vector3fLoad(TargetBoneLinearVelocities, FrameIdx * BoneNum + BoneIdx));
		Vector3fSave(InOutBoneAngularVelocities, FrameIdx * BoneNum + BoneIdx, Vector3fLoad(TargetBoneAngularVelocities, FrameIdx * BoneNum + BoneIdx));
		Vector3fSave(InOutBoneScalarVelocities, FrameIdx * BoneNum + BoneIdx, Vector3fLoad(TargetBoneScalarVelocities, FrameIdx * BoneNum + BoneIdx));
	}
}

export void AnimDatabasePoseDampInplaceUsingAdaptiveVelocityIntegrationMix(
	uniform float InOutBoneLocations[],
	uniform float InOutBoneRotations[],
	uniform float InOutBoneScales[],
	uniform float InOutBoneLinearVelocities[],
	uniform float InOutBoneAngularVelocities[],
	uniform float InOutBoneScalarVelocities[],
	uniform const float TargetBoneLocations[],
	uniform const float TargetBoneRotations[],
	uniform const float TargetBoneScales[],
	uniform const float TargetBoneLinearVelocities[],
	uniform const float TargetBoneAngularVelocities[],
	uniform const float TargetBoneScalarVelocities[],
	uniform const float DeltaTime,
	uniform const float MinPoseSmoothingTime,
	uniform const float MaxPoseSmoothingTime,
	uniform const float LinearVelocitySmoothingThreshold,
	uniform const float AngularVelocitySmoothingThreshold,
	uniform const float ScalarVelocitySmoothingThreshold,
	uniform const int FrameNum,
	uniform const int BoneNum)
{
	uniform const float ClampedMinPoseSmoothingTime = max(MinPoseSmoothingTime, UE_SMALL_NUMBER);
	uniform const float ClampedMaxPoseSmoothingTime = max(MaxPoseSmoothingTime, UE_SMALL_NUMBER);

	foreach (FrameIdx = 0 ... FrameNum, BoneIdx = 0 ... BoneNum)
	{
		// Doing the integration with a 50/50 velocity mix between start and end pose is more "implicit"
		const Vector3f LinearVelocity = Vector3fLerp(Vector3fLoad(TargetBoneLinearVelocities, FrameIdx * BoneNum + BoneIdx), Vector3fLoad(InOutBoneLinearVelocities, FrameIdx * BoneNum + BoneIdx), 0.5f);
		const Vector3f AngularVelocity = Vector3fLerp(Vector3fLoad(TargetBoneAngularVelocities, FrameIdx * BoneNum + BoneIdx), Vector3fLoad(InOutBoneAngularVelocities, FrameIdx * BoneNum + BoneIdx), 0.5f);
		const Vector3f ScalarVelocity = Vector3fLerp(Vector3fLoad(TargetBoneScalarVelocities, FrameIdx * BoneNum + BoneIdx), Vector3fLoad(InOutBoneScalarVelocities, FrameIdx * BoneNum + BoneIdx), 0.5f);

		const Vector3f BoneLocationIntegrated = DeltaTime * LinearVelocity + Vector3fLoad(InOutBoneLocations, FrameIdx * BoneNum + BoneIdx);
		const Quat4f BoneRotationIntegrated = Quat4fMul(Quat4fFromRotationVector(DeltaTime * AngularVelocity), Quat4fLoad(InOutBoneRotations, FrameIdx * BoneNum + BoneIdx));
		const Vector3f BoneScaleIntegrated = Vector3fExpSafe(DeltaTime * ScalarVelocity) * Vector3fLoad(InOutBoneScales, FrameIdx * BoneNum + BoneIdx);

		const float LinearVelocitySmoothingTime = (ClampedMaxPoseSmoothingTime * LinearVelocitySmoothingThreshold) / max(Vector3fLength(LinearVelocity), UE_SMALL_NUMBER);
		const float AngularVelocitySmoothingTime = (ClampedMaxPoseSmoothingTime * AngularVelocitySmoothingThreshold) / max(Vector3fLength(AngularVelocity), UE_SMALL_NUMBER);
		const float ScalarVelocitySmoothingTime = (ClampedMaxPoseSmoothingTime * ScalarVelocitySmoothingThreshold) / max(Vector3fLength(ScalarVelocity), UE_SMALL_NUMBER);

		const float LinearVelocityFactor = 1.0f - InvExpApprox(DeltaTime / clamp(LinearVelocitySmoothingTime, ClampedMinPoseSmoothingTime, ClampedMaxPoseSmoothingTime));
		const float AngularVelocityFactor = 1.0f - InvExpApprox(DeltaTime / clamp(AngularVelocitySmoothingTime, ClampedMinPoseSmoothingTime, ClampedMaxPoseSmoothingTime));
		const float ScalarVelocityFactor = 1.0f - InvExpApprox(DeltaTime / clamp(ScalarVelocitySmoothingTime, ClampedMinPoseSmoothingTime, ClampedMaxPoseSmoothingTime));

		Vector3fSave(InOutBoneLocations, FrameIdx * BoneNum + BoneIdx, Vector3fLerp(BoneLocationIntegrated, Vector3fLoad(TargetBoneLocations, FrameIdx * BoneNum + BoneIdx), LinearVelocityFactor));
		Quat4fSave(InOutBoneRotations, FrameIdx * BoneNum + BoneIdx, Quat4fSlerp(BoneRotationIntegrated, Quat4fLoad(TargetBoneRotations, FrameIdx * BoneNum + BoneIdx), AngularVelocityFactor));
		Vector3fSave(InOutBoneScales, FrameIdx * BoneNum + BoneIdx, Vector3fEerp(BoneScaleIntegrated, Vector3fLoad(TargetBoneScales, FrameIdx * BoneNum + BoneIdx), ScalarVelocityFactor));
		Vector3fSave(InOutBoneLinearVelocities, FrameIdx * BoneNum + BoneIdx, Vector3fLoad(TargetBoneLinearVelocities, FrameIdx * BoneNum + BoneIdx));
		Vector3fSave(InOutBoneAngularVelocities, FrameIdx * BoneNum + BoneIdx, Vector3fLoad(TargetBoneAngularVelocities, FrameIdx * BoneNum + BoneIdx));
		Vector3fSave(InOutBoneScalarVelocities, FrameIdx * BoneNum + BoneIdx, Vector3fLoad(TargetBoneScalarVelocities, FrameIdx * BoneNum + BoneIdx));
	}
}


export void AnimDatabaseFromPoseVectorCopyVector3(
	uniform float OutBoneVectors[],
	uniform const float InPoseVector[],
	uniform const int32 BoneDataIndices[],
	uniform const int BoneNum,
	uniform const int Offset)
{
	foreach (BoneIdx = 0 ... BoneNum)
	{
		const int32 BoneVectorIdx = BoneDataIndices[BoneIdx];

		if (BoneVectorIdx != INDEX_NONE)
		{
			const Vector3f B = { 
#pragma ignore warning(perf)
				InPoseVector[Offset + BoneIdx * 3 + 0],
#pragma ignore warning(perf)
				InPoseVector[Offset + BoneIdx * 3 + 1],
#pragma ignore warning(perf)
				InPoseVector[Offset + BoneIdx * 3 + 2]
			};

			Vector3fSave(OutBoneVectors, BoneVectorIdx, B);
		}
	}
}

export void AnimDatabaseFromPoseVectorCopyScale(
	uniform float OutBoneScales[],
	uniform const float InPoseVector[],
	uniform const int32 BoneScaleIndices[],
	uniform const int BoneNum,
	uniform const int Offset,
	uniform const float MaxLogScale)
{
	foreach (BoneIdx = 0 ... BoneNum)
	{
		const int32 BoneScaleIdx = BoneScaleIndices[BoneIdx];

		if (BoneScaleIdx != INDEX_NONE)
		{
			const Vector3f B = { 
#pragma ignore warning(perf)
				InPoseVector[Offset + BoneIdx * 3 + 0],
#pragma ignore warning(perf)
				InPoseVector[Offset + BoneIdx * 3 + 1],
#pragma ignore warning(perf)
				InPoseVector[Offset + BoneIdx * 3 + 2]
			};

			Vector3fSave(OutBoneScales, BoneScaleIdx, Vector3fExpSafe(B, MaxLogScale));
		}
	}
}

export void AnimDatabaseFromPoseVectorCopyRotation(
	uniform float OutBoneRotations[],
	uniform const float InPoseVector[],
	uniform const int32 BoneRotationIndices[],
	uniform const int BoneNum,
	uniform const int Offset)
{
	const uniform Vector3f DefaultForward = { 1.0f, 0.0f, 0.0f };
	const uniform Vector3f DefaultRight = { 0.0f, 1.0f, 0.0f };
	const uniform Vector3f DefaultUp = { 0.0f, 0.0f, 1.0f };

	foreach (BoneIdx = 0 ... BoneNum)
	{
		const int32 BoneRotationIdx = BoneRotationIndices[BoneIdx];

		if (BoneRotationIdx != INDEX_NONE)
		{
			Vector3f BoneRotationForward = {
#pragma ignore warning(perf)
				InPoseVector[Offset + BoneIdx * 6 + 0],
#pragma ignore warning(perf)
				InPoseVector[Offset + BoneIdx * 6 + 1],
#pragma ignore warning(perf)
				InPoseVector[Offset + BoneIdx * 6 + 2]
			};

			Vector3f BoneRotationRight = {
#pragma ignore warning(perf)
				InPoseVector[Offset + BoneIdx * 6 + 3],
#pragma ignore warning(perf)
				InPoseVector[Offset + BoneIdx * 6 + 4],
#pragma ignore warning(perf)
				InPoseVector[Offset + BoneIdx * 6 + 5]
			};

			Vector3f BoneRotationUp = Vector3fNormalizeSafe(Vector3fCross(BoneRotationForward, BoneRotationRight), UE_SMALL_NUMBER, DefaultUp);
			BoneRotationRight = Vector3fNormalizeSafe(Vector3fCross(BoneRotationUp, BoneRotationForward), UE_SMALL_NUMBER, DefaultRight);
			BoneRotationForward = Vector3fNormalizeSafe(BoneRotationForward, UE_SMALL_NUMBER, DefaultUp);

			Quat4fSave(OutBoneRotations, BoneRotationIdx, Quat4fFromBasis(BoneRotationForward, BoneRotationRight, BoneRotationUp));
		}
	}
}