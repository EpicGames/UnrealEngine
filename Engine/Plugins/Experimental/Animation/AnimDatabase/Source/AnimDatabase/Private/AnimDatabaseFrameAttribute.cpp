// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimDatabaseFrameAttribute.h"

#include "AnimDatabase.h"
#include "AnimDatabaseMath.h"
#include "AnimDatabasePose.h"

#include "LearningFrameSet.h"
#include "LearningFrameRangeSet.h"
#include "LearningFrameAttribute.h"
#include "LearningRandom.h"

#include "UObject/Package.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimNotifies/AnimNotify.h"
#include "Animation/AnimNotifies/AnimNotifyState.h"
#include "Async/ParallelFor.h"
#include "Tasks/Task.h"

#ifndef UE_ANIMDATABASE_ISPC
#define UE_ANIMDATABASE_ISPC INTEL_ISPC
#endif

#if UE_ANIMDATABASE_ISPC
#include "AnimDatabase.ispc.generated.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimDatabaseFrameAttribute)

#define LOCTEXT_NAMESPACE "AnimDatabaseFrameAttribute"

namespace UE::AnimDatabase::FrameAttribute::Private
{
	static inline void EventDifference(
		float* RESTRICT OutValues,
		const float* RESTRICT TimeUntilEventsKnown,
		const float* RESTRICT TimeUntilEvents,
		const bool TimeUntilEventKnown,
		const float TimeUntilEvent,
		const int32 ValueNum,
		const float ValueScale)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::AnimDatabase::FrameAttrbute::Private::EventDifference);

		const float Apprehension = 1.0f; // TODO: Don't hard-code

		for (int32 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
		{
			if (TimeUntilEventKnown && TimeUntilEventsKnown[ValueIdx])
			{
				OutValues[ValueIdx] = ValueScale * FMath::Abs(
					FMath::Clamp(TimeUntilEvent, -Apprehension, +Apprehension) -
					FMath::Clamp(TimeUntilEvents[ValueIdx], -Apprehension, +Apprehension));
			}
			else if (TimeUntilEventKnown && !TimeUntilEventsKnown[ValueIdx])
			{
				OutValues[ValueIdx] = ValueScale * FMath::Abs(
					FMath::Abs(FMath::Clamp(TimeUntilEvent, -Apprehension, +Apprehension)) - Apprehension);
			}
			else if (!TimeUntilEventKnown && TimeUntilEventsKnown[ValueIdx])
			{
				OutValues[ValueIdx] = ValueScale * FMath::Abs(
					FMath::Abs(FMath::Clamp(TimeUntilEvents[ValueIdx], -Apprehension, +Apprehension)) - Apprehension);
			}
			else if (!TimeUntilEventKnown && !TimeUntilEventsKnown[ValueIdx])
			{
				OutValues[ValueIdx] = 0.0f;
			}
		}
	}

	static inline FString FrameAttributeFrameToString(const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 FrameIdx)
	{
		switch (FrameAttribute.Type)
		{

		case EAnimDatabaseAttributeType::Null:
		{
			return TEXT("null");
		}

		case EAnimDatabaseAttributeType::Bool:
		{
			return (FrameAttribute.FrameAttribute->AttributeData[0][FrameIdx] > 0.5f) ? TEXT("true") : TEXT("false");
		}

		case EAnimDatabaseAttributeType::Float:
		case EAnimDatabaseAttributeType::Angle:
		{
			return FString::Printf(TEXT("%0.2f"), FrameAttribute.FrameAttribute->AttributeData[0][FrameIdx]);
		}

		case EAnimDatabaseAttributeType::Location:
		case EAnimDatabaseAttributeType::Direction:
		case EAnimDatabaseAttributeType::Scale:
		case EAnimDatabaseAttributeType::LinearVelocity:
		case EAnimDatabaseAttributeType::AngularVelocity:
		case EAnimDatabaseAttributeType::ScalarVelocity:
		{
			return FString::Printf(TEXT("[%0.2f %0.2f %0.2f]"),
				FrameAttribute.FrameAttribute->AttributeData[0][FrameIdx],
				FrameAttribute.FrameAttribute->AttributeData[1][FrameIdx],
				FrameAttribute.FrameAttribute->AttributeData[2][FrameIdx]);
		}

		case EAnimDatabaseAttributeType::Rotation:
		{
			return FString::Printf(TEXT("[%0.2f %0.2f %0.2f %0.2f]"),
				FrameAttribute.FrameAttribute->AttributeData[0][FrameIdx],
				FrameAttribute.FrameAttribute->AttributeData[1][FrameIdx],
				FrameAttribute.FrameAttribute->AttributeData[2][FrameIdx],
				FrameAttribute.FrameAttribute->AttributeData[3][FrameIdx]);
		}

		case EAnimDatabaseAttributeType::Transform:
		{
			return FString::Printf(TEXT("(Location=[%0.2f %0.2f %0.2f] Rotation=[%0.2f %0.2f %0.2f %0.2f] Scale=[%0.2f %0.2f %0.2f])"),
				FrameAttribute.FrameAttribute->AttributeData[0][FrameIdx],
				FrameAttribute.FrameAttribute->AttributeData[1][FrameIdx],
				FrameAttribute.FrameAttribute->AttributeData[2][FrameIdx],
				FrameAttribute.FrameAttribute->AttributeData[3][FrameIdx],
				FrameAttribute.FrameAttribute->AttributeData[4][FrameIdx],
				FrameAttribute.FrameAttribute->AttributeData[5][FrameIdx],
				FrameAttribute.FrameAttribute->AttributeData[6][FrameIdx],
				FrameAttribute.FrameAttribute->AttributeData[7][FrameIdx],
				FrameAttribute.FrameAttribute->AttributeData[8][FrameIdx],
				FrameAttribute.FrameAttribute->AttributeData[9][FrameIdx]);
		}

		case EAnimDatabaseAttributeType::Event:
		{
			return FrameAttribute.FrameAttribute->AttributeData[0][FrameIdx] > 0.5f ?
				FString::Printf(TEXT("%0.2f"), FrameAttribute.FrameAttribute->AttributeData[1][FrameIdx]) :
				TEXT("  --  ");
		}

		default:
		{
			return TEXT("Unknown");
		}
		}
	}
}

bool FAnimDatabaseFrameAttribute::IsValid() const
{
	return FrameAttribute.IsValid();
}

bool FAnimDatabaseFrameAttribute::GetAsBool(const int32 FrameIdx) const
{
	return FrameAttribute->AttributeData[0][FrameIdx] == 1.0f;
}

float FAnimDatabaseFrameAttribute::GetAsFloat(const int32 FrameIdx) const
{
	return FrameAttribute->AttributeData[0][FrameIdx];
}

void FAnimDatabaseFrameAttribute::GetAsEvent(bool& bOutTimeUntilEventKnown, float& OutTimeUntilEvent, const int32 FrameIdx) const
{
	bOutTimeUntilEventKnown = FrameAttribute->AttributeData[0][FrameIdx] == 1.0f;
	OutTimeUntilEvent = FrameAttribute->AttributeData[1][FrameIdx];
}

FVector3f FAnimDatabaseFrameAttribute::GetAsLocation(const int32 FrameIdx) const
{
	return FVector3f(
		FrameAttribute->AttributeData[0][FrameIdx],
		FrameAttribute->AttributeData[1][FrameIdx],
		FrameAttribute->AttributeData[2][FrameIdx]);
}

FVector3f FAnimDatabaseFrameAttribute::GetAsScale(const int32 FrameIdx) const
{
	return FVector3f(
		FrameAttribute->AttributeData[0][FrameIdx],
		FrameAttribute->AttributeData[1][FrameIdx],
		FrameAttribute->AttributeData[2][FrameIdx]);
}

FVector3f FAnimDatabaseFrameAttribute::GetAsVelocity(const int32 FrameIdx) const
{
	return FVector3f(
		FrameAttribute->AttributeData[0][FrameIdx],
		FrameAttribute->AttributeData[1][FrameIdx],
		FrameAttribute->AttributeData[2][FrameIdx]);
}

FVector3f FAnimDatabaseFrameAttribute::GetAsDirection(const int32 FrameIdx) const
{
	return FVector3f(
		FrameAttribute->AttributeData[0][FrameIdx],
		FrameAttribute->AttributeData[1][FrameIdx],
		FrameAttribute->AttributeData[2][FrameIdx]);
}

FQuat4f FAnimDatabaseFrameAttribute::GetAsRotation(const int32 FrameIdx) const
{
	return FQuat4f(
		FrameAttribute->AttributeData[0][FrameIdx],
		FrameAttribute->AttributeData[1][FrameIdx],
		FrameAttribute->AttributeData[2][FrameIdx],
		FrameAttribute->AttributeData[3][FrameIdx]);
}

FTransform3f FAnimDatabaseFrameAttribute::GetAsTransform(const int32 FrameIdx) const
{
	return FTransform3f(
		FQuat4f(
			FrameAttribute->AttributeData[3][FrameIdx],
			FrameAttribute->AttributeData[4][FrameIdx],
			FrameAttribute->AttributeData[5][FrameIdx],
			FrameAttribute->AttributeData[6][FrameIdx]),
		FVector3f(
			FrameAttribute->AttributeData[0][FrameIdx],
			FrameAttribute->AttributeData[1][FrameIdx],
			FrameAttribute->AttributeData[2][FrameIdx]),
		FVector3f(
			FrameAttribute->AttributeData[7][FrameIdx],
			FrameAttribute->AttributeData[8][FrameIdx],
			FrameAttribute->AttributeData[9][FrameIdx]));
}

FVector FAnimDatabaseFrameAttribute::GetAsLocationDouble(const int32 FrameIdx) const
{
	return FVector(
		FrameAttribute->AttributeData[0][FrameIdx],
		FrameAttribute->AttributeData[1][FrameIdx],
		FrameAttribute->AttributeData[2][FrameIdx]);
}

FVector FAnimDatabaseFrameAttribute::GetAsScaleDouble(const int32 FrameIdx) const
{
	return FVector(
		FrameAttribute->AttributeData[0][FrameIdx],
		FrameAttribute->AttributeData[1][FrameIdx],
		FrameAttribute->AttributeData[2][FrameIdx]);
}

FVector FAnimDatabaseFrameAttribute::GetAsVelocityDouble(const int32 FrameIdx) const
{
	return FVector(
		FrameAttribute->AttributeData[0][FrameIdx],
		FrameAttribute->AttributeData[1][FrameIdx],
		FrameAttribute->AttributeData[2][FrameIdx]);
}

FVector FAnimDatabaseFrameAttribute::GetAsDirectionDouble(const int32 FrameIdx) const
{
	return FVector(
		FrameAttribute->AttributeData[0][FrameIdx],
		FrameAttribute->AttributeData[1][FrameIdx],
		FrameAttribute->AttributeData[2][FrameIdx]).GetSafeNormal(UE_SMALL_NUMBER, FVector::ForwardVector);
}

FQuat FAnimDatabaseFrameAttribute::GetAsRotationDouble(const int32 FrameIdx) const
{
	return FQuat(
		FrameAttribute->AttributeData[0][FrameIdx],
		FrameAttribute->AttributeData[1][FrameIdx],
		FrameAttribute->AttributeData[2][FrameIdx],
		FrameAttribute->AttributeData[3][FrameIdx]).GetNormalized();
}

FTransform FAnimDatabaseFrameAttribute::GetAsTransformDouble(const int32 FrameIdx) const
{
	return FTransform(
		FQuat(
			FrameAttribute->AttributeData[3][FrameIdx],
			FrameAttribute->AttributeData[4][FrameIdx],
			FrameAttribute->AttributeData[5][FrameIdx],
			FrameAttribute->AttributeData[6][FrameIdx]).GetNormalized(),
		FVector(
			FrameAttribute->AttributeData[0][FrameIdx],
			FrameAttribute->AttributeData[1][FrameIdx],
			FrameAttribute->AttributeData[2][FrameIdx]),
		FVector(
			FrameAttribute->AttributeData[7][FrameIdx],
			FrameAttribute->AttributeData[8][FrameIdx],
			FrameAttribute->AttributeData[9][FrameIdx]));
}

float FAnimDatabaseFrameAttribute::GetAsAngleDegrees(const int32 FrameIdx) const
{
	return FMath::RadiansToDegrees(FrameAttribute->AttributeData[0][FrameIdx]);
}

float FAnimDatabaseFrameAttribute::GetAsAngleRadians(const int32 FrameIdx) const
{
	return FrameAttribute->AttributeData[0][FrameIdx];
}

bool FAnimDatabaseFrameAttribute::Serialize(FArchive& Ar)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAnimDatabaseFrameAttribute::Serialize);

	static constexpr int32 Magic = 0x8923e41c;
	static constexpr int32 Version = 0;

	if (Ar.IsSaving())
	{
		int32 SaveMagic = Magic;
		int32 SaveVersion = Version;
		bool bSaveIsValid = FrameAttribute.IsValid();

		Ar << SaveMagic;
		Ar << SaveVersion;
		Ar << Type;
		Ar << bSaveIsValid;
		if (bSaveIsValid)
		{
			UE::Learning::Array::Serialize(Ar, FrameAttribute->FrameRangeSet.EntrySequences);
			UE::Learning::Array::Serialize(Ar, FrameAttribute->FrameRangeSet.EntryRangeOffsets);
			UE::Learning::Array::Serialize(Ar, FrameAttribute->FrameRangeSet.EntryRangeNums);
			UE::Learning::Array::Serialize(Ar, FrameAttribute->FrameRangeSet.RangeStarts);
			UE::Learning::Array::Serialize(Ar, FrameAttribute->FrameRangeSet.RangeLengths);
			UE::Learning::Array::Serialize(Ar, FrameAttribute->FrameRangeSet.RangeOffsets);
			UE::Learning::Array::Serialize(Ar, FrameAttribute->AttributeData);
		}
	}
	else if (Ar.IsLoading())
	{
		int64 Offset = Ar.Tell();
		int32 LoadMagic = 0;
		int32 LoadVersion = 0;
		bool bLoadIsValid = false;

		Ar << LoadMagic;
		if (!ensure(LoadMagic == Magic)) { Ar.Seek(Offset); FrameAttribute.Reset(); return false; }
		Ar << LoadVersion;
		if (!ensure(LoadVersion == Version)) { Ar.Seek(Offset); FrameAttribute.Reset(); return false; }
		Ar << Type;
		Ar << bLoadIsValid;
		if (bLoadIsValid)
		{
			FrameAttribute = MakeShared<UE::Learning::FFrameAttribute>();
			UE::Learning::Array::Serialize(Ar, FrameAttribute->FrameRangeSet.EntrySequences);
			UE::Learning::Array::Serialize(Ar, FrameAttribute->FrameRangeSet.EntryRangeOffsets);
			UE::Learning::Array::Serialize(Ar, FrameAttribute->FrameRangeSet.EntryRangeNums);
			UE::Learning::Array::Serialize(Ar, FrameAttribute->FrameRangeSet.RangeStarts);
			UE::Learning::Array::Serialize(Ar, FrameAttribute->FrameRangeSet.RangeLengths);
			UE::Learning::Array::Serialize(Ar, FrameAttribute->FrameRangeSet.RangeOffsets);
			UE::Learning::Array::Serialize(Ar, FrameAttribute->AttributeData);
		}
		else
		{
			FrameAttribute.Reset();
		}
	}

	return true;
}

bool operator==(const FAnimDatabaseFrameAttribute& Lhs, const FAnimDatabaseFrameAttribute& Rhs)
{
	if (Lhs.Type != Rhs.Type) { return false; }

	if (Lhs.IsValid() && Rhs.IsValid())
	{
		return UE::Learning::FrameAttribute::Equal(*Lhs.FrameAttribute, *Rhs.FrameAttribute);
	}
	else if (!Lhs.IsValid() && !Rhs.IsValid())
	{
		return true;
	}
	else
	{
		return false;
	}
}

bool operator==(const FAnimDatabaseFrameAttributeEntry& Lhs, const FAnimDatabaseFrameAttributeEntry& Rhs)
{
	return Lhs.Name == Rhs.Name && Lhs.FrameAttribute == Rhs.FrameAttribute;
}

int32 UAnimDatabaseFrameAttributeLibrary::AttributeTypeSize(const EAnimDatabaseAttributeType Type)
{
	switch (Type)
	{
	case EAnimDatabaseAttributeType::Null: return 0;
	case EAnimDatabaseAttributeType::Bool: return 1;
	case EAnimDatabaseAttributeType::Float: return 1;
	case EAnimDatabaseAttributeType::Location: return 3;
	case EAnimDatabaseAttributeType::Rotation: return 4;
	case EAnimDatabaseAttributeType::Scale: return 3;
	case EAnimDatabaseAttributeType::LinearVelocity: return 3;
	case EAnimDatabaseAttributeType::AngularVelocity: return 3;
	case EAnimDatabaseAttributeType::ScalarVelocity: return 3;
	case EAnimDatabaseAttributeType::Direction: return 3;
	case EAnimDatabaseAttributeType::Transform: return 10;
	case EAnimDatabaseAttributeType::Event: return 2;
	case EAnimDatabaseAttributeType::Angle: return 1;
	default: checkNoEntry(); return 0;
	}
}

FString UAnimDatabaseFrameAttributeLibrary::AttributeTypeName(const EAnimDatabaseAttributeType Type)
{
	return AttributeTypeNameInternal(Type);
}

const TCHAR* UAnimDatabaseFrameAttributeLibrary::AttributeTypeNameInternal(const EAnimDatabaseAttributeType Type)
{
	switch (Type)
	{
	case EAnimDatabaseAttributeType::Null: return TEXT("Null");
	case EAnimDatabaseAttributeType::Bool: return TEXT("Bool");
	case EAnimDatabaseAttributeType::Float: return TEXT("Float");
	case EAnimDatabaseAttributeType::Location: return TEXT("Location");
	case EAnimDatabaseAttributeType::Rotation: return TEXT("Rotation");
	case EAnimDatabaseAttributeType::Scale: return TEXT("Scale");
	case EAnimDatabaseAttributeType::LinearVelocity: return TEXT("LinearVelocity");
	case EAnimDatabaseAttributeType::AngularVelocity: return TEXT("AngularVelocity");
	case EAnimDatabaseAttributeType::ScalarVelocity: return TEXT("ScalarVelocity");
	case EAnimDatabaseAttributeType::Direction: return TEXT("Direction");
	case EAnimDatabaseAttributeType::Transform: return TEXT("Transform");
	case EAnimDatabaseAttributeType::Event: return TEXT("Event");
	case EAnimDatabaseAttributeType::Angle: return TEXT("Angle");
	default: return TEXT("Unknown");
	}
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeEmptyFrameAttribute()
{
	FAnimDatabaseFrameAttribute Out;
	Out.Type = EAnimDatabaseAttributeType::Null;
	Out.FrameAttribute = MakeShared<UE::Learning::FFrameAttribute>();
	return Out;
} 

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeNullFrameAttribute(const FAnimDatabaseFrameRanges& FrameRanges)
{
	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeNullFrameAttribute: FrameRanges is Invalid.");
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.FrameAttribute->FrameRangeSet = *FrameRanges.FrameRangeSet;
	Out.FrameAttribute->AttributeData.SetNumUninitialized({ AttributeTypeSize(Out.Type), FrameRanges.FrameRangeSet->GetTotalFrameNum() });
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeFrameAttributeFromFunction(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const UAnimDatabaseFrameAttributeFunction* Function)
{
	if (!Function)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameAttributeFromFunction: Function is null.");
		return FAnimDatabaseFrameAttribute();
	}

	const FAnimDatabaseFrameAttribute OutFrameAttribute = Function->MakeFrameAttribute(Database, FrameRanges);

	if (!OutFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameAttributeFromFunction: Invalid Frame Attribute Object.");
		return FAnimDatabaseFrameAttribute();
	}

	return OutFrameAttribute;
}

void UAnimDatabaseFrameAttributeLibrary::MakeFrameAttributesFromFunctions(
	TArray<FAnimDatabaseFrameAttribute>& OutFrameAttributes,
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const TArray<UAnimDatabaseFrameAttributeFunction*>& Functions)
{
	OutFrameAttributes.SetNum(Functions.Num());
	MakeFrameAttributesFromFunctionsArrayView(OutFrameAttributes, Database, FrameRanges, Functions);
}

void UAnimDatabaseFrameAttributeLibrary::MakeFrameAttributesFromFunctionsArrayView(
	const TArrayView<FAnimDatabaseFrameAttribute> OutFrameAttributes,
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const TArrayView<UAnimDatabaseFrameAttributeFunction* const> Functions)
{
	check(OutFrameAttributes.Num() == Functions.Num());
	const int32 FunctionNum = Functions.Num();

	for (int32 FunctionIdx = 0; FunctionIdx < FunctionNum; FunctionIdx++)
	{
		OutFrameAttributes[FunctionIdx] = MakeFrameAttributeFromFunction(Database, FrameRanges, Functions[FunctionIdx]);
	}
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeFrameAttributeFromClass(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TSubclassOf<UAnimDatabaseFrameAttributeFunction> Class)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameAttributeFromClass: Database is null.");
		return FAnimDatabaseFrameAttribute();
	}

	if (!Class)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameAttributeFromClass: Class is null.");
		return FAnimDatabaseFrameAttribute();
	}

	return MakeFrameAttributeFromFunction(Database, FrameRanges, Class->GetDefaultObject<UAnimDatabaseFrameAttributeFunction>());
}

void UAnimDatabaseFrameAttributeLibrary::MakeFrameAttributesFromClasses(
	TArray<FAnimDatabaseFrameAttribute>& OutFrameAttributes,
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const TArray<TSubclassOf<UAnimDatabaseFrameAttributeFunction>>& Classes)
{
	OutFrameAttributes.SetNum(Classes.Num());
	MakeFrameAttributesFromClassesArrayView(OutFrameAttributes, Database, FrameRanges, Classes);
}

void UAnimDatabaseFrameAttributeLibrary::MakeFrameAttributesFromClassesArrayView(
	const TArrayView<FAnimDatabaseFrameAttribute> OutFrameAttributes,
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const TArrayView<const TSubclassOf<UAnimDatabaseFrameAttributeFunction>> Classes)
{
	check(OutFrameAttributes.Num() == Classes.Num());
	const int32 ClassNum = Classes.Num();

	for (int32 ClassIdx = 0; ClassIdx < ClassNum; ClassIdx++)
	{
		OutFrameAttributes[ClassIdx] = MakeFrameAttributeFromClass(Database, FrameRanges, Classes[ClassIdx]);
	}
}

void UAnimDatabaseFrameAttributeLibrary::MakeFrameAttributesFromClassesMap(
	TMap<FName, FAnimDatabaseFrameAttribute>& OutFrameAttributes,
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const TMap<FName, TSubclassOf<UAnimDatabaseFrameAttributeFunction>>& Classes)
{
	OutFrameAttributes.Empty(Classes.Num());
	for (const TPair<FName, TSubclassOf<UAnimDatabaseFrameAttributeFunction>>& Entry : Classes)
	{
		OutFrameAttributes.Add(Entry.Key, MakeFrameAttributeFromClass(Database, FrameRanges, Entry.Value));
	}
}


FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeBoolFrameAttributeFromActiveRanges(const FAnimDatabaseFrameRanges& Active, const FAnimDatabaseFrameRanges& FrameRanges)
{
	if (!Active.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoolFrameAttributeFromActiveRanges: Active is Invalid.");
		return FAnimDatabaseFrameAttribute();
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoolFrameAttributeFromActiveRanges: FrameRanges is Invalid.");
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Bool;
	UE::Learning::FrameAttribute::Zeros(*Out.FrameAttribute, *FrameRanges.FrameRangeSet, 1);

	UE::Learning::FrameRangeSet::ParallelForEachRange(Out.FrameAttribute->FrameRangeSet, [&Out, &Active](
		const int32 TotalRangeIdx,
		const int32 EntryIdx,
		const int32 RangeIdx) {

			const int32 FrameOffset = Out.FrameAttribute->FrameRangeSet.GetEntryRangeOffset(EntryIdx, RangeIdx);
			const int32 FrameNum = Out.FrameAttribute->FrameRangeSet.GetEntryRangeLength(EntryIdx, RangeIdx);
			const int32 Sequence = Out.FrameAttribute->FrameRangeSet.GetEntrySequence(EntryIdx);
			const int32 StartFrame = Out.FrameAttribute->FrameRangeSet.GetEntryRangeStart(EntryIdx, RangeIdx);

			if (!Active.FrameRangeSet->IntersectsRange(Sequence, StartFrame, FrameNum)) { return; }

			const int32 ActiveEntry = Active.FrameRangeSet->FindSequenceEntry(Sequence);
			check(ActiveEntry != INDEX_NONE);

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				Out.FrameAttribute->AttributeData[0][FrameOffset + FrameIdx] = Active.FrameRangeSet->EntryContains(ActiveEntry, StartFrame + FrameIdx) ? 1.0f : 0.0f;
			}

		});

	UE::Learning::Array::Check(Out.FrameAttribute->AttributeData);

	return Out;
}

void UAnimDatabaseFrameAttributeLibrary::MakeBoolFrameAttributesFromActiveRanges(TArray<FAnimDatabaseFrameAttribute>& OutFrameAttributes, const TArray<FAnimDatabaseFrameRanges>& Actives, const FAnimDatabaseFrameRanges& FrameRanges)
{
	const int32 AttributeNum = Actives.Num();
	OutFrameAttributes.SetNum(AttributeNum);
	MakeBoolFrameAttributesFromActiveRangesArrayViews(OutFrameAttributes, Actives, FrameRanges);
}

void UAnimDatabaseFrameAttributeLibrary::MakeBoolFrameAttributesFromActiveRangesArrayViews(const TArrayView<FAnimDatabaseFrameAttribute> OutFrameAttributes, const TArrayView<const FAnimDatabaseFrameRanges> Actives, const FAnimDatabaseFrameRanges& FrameRanges)
{
	check(OutFrameAttributes.Num() == Actives.Num());

	const int32 AttributeNum = Actives.Num();
	ParallelFor(AttributeNum, [OutFrameAttributes, Actives, &FrameRanges](int32 AttributeIdx) {

		OutFrameAttributes[AttributeIdx] = MakeBoolFrameAttributeFromActiveRanges(Actives[AttributeIdx], FrameRanges);

	});
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeBoolFrameAttributeFromConstant(const bool bActive, const FAnimDatabaseFrameRanges& FrameRanges)
{
	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoolFrameAttributeFromConstant: FrameRanges is Invalid.");
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Bool;
	UE::Learning::FrameAttribute::Fill(*Out.FrameAttribute, *FrameRanges.FrameRangeSet, { bActive ? 1.0f : 0.0f });
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeBoolFrameAttributeFromTrue(const FAnimDatabaseFrameRanges& FrameRanges)
{
	return MakeBoolFrameAttributeFromConstant(true, FrameRanges);
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeBoolFrameAttributeFromFalse(const FAnimDatabaseFrameRanges& FrameRanges)
{
	return MakeBoolFrameAttributeFromConstant(false, FrameRanges);
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeRandomBoolFrameAttribute(int32& State, const FAnimDatabaseFrameRanges& FrameRanges)
{
	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeRandomBoolFrameAttribute: FrameRanges is Invalid.");
		return FAnimDatabaseFrameAttribute();
	}

	const int32 TotalFrameNum = FrameRanges.FrameRangeSet->GetTotalFrameNum();

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Bool;
	Out.FrameAttribute->FrameRangeSet = *FrameRanges.FrameRangeSet;
	Out.FrameAttribute->AttributeData.SetNumUninitialized({ AttributeTypeSize(Out.Type), TotalFrameNum });
	
	UE::Learning::Random::SampleFloatArray(Out.FrameAttribute->AttributeData[0], (uint32&)State);
	for (int32 FrameIdx = 0; FrameIdx < TotalFrameNum; FrameIdx++)
	{
		Out.FrameAttribute->AttributeData[0][FrameIdx] = Out.FrameAttribute->AttributeData[0][FrameIdx] > 0.5f ? 1.0f : 0.0f;
	}

	UE::Learning::Array::Check(Out.FrameAttribute->AttributeData);

	return Out;
}

void UAnimDatabaseFrameAttributeLibrary::MakeRandomBoolFrameAttributes(TArray<FAnimDatabaseFrameAttribute>& OutFrameAttributes, int32& State, const FAnimDatabaseFrameRanges& FrameRanges, const int32 FrameAttributeNum)
{
	OutFrameAttributes.Empty(FrameAttributeNum);
	for (int32 FrameAttributeIdx = 0; FrameAttributeIdx < FrameAttributeNum; FrameAttributeIdx++)
	{
		OutFrameAttributes.Add(MakeRandomBoolFrameAttribute(State, FrameRanges));
	}
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeBoolFrameAttributeFromAnimNotifyState(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TSubclassOf<UAnimNotifyState> NotifyState)
{
	return MakeBoolFrameAttributeFromActiveRanges(UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAnimNotifyState(Database, FrameRanges, NotifyState), FrameRanges);
}


void UAnimDatabaseFrameAttributeLibrary::MakeBoolFrameAttributesFromCurvesActive(TArray<FAnimDatabaseFrameAttribute>& OutBoolFrameAttributes, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArray<FName>& CurveNames)
{
	OutBoolFrameAttributes.SetNum(CurveNames.Num());
	MakeBoolFrameAttributesFromCurvesActiveArrayView(OutBoolFrameAttributes, Database, FrameRanges, CurveNames);
}

void UAnimDatabaseFrameAttributeLibrary::MakeBoolFrameAttributesFromCurvesActiveArrayView(const TArrayView<FAnimDatabaseFrameAttribute> OutBoolFrameAttributes, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const FName> CurveNames)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::MakeBoolFrameAttributesFromCurvesActiveArrayView);

	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoolFrameAttributesFromCurvesActiveArrayView: Database is nullptr.");
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutBoolFrameAttributes, FAnimDatabaseFrameAttribute());
		return;
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoolFrameAttributesFromCurvesActiveArrayView: FrameRanges is Invalid.");
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutBoolFrameAttributes, FAnimDatabaseFrameAttribute());
		return;
	}

	const int32 CurveNum = CurveNames.Num();
	check(OutBoolFrameAttributes.Num() == CurveNum);

	const int32 TotalFrameNum = FrameRanges.FrameRangeSet->GetTotalFrameNum();

	for (int32 CurveIdx = 0; CurveIdx < CurveNum; CurveIdx++)
	{
		OutBoolFrameAttributes[CurveIdx] = MakeEmptyFrameAttribute();
		OutBoolFrameAttributes[CurveIdx].Type = EAnimDatabaseAttributeType::Bool;
		OutBoolFrameAttributes[CurveIdx].FrameAttribute->FrameRangeSet = *FrameRanges.FrameRangeSet;
		OutBoolFrameAttributes[CurveIdx].FrameAttribute->AttributeData.SetNumUninitialized({ AttributeTypeSize(OutBoolFrameAttributes[CurveIdx].Type), TotalFrameNum });
	}

	TLearningArray<2, bool> CurveActives;
	CurveActives.SetNumUninitialized({ TotalFrameNum, CurveNum });

	Database->WaitForCompressionOnAnimSequencesFromArrayView(FrameRanges.FrameRangeSet->GetEntrySequences());

	UE::Learning::FrameRangeSet::ParallelForEachRange(*FrameRanges.FrameRangeSet, [&OutBoolFrameAttributes, &FrameRanges, &Database, &CurveActives, &CurveNames, CurveNum](
		const int32 TotalRangeIdx,
		const int32 EntryIdx,
		const int32 RangeIdx) {

			const int32 FrameOffset = FrameRanges.FrameRangeSet->GetEntryRangeOffset(EntryIdx, RangeIdx);
			const int32 FrameNum = FrameRanges.FrameRangeSet->GetEntryRangeLength(EntryIdx, RangeIdx);
			const int32 Sequence = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);
			const int32 StartFrame = FrameRanges.FrameRangeSet->GetEntryRangeStart(EntryIdx, RangeIdx);

			Database->GetCurveActiveData(
				CurveActives.Slice(FrameOffset, FrameNum),
				Sequence,
				StartFrame,
				CurveNames);

			for (int32 FrameIdx = FrameOffset; FrameIdx < FrameOffset + FrameNum; FrameIdx++)
			{
				for (int32 CurveIdx = 0; CurveIdx < CurveNum; CurveIdx++)
				{
					OutBoolFrameAttributes[CurveIdx].FrameAttribute->AttributeData[0][FrameIdx] = CurveActives[FrameIdx][CurveIdx] ? 1.0f : 0.0f;
				}
			}
		});

	for (int32 CurveIdx = 0; CurveIdx < CurveNum; CurveIdx++)
	{
		UE::Learning::Array::Check(OutBoolFrameAttributes[CurveIdx].FrameAttribute->AttributeData);
	}
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeBoolFrameAttributeFromCurveActive(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName CurveName)
{
	FAnimDatabaseFrameAttribute Out;
	MakeBoolFrameAttributesFromCurvesActiveArrayView(MakeArrayView(&Out, 1), Database, FrameRanges, { CurveName });
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeBoolFrameAttributeFromMirrored(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges)
{
	return MakeBoolFrameAttributeFromActiveRanges(UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromMirrored(Database, FrameRanges), FrameRanges);
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeBoolFrameAttributeFromNotMirrored(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges)
{
	return MakeBoolFrameAttributeFromActiveRanges(UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromNotMirrored(Database, FrameRanges), FrameRanges);
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeEventFrameAttribute(const FAnimDatabaseFrames& EventFrames, const FAnimDatabaseFrameRanges& FrameRanges, const FFrameRate& FrameRate)
{
	if (!EventFrames.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeEventFrameAttribute: EventFrames is invalid.");
		return FAnimDatabaseFrameAttribute();
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeEventFrameAttribute: FrameRanges is invalid.");
		return FAnimDatabaseFrameAttribute();
	}

	const int32 TotalFrameNum = FrameRanges.FrameRangeSet->GetTotalFrameNum();

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Event;
	Out.FrameAttribute->FrameRangeSet = *FrameRanges.FrameRangeSet;
	Out.FrameAttribute->AttributeData.SetNumUninitialized({ AttributeTypeSize(Out.Type), TotalFrameNum });

	UE::Learning::FrameRangeSet::ParallelForEachRange(Out.FrameAttribute->FrameRangeSet, [&Out, &EventFrames, FrameRate](
		const int32 TotalRangeIdx,
		const int32 EntryIdx,
		const int32 RangeIdx) {

			const int32 FrameOffset = Out.FrameAttribute->FrameRangeSet.GetEntryRangeOffset(EntryIdx, RangeIdx);
			const int32 FrameNum = Out.FrameAttribute->FrameRangeSet.GetEntryRangeLength(EntryIdx, RangeIdx);
			const int32 Sequence = Out.FrameAttribute->FrameRangeSet.GetEntrySequence(EntryIdx);
			const int32 StartFrame = Out.FrameAttribute->FrameRangeSet.GetEntryRangeStart(EntryIdx, RangeIdx);

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				int32 NearestEntryIdx = INDEX_NONE;
				int32 NearestFrameIdx = INDEX_NONE;
				int32 NearestFrameDifference = INT32_MAX;
				bool bNearestFound = EventFrames.FrameSet->FindNearestInRange(
					NearestEntryIdx,
					NearestFrameIdx,
					NearestFrameDifference,
					Sequence,
					StartFrame + FrameIdx,
					StartFrame,
					FrameNum);

				if (bNearestFound)
				{
					Out.FrameAttribute->AttributeData[0][FrameOffset + FrameIdx] = 1.0f;
					Out.FrameAttribute->AttributeData[1][FrameOffset + FrameIdx] = NearestFrameDifference / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER);
				}
				else
				{
					Out.FrameAttribute->AttributeData[0][FrameOffset + FrameIdx] = 0.0f;
					Out.FrameAttribute->AttributeData[1][FrameOffset + FrameIdx] = UE_MAX_FLT;
				}
			}
		});

	UE::Learning::Array::Check(Out.FrameAttribute->AttributeData);

	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeEventFrameAttributeAtRangeStarts(const FAnimDatabaseFrameRanges& EventFrameRanges, const FAnimDatabaseFrameRanges& FrameRanges, const FFrameRate& FrameRate)
{
	return MakeEventFrameAttribute(UAnimDatabaseFrameRangesLibrary::MakeFramesAtFrameRangesStarts(EventFrameRanges), FrameRanges, FrameRate);
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeEventFrameAttributeAtRangeEnds(const FAnimDatabaseFrameRanges& EventFrameRanges, const FAnimDatabaseFrameRanges& FrameRanges, const FFrameRate& FrameRate)
{
	return MakeEventFrameAttribute(UAnimDatabaseFrameRangesLibrary::MakeFramesAtFrameRangesEnds(EventFrameRanges), FrameRanges, FrameRate);
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeEventFrameAttributeFromAnimNotify(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TSubclassOf<UAnimNotify> Notify)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeEventFrameAttributeFromAnimNotify: Database is invalid.");
		return FAnimDatabaseFrameAttribute();
	}

	return MakeEventFrameAttribute(UAnimDatabaseFramesLibrary::MakeFramesFromAnimNotify(Database, FrameRanges, Notify), FrameRanges, Database->GetFrameRate());
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeEventFrameAttributeFromAnimNotifyUnion(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArray<TSubclassOf<UAnimNotify>>& Notifies)
{
	return MakeEventFrameAttributeFromAnimNotifyUnionArrayView(Database, FrameRanges, Notifies);
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeEventFrameAttributeFromAnimNotifyUnionArrayView(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const TSubclassOf<UAnimNotify>> Notifies)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeEventFrameAttributeFromAnimNotifyUnionArrayView: Database is invalid.");
		return FAnimDatabaseFrameAttribute();
	}

	return MakeEventFrameAttribute(UAnimDatabaseFramesLibrary::MakeFramesFromAnimNotifyArrayViewUnion(Database, FrameRanges, Notifies), FrameRanges, Database->GetFrameRate());
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeEventFrameAttributeFromAnimNotifyStateStarts(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TSubclassOf<UAnimNotifyState> NotifyState)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeEventFrameAttributeFromAnimNotifyStateStarts: Database is invalid.");
		return FAnimDatabaseFrameAttribute();
	}

	return MakeEventFrameAttribute(UAnimDatabaseFramesLibrary::MakeFramesFromAnimNotifyStateStart(Database, FrameRanges, NotifyState), FrameRanges, Database->GetFrameRate());
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeEventFrameAttributeFromAnimNotifyStateEnds(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TSubclassOf<UAnimNotifyState> NotifyState)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeEventFrameAttributeFromAnimNotifyStateEnds: Database is invalid.");
		return FAnimDatabaseFrameAttribute();
	}

	return MakeEventFrameAttribute(UAnimDatabaseFramesLibrary::MakeFramesFromAnimNotifyStateEnd(Database, FrameRanges, NotifyState), FrameRanges, Database->GetFrameRate());
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeTransformFrameAttributeFromConstant(const FAnimDatabaseFrameRanges& FrameRanges, const FTransform Transform)
{
	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeTransformFrameAttributeFromConstant: Invalid FrameRanges.");
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Transform;

	UE::Learning::FrameAttribute::Fill(
		*Out.FrameAttribute,
		*FrameRanges.FrameRangeSet,
		{
			(float)Transform.GetLocation().X, (float)Transform.GetLocation().Y, (float)Transform.GetLocation().Z,
			(float)Transform.GetRotation().X, (float)Transform.GetRotation().Y, (float)Transform.GetRotation().Z, (float)Transform.GetRotation().W,
			(float)Transform.GetScale3D().X, (float)Transform.GetScale3D().Y, (float)Transform.GetScale3D().Z,
		});

	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeTransformFrameAttributeFromIdentity(const FAnimDatabaseFrameRanges& FrameRanges)
{
	return MakeTransformFrameAttributeFromConstant(FrameRanges, FTransform::Identity);
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeTransformFrameAttribute(const FAnimDatabaseFrameAttribute& Location, const FAnimDatabaseFrameAttribute& Rotation, const FAnimDatabaseFrameAttribute& Scale)
{
	if (!Location.IsValid() || !Rotation.IsValid() || !Scale.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeTransformFrameAttribute: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (Location.Type != EAnimDatabaseAttributeType::Location || Rotation.Type != EAnimDatabaseAttributeType::Rotation || Scale.Type != EAnimDatabaseAttributeType::Scale)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeTransformFrameAttribute: Invalid FrameAttribute Types. Got {Type0}, {Type1}, and {Type2}, Expected Location, Rotation, and Scale.", AttributeTypeNameInternal(Location.Type), AttributeTypeNameInternal(Rotation.Type), AttributeTypeNameInternal(Scale.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Transform;
	UE::Learning::FrameAttribute::TransformMake(*Out.FrameAttribute, *Location.FrameAttribute, *Rotation.FrameAttribute, *Scale.FrameAttribute);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeTransformFrameAttributeNoScale(const FAnimDatabaseFrameAttribute& Location, const FAnimDatabaseFrameAttribute& Rotation)
{
	if (!Location.IsValid() || !Rotation.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeTransformFrameAttributeNoScale: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (Location.Type != EAnimDatabaseAttributeType::Location || Rotation.Type != EAnimDatabaseAttributeType::Rotation)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeTransformFrameAttributeNoScale: Invalid FrameAttribute Types. Got {Type0} and {Type1}, Expected Location and Rotation.", AttributeTypeNameInternal(Location.Type), AttributeTypeNameInternal(Rotation.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Transform;
	UE::Learning::FrameAttribute::TransformMake(*Out.FrameAttribute, *Location.FrameAttribute, *Rotation.FrameAttribute);
	return Out;
}

void UAnimDatabaseFrameAttributeLibrary::MakeFloatFrameAttributesFromCurves(TArray<FAnimDatabaseFrameAttribute>& OutFloatFrameAttributes, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArray<FName>& CurveNames)
{
	OutFloatFrameAttributes.SetNum(CurveNames.Num());
	MakeFloatFrameAttributesFromCurvesArrayView(OutFloatFrameAttributes, Database, FrameRanges, CurveNames);
}

void UAnimDatabaseFrameAttributeLibrary::MakeFloatFrameAttributesFromCurvesArrayView(const TArrayView<FAnimDatabaseFrameAttribute> OutFloatFrameAttributes, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const FName> CurveNames)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::MakeFloatFrameAttributeFromCurve);

	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFloatFrameAttributesFromCurvesArrayView: Database is nullptr.");
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutFloatFrameAttributes, FAnimDatabaseFrameAttribute());
		return;
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFloatFrameAttributesFromCurvesArrayView: FrameRanges is Invalid.");
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutFloatFrameAttributes, FAnimDatabaseFrameAttribute());
		return;
	}

	const int32 CurveNum = CurveNames.Num();
	check(OutFloatFrameAttributes.Num() == CurveNum);

	const int32 TotalFrameNum = FrameRanges.FrameRangeSet->GetTotalFrameNum();

	for (int32 CurveIdx = 0; CurveIdx < CurveNum; CurveIdx++)
	{
		OutFloatFrameAttributes[CurveIdx] = MakeEmptyFrameAttribute();
		OutFloatFrameAttributes[CurveIdx].Type = EAnimDatabaseAttributeType::Float;
		OutFloatFrameAttributes[CurveIdx].FrameAttribute->FrameRangeSet = *FrameRanges.FrameRangeSet;
		OutFloatFrameAttributes[CurveIdx].FrameAttribute->AttributeData.SetNumUninitialized({ AttributeTypeSize(OutFloatFrameAttributes[CurveIdx].Type), TotalFrameNum });
	}

	TLearningArray<2, float> CurveValues;
	TLearningArray<2, float> CurveVelocities;
	TLearningArray<2, bool> CurveActives;
	CurveValues.SetNumUninitialized({ TotalFrameNum, CurveNum });
	CurveVelocities.SetNumUninitialized({ TotalFrameNum, CurveNum });
	CurveActives.SetNumUninitialized({ TotalFrameNum, CurveNum });

	Database->WaitForCompressionOnAnimSequencesFromArrayView(FrameRanges.FrameRangeSet->GetEntrySequences());

	UE::Learning::FrameRangeSet::ParallelForEachRange(*FrameRanges.FrameRangeSet, [&OutFloatFrameAttributes, &FrameRanges, &Database, &CurveValues, &CurveVelocities, &CurveActives, &CurveNames, CurveNum](
		const int32 TotalRangeIdx,
		const int32 EntryIdx,
		const int32 RangeIdx) {

			const int32 FrameOffset = FrameRanges.FrameRangeSet->GetEntryRangeOffset(EntryIdx, RangeIdx);
			const int32 FrameNum = FrameRanges.FrameRangeSet->GetEntryRangeLength(EntryIdx, RangeIdx);
			const int32 Sequence = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);
			const int32 StartFrame = FrameRanges.FrameRangeSet->GetEntryRangeStart(EntryIdx, RangeIdx);

			Database->GetCurveData(
				CurveValues.Slice(FrameOffset, FrameNum),
				CurveVelocities.Slice(FrameOffset, FrameNum),
				CurveActives.Slice(FrameOffset, FrameNum),
				Sequence,
				StartFrame,
				CurveNames);

			for (int32 FrameIdx = FrameOffset; FrameIdx < FrameOffset + FrameNum; FrameIdx++)
			{
				for (int32 CurveIdx = 0; CurveIdx < CurveNum; CurveIdx++)
				{
					OutFloatFrameAttributes[CurveIdx].FrameAttribute->AttributeData[0][FrameIdx] = CurveValues[FrameIdx][CurveIdx];
				}
			}
		});

	for (int32 CurveIdx = 0; CurveIdx < CurveNum; CurveIdx++)
	{
		UE::Learning::Array::Check(OutFloatFrameAttributes[CurveIdx].FrameAttribute->AttributeData);
	}
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeFloatFrameAttributeFromCurve(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName CurveName)
{
	FAnimDatabaseFrameAttribute Out;
	MakeFloatFrameAttributesFromCurvesArrayView(MakeArrayView(&Out, 1), Database, FrameRanges, { CurveName });
	return Out;
}

void UAnimDatabaseFrameAttributeLibrary::MakeFloatFrameAttributesFromCurvesWhenActive(TArray<FAnimDatabaseFrameAttribute>& OutFloatFrameAttributes, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArray<FName>& CurveNames)
{
	OutFloatFrameAttributes.SetNum(CurveNames.Num());
	MakeFloatFrameAttributesFromCurvesWhenActiveArrayView(OutFloatFrameAttributes, Database, FrameRanges, CurveNames);
}

void UAnimDatabaseFrameAttributeLibrary::MakeFloatFrameAttributesFromCurvesWhenActiveArrayView(const TArrayView<FAnimDatabaseFrameAttribute> OutFloatFrameAttributes, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const FName> CurveNames)
{
	const int32 CurveNum = CurveNames.Num();
	check(OutFloatFrameAttributes.Num() == CurveNum);

	TArray<FAnimDatabaseFrameAttribute, TInlineAllocator<8>> CurveValue;
	TArray<FAnimDatabaseFrameAttribute, TInlineAllocator<8>> CurveVelocity;
	TArray<FAnimDatabaseFrameAttribute, TInlineAllocator<8>> CurveActive;
	CurveValue.SetNum(CurveNum);
	CurveVelocity.SetNum(CurveNum);
	CurveActive.SetNum(CurveNum);

	MakeFrameAttributesFromCurvesArrayViews(CurveValue, CurveVelocity, CurveActive, Database, FrameRanges, CurveNames);

	for (int32 CurveIdx = 0; CurveIdx < CurveNum; CurveIdx++)
	{
		OutFloatFrameAttributes[CurveIdx] = FrameAttributeIntersection(CurveValue[CurveIdx], FrameAttributeBoolToFrameRanges(CurveActive[CurveIdx]));
	}
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeFloatFrameAttributeFromCurveWhenActive(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName CurveName)
{
	FAnimDatabaseFrameAttribute Out;
	MakeFloatFrameAttributesFromCurvesWhenActiveArrayView(MakeArrayView(&Out, 1), Database, FrameRanges, { CurveName });
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeFloatFrameAttributeFromRangeTimes(const FAnimDatabaseFrameRanges& FrameRanges, const FFrameRate& FrameRate)
{
	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFloatFrameAttributeFromRangeTimes: Invalid FrameRanges.");
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Float;
	Out.FrameAttribute->FrameRangeSet = *FrameRanges.FrameRangeSet;
	Out.FrameAttribute->AttributeData.SetNumUninitialized({ AttributeTypeSize(Out.Type), FrameRanges.FrameRangeSet->GetTotalFrameNum() });

	UE::Learning::FrameRangeSet::ParallelForEachRange(Out.FrameAttribute->FrameRangeSet, [&Out, FrameRate](
		const int32 TotalRangeIdx,
		const int32 EntryIdx,
		const int32 RangeIdx) {

			const int32 FrameOffset = Out.FrameAttribute->FrameRangeSet.GetEntryRangeOffset(EntryIdx, RangeIdx);
			const int32 FrameNum = Out.FrameAttribute->FrameRangeSet.GetEntryRangeLength(EntryIdx, RangeIdx);

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				Out.FrameAttribute->AttributeData[0][FrameOffset + FrameIdx] = FrameIdx / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER);
			}
		});

	UE::Learning::Array::Check(Out.FrameAttribute->AttributeData);

	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeFloatFrameAttributeFromSequenceTimes(const FAnimDatabaseFrameRanges& FrameRanges, const FFrameRate& FrameRate)
{
	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFloatFrameAttributeFromSequenceTimes: Invalid FrameRanges.");
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Float;
	Out.FrameAttribute->FrameRangeSet = *FrameRanges.FrameRangeSet;
	Out.FrameAttribute->AttributeData.SetNumUninitialized({ AttributeTypeSize(Out.Type), FrameRanges.FrameRangeSet->GetTotalFrameNum() });

	UE::Learning::FrameRangeSet::ParallelForEachRange(Out.FrameAttribute->FrameRangeSet, [&Out, FrameRate](
		const int32 TotalRangeIdx,
		const int32 EntryIdx,
		const int32 RangeIdx) {

			const int32 FrameStart = Out.FrameAttribute->FrameRangeSet.GetEntryRangeStart(EntryIdx, RangeIdx);
			const int32 FrameOffset = Out.FrameAttribute->FrameRangeSet.GetEntryRangeOffset(EntryIdx, RangeIdx);
			const int32 FrameNum = Out.FrameAttribute->FrameRangeSet.GetEntryRangeLength(EntryIdx, RangeIdx);

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				Out.FrameAttribute->AttributeData[0][FrameOffset + FrameIdx] = (FrameStart + FrameIdx) / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER);
			}
		});

	UE::Learning::Array::Check(Out.FrameAttribute->AttributeData);

	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeFloatFrameAttributeFromNearestFrameSequenceTimes(const FAnimDatabaseFrameRanges& FrameRanges, const FAnimDatabaseFrames& Frames, const FFrameRate& FrameRate)
{
	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFloatFrameAttributeFromNearestFrameSequenceTimes: Invalid FrameRanges.");
		return FAnimDatabaseFrameAttribute();
	}

	if (!Frames.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFloatFrameAttributeFromNearestFrameSequenceTimes: Invalid Frames.");
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Float;
	Out.FrameAttribute->FrameRangeSet = *FrameRanges.FrameRangeSet;
	Out.FrameAttribute->AttributeData.SetNumUninitialized({ AttributeTypeSize(Out.Type), FrameRanges.FrameRangeSet->GetTotalFrameNum() });

	UE::Learning::FrameRangeSet::ParallelForEachRange(Out.FrameAttribute->FrameRangeSet, [&Out, &Frames, FrameRate](
		const int32 TotalRangeIdx,
		const int32 EntryIdx,
		const int32 RangeIdx) {

			const int32 RangeSequence = Out.FrameAttribute->FrameRangeSet.GetEntrySequence(EntryIdx);
			const int32 FrameStart = Out.FrameAttribute->FrameRangeSet.GetEntryRangeStart(EntryIdx, RangeIdx);
			const int32 FrameOffset = Out.FrameAttribute->FrameRangeSet.GetEntryRangeOffset(EntryIdx, RangeIdx);
			const int32 FrameNum = Out.FrameAttribute->FrameRangeSet.GetEntryRangeLength(EntryIdx, RangeIdx);

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				int32 NearestEntry = INDEX_NONE, NearestFrame = INDEX_NONE, FrameDifference = INDEX_NONE, NearestSequence = INDEX_NONE;
				if (Frames.FrameSet->FindNearestInRange(NearestEntry, NearestFrame, FrameDifference, RangeSequence, FrameStart + FrameIdx, FrameStart, FrameNum))
				{
					const int32 MatchingFrame = Frames.FrameSet->GetEntryFrame(NearestEntry, NearestFrame);
					Out.FrameAttribute->AttributeData[0][FrameOffset + FrameIdx] = MatchingFrame / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER);
				}
				else
				{
					Out.FrameAttribute->AttributeData[0][FrameOffset + FrameIdx] = 0.0f;
				}
			}
		});

	UE::Learning::Array::Check(Out.FrameAttribute->AttributeData);

	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeFloatFrameAttributeFromConstant(const FAnimDatabaseFrameRanges& FrameRanges, const float Value)
{
	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFloatFrameAttributeFromConstant: Invalid FrameRanges.");
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Float;
	UE::Learning::FrameAttribute::Fill(*Out.FrameAttribute, *FrameRanges.FrameRangeSet, { Value });
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeFloatFrameAttributeFromZero(const FAnimDatabaseFrameRanges& FrameRanges)
{
	return MakeFloatFrameAttributeFromConstant(FrameRanges, 0.0f);
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeFloatFrameAttributeFromOne(const FAnimDatabaseFrameRanges& FrameRanges)
{
	return MakeFloatFrameAttributeFromConstant(FrameRanges, 1.0f);
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeFloatFrameAttributeFromUniformRandom(const FAnimDatabaseFrameRanges& FrameRanges, int32& State, const float Min, const float Max)
{
	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFloatFrameAttributeFromUniformRandom: Invalid FrameRanges.");
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Float;
	Out.FrameAttribute->FrameRangeSet = *FrameRanges.FrameRangeSet;
	Out.FrameAttribute->AttributeData.SetNumUninitialized({ AttributeTypeSize(Out.Type), FrameRanges.FrameRangeSet->GetTotalFrameNum() });
	uint32 RandomState = State;
	UE::Learning::Random::SampleUniformArray(Out.FrameAttribute->AttributeData.Flatten(), RandomState, Min, Max);
	State = RandomState;
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeLocationFrameAttribute(const FAnimDatabaseFrameAttribute& X, const FAnimDatabaseFrameAttribute& Y, const FAnimDatabaseFrameAttribute& Z)
{
	if (!X.IsValid() || !Y.IsValid() || !Z.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeLocationFrameAttribute: Invalid Frame Attribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (X.Type != EAnimDatabaseAttributeType::Float ||
		Y.Type != EAnimDatabaseAttributeType::Float ||
		Z.Type != EAnimDatabaseAttributeType::Float)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeLocationFrameAttribute: FrameAttribute Types don't match. Got {Type}, {Type}, and {Type}. Expected Floats.",
			AttributeTypeNameInternal(X.Type), AttributeTypeNameInternal(Y.Type), AttributeTypeNameInternal(Z.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Location;
	UE::Learning::FrameAttribute::Concat(*Out.FrameAttribute, { X.FrameAttribute.Get(), Y.FrameAttribute.Get(), Z.FrameAttribute.Get() });
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeLocationFrameAttributeFromConstant(const FAnimDatabaseFrameRanges& FrameRanges, const FVector Location)
{
	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeLocationFrameAttributeFromConstant: Invalid FrameRanges.");
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Location;
	UE::Learning::FrameAttribute::Fill(*Out.FrameAttribute, *FrameRanges.FrameRangeSet, { (float)Location.X, (float)Location.Y, (float)Location.Z });
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeRotationFrameAttribute(const FAnimDatabaseFrameAttribute& Pitch, const FAnimDatabaseFrameAttribute& Yaw, const FAnimDatabaseFrameAttribute& Roll)
{
	if (!Pitch.IsValid() || !Yaw.IsValid() || !Roll.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeRotationFrameAttribute: Invalid Frame Attribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (Pitch.Type != EAnimDatabaseAttributeType::Angle ||
		Yaw.Type != EAnimDatabaseAttributeType::Angle ||
		Roll.Type != EAnimDatabaseAttributeType::Angle)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeRotationFrameAttribute: FrameAttribute Types don't match. Got {Type}, {Type}, and {Type}. Expected Angles.",
			AttributeTypeNameInternal(Pitch.Type), AttributeTypeNameInternal(Yaw.Type), AttributeTypeNameInternal(Roll.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Rotation;
	UE::Learning::FrameAttribute::QuatFromPitchYawRoll(*Out.FrameAttribute, *Pitch.FrameAttribute, *Yaw.FrameAttribute, *Roll.FrameAttribute);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeRotationFrameAttributeFromConstant(const FAnimDatabaseFrameRanges& FrameRanges, const FRotator Rotation)
{
	return MakeRotationFrameAttributeFromConstantQuat(FrameRanges, Rotation.Quaternion());
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeRotationFrameAttributeFromConstantQuat(const FAnimDatabaseFrameRanges& FrameRanges, const FQuat Rotation)
{
	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeRotationFrameAttributeFromConstantQuat: Invalid FrameRanges.");
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Rotation;
	UE::Learning::FrameAttribute::Fill(*Out.FrameAttribute, *FrameRanges.FrameRangeSet, { (float)Rotation.X, (float)Rotation.Y, (float)Rotation.Z, (float)Rotation.W });
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeRotationFrameAttributeFromBasisDirections(const FAnimDatabaseFrameAttribute& ForwardDirection, const FAnimDatabaseFrameAttribute& RightDirection, const FAnimDatabaseFrameAttribute& UpDirection)
{
	if (!ForwardDirection.IsValid() || !RightDirection.IsValid() || !UpDirection.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeRotationFrameAttributeFromBasisDirections: Invalid Frame Attribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (ForwardDirection.Type != EAnimDatabaseAttributeType::Direction ||
		RightDirection.Type != EAnimDatabaseAttributeType::Direction ||
		UpDirection.Type != EAnimDatabaseAttributeType::Direction)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeRotationFrameAttributeFromBasisDirections: FrameAttribute Types don't match. Got {Type}, {Type}, and {Type}. Expected Directions.", 
			AttributeTypeNameInternal(ForwardDirection.Type), AttributeTypeNameInternal(RightDirection.Type), AttributeTypeNameInternal(UpDirection.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Rotation;
	UE::Learning::FrameAttribute::QuatFromBasisDirections(*Out.FrameAttribute, *ForwardDirection.FrameAttribute, *RightDirection.FrameAttribute, *UpDirection.FrameAttribute);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeDirectionFrameAttribute(const FAnimDatabaseFrameAttribute& X, const FAnimDatabaseFrameAttribute& Y, const FAnimDatabaseFrameAttribute& Z)
{
	if (!X.IsValid() || !Y.IsValid() || !Z.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeDirectionFrameAttribute: Invalid Frame Attribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (X.Type != EAnimDatabaseAttributeType::Float ||
		Y.Type != EAnimDatabaseAttributeType::Float ||
		Z.Type != EAnimDatabaseAttributeType::Float)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeDirectionFrameAttribute: FrameAttribute Types don't match. Got {Type}, {Type}, and {Type}. Expected Floats.",
			AttributeTypeNameInternal(X.Type), AttributeTypeNameInternal(Y.Type), AttributeTypeNameInternal(Z.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Direction;
	UE::Learning::FrameAttribute::Concat(*Out.FrameAttribute, { X.FrameAttribute.Get(), Y.FrameAttribute.Get(), Z.FrameAttribute.Get() });
	UE::Learning::FrameAttribute::NormalizeInplace(*Out.FrameAttribute);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeDirectionFrameAttributeFromConstant(const FAnimDatabaseFrameRanges& FrameRanges, const FVector Direction)
{
	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeDirectionFrameAttributeFromConstant: Invalid FrameRanges.");
		return FAnimDatabaseFrameAttribute();
	}

	const FVector SafeDirection = Direction.GetSafeNormal(UE_SMALL_NUMBER, FVector::ForwardVector);

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Direction;
	UE::Learning::FrameAttribute::Fill(*Out.FrameAttribute, *FrameRanges.FrameRangeSet, { (float)SafeDirection.X, (float)SafeDirection.Y, (float)SafeDirection.Z });
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeScaleFrameAttribute(const FAnimDatabaseFrameAttribute& X, const FAnimDatabaseFrameAttribute& Y, const FAnimDatabaseFrameAttribute& Z)
{
	if (!X.IsValid() || !Y.IsValid() || !Z.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeScaleFrameAttribute: Invalid Frame Attribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (X.Type != EAnimDatabaseAttributeType::Float ||
		Y.Type != EAnimDatabaseAttributeType::Float ||
		Z.Type != EAnimDatabaseAttributeType::Float)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeScaleFrameAttribute: FrameAttribute Types don't match. Got {Type}, {Type}, and {Type}. Expected Floats.",
			AttributeTypeNameInternal(X.Type), AttributeTypeNameInternal(Y.Type), AttributeTypeNameInternal(Z.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Scale;
	UE::Learning::FrameAttribute::Concat(*Out.FrameAttribute, { X.FrameAttribute.Get(), Y.FrameAttribute.Get(), Z.FrameAttribute.Get() });
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeScaleFrameAttributeFromConstant(const FAnimDatabaseFrameRanges& FrameRanges, const FVector Scale)
{
	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeScaleFrameAttributeFromConstant: Invalid FrameRanges.");
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Scale;
	UE::Learning::FrameAttribute::Fill(*Out.FrameAttribute, *FrameRanges.FrameRangeSet, { (float)Scale.X, (float)Scale.Y, (float)Scale.Z });
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeAngleFrameAttributeFromConstantDegrees(const FAnimDatabaseFrameRanges& FrameRanges, const float Value)
{
	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeAngleFrameAttributeFromConstantDegrees: Invalid FrameRanges.");
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Angle;
	UE::Learning::FrameAttribute::Fill(*Out.FrameAttribute, *FrameRanges.FrameRangeSet, { FMath::DegreesToRadians(Value) });
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeAngleFrameAttributeFromConstantRadians(const FAnimDatabaseFrameRanges& FrameRanges, const float Value)
{
	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeAngleFrameAttributeFromConstantRadians: Invalid FrameRanges.");
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Angle;
	UE::Learning::FrameAttribute::Fill(*Out.FrameAttribute, *FrameRanges.FrameRangeSet, { Value });
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeAngleFrameAttributeFromFloatDegrees(const FAnimDatabaseFrameAttribute& A)
{
	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeAngleFrameAttributeFromFloatDegrees: Invalid Frame Attribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type != EAnimDatabaseAttributeType::Float)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeAngleFrameAttributeFromFloatDegrees: FrameAttribute Types don't match. Got {Type}. Expected Float.", AttributeTypeNameInternal(A.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = FrameAttributeCopy(A);
	Out.Type = EAnimDatabaseAttributeType::Angle;
	UE::Learning::FrameAttribute::MulConstantInplace(*Out.FrameAttribute, { UE_PI / 180.f });
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeAngleFrameAttributeFromFloatRadians(const FAnimDatabaseFrameAttribute& A)
{
	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeAngleFrameAttributeFromFloatRadians: Invalid Frame Attribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type != EAnimDatabaseAttributeType::Float)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeAngleFrameAttributeFromFloatRadians: FrameAttribute Types don't match. Got {Type}. Expected Float.", AttributeTypeNameInternal(A.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = FrameAttributeCopy(A);
	Out.Type = EAnimDatabaseAttributeType::Angle;
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeRootLocationFrameAttribute(
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const float RelativeTime)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::MakeRootLocationFrameAttribute);

	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeRootLocationFrameAttribute: Database is nullptr.");
		return FAnimDatabaseFrameAttribute();
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeRootLocationFrameAttribute: Invalid FrameRanges.");
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Location;
	Out.FrameAttribute->FrameRangeSet = *FrameRanges.FrameRangeSet;
	Out.FrameAttribute->AttributeData.SetNumUninitialized({ AttributeTypeSize(Out.Type), FrameRanges.FrameRangeSet->GetTotalFrameNum() });

	Database->WaitForCompressionOnAnimSequencesFromArrayView(FrameRanges.FrameRangeSet->GetEntrySequences());

	UE::Learning::FrameRangeSet::ParallelForEachRange(Out.FrameAttribute->FrameRangeSet, [&Out, &Database, RelativeTime](
		const int32 TotalRangeIdx,
		const int32 EntryIdx,
		const int32 RangeIdx) {

			const int32 FrameOffset = Out.FrameAttribute->FrameRangeSet.GetEntryRangeOffset(EntryIdx, RangeIdx);
			const int32 FrameNum = Out.FrameAttribute->FrameRangeSet.GetEntryRangeLength(EntryIdx, RangeIdx);
			const int32 Sequence = Out.FrameAttribute->FrameRangeSet.GetEntrySequence(EntryIdx);
			const int32 StartFrame = Out.FrameAttribute->FrameRangeSet.GetEntryRangeStart(EntryIdx, RangeIdx) + FMath::RoundToInt(RelativeTime * Database->GetFrameRate().AsDecimal());
			
			TLearningArray<1, FVector> RootLocations;
			RootLocations.SetNumUninitialized({ FrameNum });
			Database->GetRootLocation(RootLocations, Sequence, StartFrame);

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				Out.FrameAttribute->AttributeData[0][FrameOffset + FrameIdx] = RootLocations[FrameIdx].X;
				Out.FrameAttribute->AttributeData[1][FrameOffset + FrameIdx] = RootLocations[FrameIdx].Y;
				Out.FrameAttribute->AttributeData[2][FrameOffset + FrameIdx] = RootLocations[FrameIdx].Z;
			}
		});

	UE::Learning::Array::Check(Out.FrameAttribute->AttributeData);

	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeRootDirectionFrameAttribute(
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const float RelativeTime,
	const FVector ForwardVector)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::MakeRootDirectionFrameAttribute);

	return FrameAttributeRotateDirection(MakeRootRotationFrameAttribute(Database, FrameRanges, RelativeTime), ForwardVector);
}

void UAnimDatabaseFrameAttributeLibrary::MakeRootLocationAndRotationFrameAttribute(
	FAnimDatabaseFrameAttribute& OutRootLocationFrameAttribute,
	FAnimDatabaseFrameAttribute& OutRootRotationFrameAttribute,
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const float RelativeTime)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::MakeRootLocationAndRotationFrameAttribute);

	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeRootLocationAndRotationFrameAttribute: Database is nullptr.");
		OutRootLocationFrameAttribute = FAnimDatabaseFrameAttribute();
		OutRootRotationFrameAttribute = FAnimDatabaseFrameAttribute();
		return;
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeRootLocationAndRotationFrameAttribute: Invalid FrameRanges.");
		OutRootLocationFrameAttribute = FAnimDatabaseFrameAttribute();
		OutRootRotationFrameAttribute = FAnimDatabaseFrameAttribute();
		return;
	}

	OutRootLocationFrameAttribute = MakeEmptyFrameAttribute();
	OutRootLocationFrameAttribute.Type = EAnimDatabaseAttributeType::Location;
	OutRootLocationFrameAttribute.FrameAttribute->FrameRangeSet = *FrameRanges.FrameRangeSet;
	OutRootLocationFrameAttribute.FrameAttribute->AttributeData.SetNumUninitialized({ AttributeTypeSize(OutRootLocationFrameAttribute.Type), FrameRanges.FrameRangeSet->GetTotalFrameNum() });

	OutRootRotationFrameAttribute = MakeEmptyFrameAttribute();
	OutRootRotationFrameAttribute.Type = EAnimDatabaseAttributeType::Rotation;
	OutRootRotationFrameAttribute.FrameAttribute->FrameRangeSet = *FrameRanges.FrameRangeSet;
	OutRootRotationFrameAttribute.FrameAttribute->AttributeData.SetNumUninitialized({ AttributeTypeSize(OutRootRotationFrameAttribute.Type), FrameRanges.FrameRangeSet->GetTotalFrameNum() });

	Database->WaitForCompressionOnAnimSequencesFromArrayView(FrameRanges.FrameRangeSet->GetEntrySequences());

	UE::Learning::FrameRangeSet::ParallelForEachRange(*FrameRanges.FrameRangeSet, [&FrameRanges, &OutRootLocationFrameAttribute, OutRootRotationFrameAttribute, &Database, RelativeTime](
		const int32 TotalRangeIdx,
		const int32 EntryIdx,
		const int32 RangeIdx) {

			const int32 FrameOffset = FrameRanges.FrameRangeSet->GetEntryRangeOffset(EntryIdx, RangeIdx);
			const int32 FrameNum = FrameRanges.FrameRangeSet->GetEntryRangeLength(EntryIdx, RangeIdx);
			const int32 Sequence = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);
			const int32 StartFrame = FrameRanges.FrameRangeSet->GetEntryRangeStart(EntryIdx, RangeIdx) + FMath::RoundToInt(RelativeTime * Database->GetFrameRate().AsDecimal());

			TLearningArray<1, FTransform> RootTransforms;
			RootTransforms.SetNumUninitialized({ FrameNum });
			Database->GetRootTransform(RootTransforms, Sequence, StartFrame);

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{

				OutRootLocationFrameAttribute.FrameAttribute->AttributeData[0][FrameOffset + FrameIdx] = RootTransforms[FrameIdx].GetLocation().X;
				OutRootLocationFrameAttribute.FrameAttribute->AttributeData[1][FrameOffset + FrameIdx] = RootTransforms[FrameIdx].GetLocation().Y;
				OutRootLocationFrameAttribute.FrameAttribute->AttributeData[2][FrameOffset + FrameIdx] = RootTransforms[FrameIdx].GetLocation().Z;

				OutRootRotationFrameAttribute.FrameAttribute->AttributeData[0][FrameOffset + FrameIdx] = RootTransforms[FrameIdx].GetRotation().X;
				OutRootRotationFrameAttribute.FrameAttribute->AttributeData[1][FrameOffset + FrameIdx] = RootTransforms[FrameIdx].GetRotation().Y;
				OutRootRotationFrameAttribute.FrameAttribute->AttributeData[2][FrameOffset + FrameIdx] = RootTransforms[FrameIdx].GetRotation().Z;
				OutRootRotationFrameAttribute.FrameAttribute->AttributeData[3][FrameOffset + FrameIdx] = RootTransforms[FrameIdx].GetRotation().W;
			}
		});

	UE::Learning::Array::Check(OutRootLocationFrameAttribute.FrameAttribute->AttributeData);
	UE::Learning::Array::Check(OutRootRotationFrameAttribute.FrameAttribute->AttributeData);
}

void  UAnimDatabaseFrameAttributeLibrary::MakeRootLocationAndDirectionFrameAttribute(
	FAnimDatabaseFrameAttribute& OutRootLocationFrameAttribute,
	FAnimDatabaseFrameAttribute& OutRootDirectionFrameAttribute,
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const float RelativeTime,
	const FVector ForwardVector)
{
	FAnimDatabaseFrameAttribute OutRootRotationFrameAttribute;
	MakeRootLocationAndRotationFrameAttribute(OutRootLocationFrameAttribute, OutRootRotationFrameAttribute, Database, FrameRanges, RelativeTime);
	OutRootDirectionFrameAttribute = FrameAttributeRotateDirection(OutRootRotationFrameAttribute, ForwardVector);
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeRootRotationFrameAttribute(
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const float RelativeTime)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::MakeRootRotationFrameAttribute);

	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeRootRotationFrameAttribute: Database is nullptr.");
		return FAnimDatabaseFrameAttribute();
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeRootRotationFrameAttribute: Invalid FrameRanges.");
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Rotation;
	Out.FrameAttribute->FrameRangeSet = *FrameRanges.FrameRangeSet;
	Out.FrameAttribute->AttributeData.SetNumUninitialized({ AttributeTypeSize(Out.Type), FrameRanges.FrameRangeSet->GetTotalFrameNum() });

	Database->WaitForCompressionOnAnimSequencesFromArrayView(FrameRanges.FrameRangeSet->GetEntrySequences());

	UE::Learning::FrameRangeSet::ParallelForEachRange(*FrameRanges.FrameRangeSet, [&FrameRanges, &Out, &Database, RelativeTime](
		const int32 TotalRangeIdx,
		const int32 EntryIdx,
		const int32 RangeIdx) {

			const int32 FrameOffset = FrameRanges.FrameRangeSet->GetEntryRangeOffset(EntryIdx, RangeIdx);
			const int32 FrameNum = FrameRanges.FrameRangeSet->GetEntryRangeLength(EntryIdx, RangeIdx);
			const int32 Sequence = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);
			const int32 StartFrame = FrameRanges.FrameRangeSet->GetEntryRangeStart(EntryIdx, RangeIdx) + FMath::RoundToInt(RelativeTime * Database->GetFrameRate().AsDecimal());

			TLearningArray<1, FQuat4f> RootRotations;
			RootRotations.SetNumUninitialized({ FrameNum });
			Database->GetRootRotation(RootRotations, Sequence, StartFrame);

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				Out.FrameAttribute->AttributeData[0][FrameOffset + FrameIdx] = RootRotations[FrameIdx].X;
				Out.FrameAttribute->AttributeData[1][FrameOffset + FrameIdx] = RootRotations[FrameIdx].Y;
				Out.FrameAttribute->AttributeData[2][FrameOffset + FrameIdx] = RootRotations[FrameIdx].Z;
				Out.FrameAttribute->AttributeData[3][FrameOffset + FrameIdx] = RootRotations[FrameIdx].W;
			}
		});

	UE::Learning::Array::Check(Out.FrameAttribute->AttributeData);

	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeRootLinearVelocityFrameAttribute(
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const float RelativeTime)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::MakeRootLinearVelocityFrameAttribute);

	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeRootLinearVelocityFrameAttribute: Database is nullptr.");
		return FAnimDatabaseFrameAttribute();
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeRootLinearVelocityFrameAttribute: Invalid FrameRanges.");
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::LinearVelocity;
	Out.FrameAttribute->FrameRangeSet = *FrameRanges.FrameRangeSet;
	Out.FrameAttribute->AttributeData.SetNumUninitialized({ AttributeTypeSize(Out.Type), FrameRanges.FrameRangeSet->GetTotalFrameNum() });

	Database->WaitForCompressionOnAnimSequencesFromArrayView(FrameRanges.FrameRangeSet->GetEntrySequences());

	UE::Learning::FrameRangeSet::ParallelForEachRange(*FrameRanges.FrameRangeSet, [&FrameRanges, &Out, &Database, RelativeTime](
		const int32 TotalRangeIdx,
		const int32 EntryIdx,
		const int32 RangeIdx) {

			const int32 FrameOffset = FrameRanges.FrameRangeSet->GetEntryRangeOffset(EntryIdx, RangeIdx);
			const int32 FrameNum = FrameRanges.FrameRangeSet->GetEntryRangeLength(EntryIdx, RangeIdx);
			const int32 Sequence = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);
			const int32 StartFrame = FrameRanges.FrameRangeSet->GetEntryRangeStart(EntryIdx, RangeIdx) + FMath::RoundToInt(RelativeTime * Database->GetFrameRate().AsDecimal());
			TLearningArray<1, FVector3f> RootVelocities;
			RootVelocities.SetNumUninitialized({ FrameNum });
			Database->GetRootLinearVelocity(RootVelocities, Sequence, StartFrame);

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				Out.FrameAttribute->AttributeData[0][FrameOffset + FrameIdx] = RootVelocities[FrameIdx].X;
				Out.FrameAttribute->AttributeData[1][FrameOffset + FrameIdx] = RootVelocities[FrameIdx].Y;
				Out.FrameAttribute->AttributeData[2][FrameOffset + FrameIdx] = RootVelocities[FrameIdx].Z;
			}
		});

	UE::Learning::Array::Check(Out.FrameAttribute->AttributeData);

	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeRootAngularVelocityFrameAttribute(
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const float RelativeTime)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::MakeRootAngularVelocityFrameAttribute);

	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeRootAngularVelocityFrameAttribute: Database is nullptr.");
		return FAnimDatabaseFrameAttribute();
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeRootAngularVelocityFrameAttribute: Invalid FrameRanges.");
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::AngularVelocity;
	Out.FrameAttribute->FrameRangeSet = *FrameRanges.FrameRangeSet;
	Out.FrameAttribute->AttributeData.SetNumUninitialized({ AttributeTypeSize(Out.Type), FrameRanges.FrameRangeSet->GetTotalFrameNum() });

	Database->WaitForCompressionOnAnimSequencesFromArrayView(FrameRanges.FrameRangeSet->GetEntrySequences());

	UE::Learning::FrameRangeSet::ParallelForEachRange(*FrameRanges.FrameRangeSet, [&FrameRanges, &Out, &Database, RelativeTime](
		const int32 TotalRangeIdx,
		const int32 EntryIdx,
		const int32 RangeIdx) {

			const int32 FrameOffset = FrameRanges.FrameRangeSet->GetEntryRangeOffset(EntryIdx, RangeIdx);
			const int32 FrameNum = FrameRanges.FrameRangeSet->GetEntryRangeLength(EntryIdx, RangeIdx);
			const int32 Sequence = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);
			const int32 StartFrame = FrameRanges.FrameRangeSet->GetEntryRangeStart(EntryIdx, RangeIdx) + FMath::RoundToInt(RelativeTime * Database->GetFrameRate().AsDecimal());
			TLearningArray<1, FVector3f> RootVelocities;
			RootVelocities.SetNumUninitialized({ FrameNum });
			Database->GetRootAngularVelocity(RootVelocities, Sequence, StartFrame);

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				Out.FrameAttribute->AttributeData[0][FrameOffset + FrameIdx] = RootVelocities[FrameIdx].X;
				Out.FrameAttribute->AttributeData[1][FrameOffset + FrameIdx] = RootVelocities[FrameIdx].Y;
				Out.FrameAttribute->AttributeData[2][FrameOffset + FrameIdx] = RootVelocities[FrameIdx].Z;
			}
		});

	UE::Learning::Array::Check(Out.FrameAttribute->AttributeData);

	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeRootTransformFrameAttribute(
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const float RelativeTime)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::MakeRootTransformFrameAttribute);

	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeRootTransformFrameAttribute: Database is nullptr.");
		return FAnimDatabaseFrameAttribute();
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeRootTransformFrameAttribute: Invalid FrameRanges.");
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Transform;
	Out.FrameAttribute->FrameRangeSet = *FrameRanges.FrameRangeSet;
	Out.FrameAttribute->AttributeData.SetNumUninitialized({ AttributeTypeSize(Out.Type), FrameRanges.FrameRangeSet->GetTotalFrameNum() });

	Database->WaitForCompressionOnAnimSequencesFromArrayView(FrameRanges.FrameRangeSet->GetEntrySequences());

	UE::Learning::FrameRangeSet::ParallelForEachRange(*FrameRanges.FrameRangeSet, [&FrameRanges, &Out, &Database, RelativeTime](
		const int32 TotalRangeIdx,
		const int32 EntryIdx,
		const int32 RangeIdx) {

			const int32 FrameOffset = FrameRanges.FrameRangeSet->GetEntryRangeOffset(EntryIdx, RangeIdx);
			const int32 FrameNum = FrameRanges.FrameRangeSet->GetEntryRangeLength(EntryIdx, RangeIdx);
			const int32 Sequence = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);
			const int32 StartFrame = FrameRanges.FrameRangeSet->GetEntryRangeStart(EntryIdx, RangeIdx) + FMath::RoundToInt(RelativeTime * Database->GetFrameRate().AsDecimal());

			TLearningArray<1, FTransform> RootTransforms;
			RootTransforms.SetNumUninitialized({ FrameNum });
			Database->GetRootTransform(RootTransforms, Sequence, StartFrame);

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				Out.FrameAttribute->AttributeData[0][FrameOffset + FrameIdx] = RootTransforms[FrameIdx].GetLocation().X;
				Out.FrameAttribute->AttributeData[1][FrameOffset + FrameIdx] = RootTransforms[FrameIdx].GetLocation().Y;
				Out.FrameAttribute->AttributeData[2][FrameOffset + FrameIdx] = RootTransforms[FrameIdx].GetLocation().Z;

				Out.FrameAttribute->AttributeData[3][FrameOffset + FrameIdx] = RootTransforms[FrameIdx].GetRotation().X;
				Out.FrameAttribute->AttributeData[4][FrameOffset + FrameIdx] = RootTransforms[FrameIdx].GetRotation().Y;
				Out.FrameAttribute->AttributeData[5][FrameOffset + FrameIdx] = RootTransforms[FrameIdx].GetRotation().Z;
				Out.FrameAttribute->AttributeData[6][FrameOffset + FrameIdx] = RootTransforms[FrameIdx].GetRotation().W;

				Out.FrameAttribute->AttributeData[7][FrameOffset + FrameIdx] = RootTransforms[FrameIdx].GetScale3D().X;
				Out.FrameAttribute->AttributeData[8][FrameOffset + FrameIdx] = RootTransforms[FrameIdx].GetScale3D().Y;
				Out.FrameAttribute->AttributeData[9][FrameOffset + FrameIdx] = RootTransforms[FrameIdx].GetScale3D().Z;
			}
		});

	UE::Learning::Array::Check(Out.FrameAttribute->AttributeData);

	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeRootTransformAtSequenceStartFrameAttribute(
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::MakeRootTransformAtSequenceStartFrameAttribute);

	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeRootTransformAtSequenceStartFrameAttribute: Database is nullptr.");
		return FAnimDatabaseFrameAttribute();
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeRootTransformAtSequenceStartFrameAttribute: Invalid FrameRanges.");
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Transform;
	Out.FrameAttribute->FrameRangeSet = *FrameRanges.FrameRangeSet;
	Out.FrameAttribute->AttributeData.SetNumUninitialized({ AttributeTypeSize(Out.Type), FrameRanges.FrameRangeSet->GetTotalFrameNum() });

	Database->WaitForCompressionOnAnimSequencesFromArrayView(FrameRanges.FrameRangeSet->GetEntrySequences());

	UE::Learning::FrameRangeSet::ParallelForEachRange(*FrameRanges.FrameRangeSet, [&FrameRanges, &Out, &Database](
		const int32 TotalRangeIdx,
		const int32 EntryIdx,
		const int32 RangeIdx) {

			const int32 FrameOffset = FrameRanges.FrameRangeSet->GetEntryRangeOffset(EntryIdx, RangeIdx);
			const int32 FrameNum = FrameRanges.FrameRangeSet->GetEntryRangeLength(EntryIdx, RangeIdx);
			const int32 Sequence = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);
			
			FTransform RootTransform;
			Database->GetRootTransform(MakeArrayView(&RootTransform, 1), Sequence, 0);

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				Out.FrameAttribute->AttributeData[0][FrameOffset + FrameIdx] = RootTransform.GetLocation().X;
				Out.FrameAttribute->AttributeData[1][FrameOffset + FrameIdx] = RootTransform.GetLocation().Y;
				Out.FrameAttribute->AttributeData[2][FrameOffset + FrameIdx] = RootTransform.GetLocation().Z;

				Out.FrameAttribute->AttributeData[3][FrameOffset + FrameIdx] = RootTransform.GetRotation().X;
				Out.FrameAttribute->AttributeData[4][FrameOffset + FrameIdx] = RootTransform.GetRotation().Y;
				Out.FrameAttribute->AttributeData[5][FrameOffset + FrameIdx] = RootTransform.GetRotation().Z;
				Out.FrameAttribute->AttributeData[6][FrameOffset + FrameIdx] = RootTransform.GetRotation().W;

				Out.FrameAttribute->AttributeData[7][FrameOffset + FrameIdx] = RootTransform.GetScale3D().X;
				Out.FrameAttribute->AttributeData[8][FrameOffset + FrameIdx] = RootTransform.GetScale3D().Y;
				Out.FrameAttribute->AttributeData[9][FrameOffset + FrameIdx] = RootTransform.GetScale3D().Z;
			}
		});

	UE::Learning::Array::Check(Out.FrameAttribute->AttributeData);

	return Out;
}

void UAnimDatabaseFrameAttributeLibrary::MakeTrajectoryFrameAttribute(
	TArray<FAnimDatabaseFrameAttribute>& OutLocationFrameAttributes,
	TArray<FAnimDatabaseFrameAttribute>& OutDirectionFrameAttributes,
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const int32 SampleNum,
	const float FutureTime,
	const float PastTime,
	const FVector ForwardVector)
{
	if (SampleNum < 0)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeTrajectoryFrameAttribute: Invalid Sample Num: {SampleNum}.", SampleNum);
		OutLocationFrameAttributes.Empty();
		OutDirectionFrameAttributes.Empty();
		return;
	}

	OutLocationFrameAttributes.SetNum(SampleNum);
	OutDirectionFrameAttributes.SetNum(SampleNum);

	ParallelFor(SampleNum, [
		SampleNum, FutureTime, PastTime, ForwardVector, Database,
		&OutLocationFrameAttributes,
		&OutDirectionFrameAttributes,
		&FrameRanges](const int32 SampleIdx) {

			const float RelativeTime = ((FutureTime + PastTime) / FMath::Max(SampleNum - 1, 1)) * SampleIdx - PastTime;

			MakeRootLocationAndDirectionFrameAttribute(
				OutLocationFrameAttributes[SampleIdx],
				OutDirectionFrameAttributes[SampleIdx],
				Database, FrameRanges, RelativeTime, ForwardVector);

		});
}

void UAnimDatabaseFrameAttributeLibrary::MakeLocationTrajectoryFrameAttribute(
	TArray<FAnimDatabaseFrameAttribute>& OutLocationFrameAttributes,
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const int32 SampleNum,
	const float FutureTime,
	const float PastTime)
{
	if (SampleNum < 0)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeLocationTrajectoryFrameAttribute: Invalid Sample Num: {SampleNum}.", SampleNum);
		OutLocationFrameAttributes.Empty();
		return;
	}

	OutLocationFrameAttributes.SetNum(SampleNum);

	ParallelFor(SampleNum, [
		SampleNum, FutureTime, PastTime, Database,
		&OutLocationFrameAttributes,
		&FrameRanges](const int32 SampleIdx) {

			const float RelativeTime = ((FutureTime + PastTime) / FMath::Max(SampleNum - 1, 1)) * SampleIdx - PastTime;
			OutLocationFrameAttributes[SampleIdx] = MakeRootLocationFrameAttribute(Database, FrameRanges, RelativeTime);

		});
}

void UAnimDatabaseFrameAttributeLibrary::MakeLookAtTrajectoryFrameAttribute(
	TArray<FAnimDatabaseFrameAttribute>& OutLocationFrameAttributes,
	TArray<FAnimDatabaseFrameAttribute>& OutDirectionFrameAttributes,
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const int32 LookAtBoneIndex,
	const int32 SampleNum,
	const float FutureTime,
	const float PastTime,
	const FVector LocalDirection,
	const float DirectionSmoothingAmount)
{
	if (SampleNum < 0)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeLookAtTrajectoryFrameAttribute: Invalid Sample Num: {SampleNum}.", SampleNum);
		OutLocationFrameAttributes.Empty();
		OutDirectionFrameAttributes.Empty();
		return;
	}

	OutLocationFrameAttributes.SetNum(SampleNum);
	OutDirectionFrameAttributes.SetNum(SampleNum);

	ParallelFor(SampleNum, [
		SampleNum, FutureTime, PastTime, Database, LookAtBoneIndex, LocalDirection, DirectionSmoothingAmount,
		&OutLocationFrameAttributes,
		&OutDirectionFrameAttributes,
		&FrameRanges](const int32 SampleIdx) {

			const float RelativeTime = ((FutureTime + PastTime) / FMath::Max(SampleNum - 1, 1)) * SampleIdx - PastTime;

			OutLocationFrameAttributes[SampleIdx] = MakeRootLocationFrameAttribute(Database, FrameRanges, RelativeTime);
			OutDirectionFrameAttributes[SampleIdx] = FrameAttributeFilterGaussian(FrameAttributeProject(
				MakeBoneGlobalDirectionFrameAttribute(Database, FrameRanges, LookAtBoneIndex, LocalDirection, RelativeTime)), DirectionSmoothingAmount);

		});
}

void UAnimDatabaseFrameAttributeLibrary::MakeRootLocalTrajectoryFrameAttributeAndScale(
	TArray<FAnimDatabaseFrameAttribute>& OutLocationFrameAttributes,
	TArray<FAnimDatabaseFrameAttribute>& OutDirectionFrameAttributes,
	float& OutLocationScale,
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const int32 SampleNum,
	const float FutureTime,
	const float PastTime,
	const FVector ForwardVector)
{
	FAnimDatabaseFrameAttribute RootLocation, RootRotation;
	MakeRootLocationAndRotationFrameAttribute(RootLocation, RootRotation, Database, FrameRanges);

	TArray<FAnimDatabaseFrameAttribute> LocationFrameAttributes, DirectionFrameAttributes;
	MakeTrajectoryFrameAttribute(LocationFrameAttributes, DirectionFrameAttributes, Database, FrameRanges, SampleNum, FutureTime, PastTime, ForwardVector);

	FrameAttributesSubtractAndUnrotate(OutLocationFrameAttributes, LocationFrameAttributes, RootLocation, RootRotation);
	FrameAttributesUnrotate(OutDirectionFrameAttributes, RootRotation, DirectionFrameAttributes);
	OutLocationScale = FrameAttributesAverageStdFromArrayView(OutLocationFrameAttributes);
}

void UAnimDatabaseFrameAttributeLibrary::MakeRootLocalLookAtTrajectoryFrameAttributeAndScale(
	TArray<FAnimDatabaseFrameAttribute>& OutLocationFrameAttributes,
	TArray<FAnimDatabaseFrameAttribute>& OutDirectionFrameAttributes,
	float& OutLocationScale,
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const int32 LookAtBoneIndex,
	const int32 SampleNum,
	const float FutureTime,
	const float PastTime,
	const FVector LocalDirection,
	const float DirectionSmoothingAmount)
{
	FAnimDatabaseFrameAttribute RootLocation, RootRotation;
	MakeRootLocationAndRotationFrameAttribute(RootLocation, RootRotation, Database, FrameRanges);

	TArray<FAnimDatabaseFrameAttribute> LocationFrameAttributes, DirectionFrameAttributes;
	MakeLookAtTrajectoryFrameAttribute(LocationFrameAttributes, DirectionFrameAttributes, Database, FrameRanges, LookAtBoneIndex, SampleNum, FutureTime, PastTime, LocalDirection);

	FrameAttributesSubtractAndUnrotate(OutLocationFrameAttributes, LocationFrameAttributes, RootLocation, RootRotation);
	FrameAttributesUnrotate(OutDirectionFrameAttributes, RootRotation, DirectionFrameAttributes);
	OutLocationScale = FrameAttributesAverageStdFromArrayView(OutLocationFrameAttributes);
}

void UAnimDatabaseFrameAttributeLibrary::MakeBoneLocalFrameAttributes(
	TArray<FAnimDatabaseFrameAttribute>& OutLocationFrameAttributes,
	TArray<FAnimDatabaseFrameAttribute>& OutRotationFrameAttributes,
	TArray<FAnimDatabaseFrameAttribute>& OutScaleFrameAttributes,
	TArray<FAnimDatabaseFrameAttribute>& OutLinearVelocityFrameAttributes,
	TArray<FAnimDatabaseFrameAttribute>& OutAngularVelocityFrameAttributes,
	TArray<FAnimDatabaseFrameAttribute>& OutScalarVelocityFrameAttributes,
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const TArray<int32>& BoneLocationIndices,
	const TArray<int32>& BoneRotationIndices,
	const TArray<int32>& BoneScaleIndices,
	const TArray<int32>& BoneLinearVelocityIndices,
	const TArray<int32>& BoneAngularVelocityIndices,
	const TArray<int32>& BoneScalarVelocityIndices,
	const float RelativeTime)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::MakeBoneLocalFrameAttributes);

	for (const TArray<int32>& BoneIndices : { BoneLocationIndices , BoneRotationIndices , BoneScaleIndices , BoneLinearVelocityIndices , BoneAngularVelocityIndices, BoneScalarVelocityIndices })
	{
		for (const int32 BoneIndex : BoneIndices)
		{
			if (BoneIndex == INDEX_NONE)
			{
				UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneLocalFrameAttributes: Invalid Bone Index.");
				return;
			}
		}
	}

	OutLocationFrameAttributes.SetNum(BoneLocationIndices.Num());
	OutRotationFrameAttributes.SetNum(BoneRotationIndices.Num());
	OutScaleFrameAttributes.SetNum(BoneScaleIndices.Num());
	OutLinearVelocityFrameAttributes.SetNum(BoneLinearVelocityIndices.Num());
	OutAngularVelocityFrameAttributes.SetNum(BoneAngularVelocityIndices.Num());
	OutScalarVelocityFrameAttributes.SetNum(BoneScalarVelocityIndices.Num());

	MakeBoneLocalFrameAttributesFromArrayViews(
		OutLocationFrameAttributes,
		OutRotationFrameAttributes,
		OutScaleFrameAttributes,
		OutLinearVelocityFrameAttributes,
		OutAngularVelocityFrameAttributes,
		OutScalarVelocityFrameAttributes,
		Database,
		FrameRanges,
		BoneLocationIndices,
		BoneRotationIndices,
		BoneScaleIndices,
		BoneLinearVelocityIndices,
		BoneAngularVelocityIndices,
		BoneScalarVelocityIndices,
		RelativeTime);
}

void UAnimDatabaseFrameAttributeLibrary::MakeBoneLocalFrameAttributesFromArrayViews(
	const TArrayView<FAnimDatabaseFrameAttribute> OutLocationFrameAttributes,
	const TArrayView<FAnimDatabaseFrameAttribute> OutRotationFrameAttributes,
	const TArrayView<FAnimDatabaseFrameAttribute> OutScaleFrameAttributes,
	const TArrayView<FAnimDatabaseFrameAttribute> OutLinearVelocityFrameAttributes,
	const TArrayView<FAnimDatabaseFrameAttribute> OutAngularVelocityFrameAttributes,
	const TArrayView<FAnimDatabaseFrameAttribute> OutScalarVelocityFrameAttributes,
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const TArrayView<const int32> BoneLocationIndices,
	const TArrayView<const int32> BoneRotationIndices,
	const TArrayView<const int32> BoneScaleIndices,
	const TArrayView<const int32> BoneLinearVelocityIndices,
	const TArrayView<const int32> BoneAngularVelocityIndices,
	const TArrayView<const int32> BoneScalarVelocityIndices,
	const float RelativeTime)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneLocalFrameAttributesFromArrayViews: Database is nullptr.");
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutLocationFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutRotationFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutScaleFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutLinearVelocityFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutAngularVelocityFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutScalarVelocityFrameAttributes, FAnimDatabaseFrameAttribute());
		return;
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneLocalFrameAttributesFromArrayViews: Invalid FrameRanges.");
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutLocationFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutRotationFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutScaleFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutLinearVelocityFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutAngularVelocityFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutScalarVelocityFrameAttributes, FAnimDatabaseFrameAttribute());
		return;
	}

	if (UE::AnimDatabase::Math::AnyBoneIndicesInvalid(BoneLocationIndices) ||
		UE::AnimDatabase::Math::AnyBoneIndicesInvalid(BoneRotationIndices) ||
		UE::AnimDatabase::Math::AnyBoneIndicesInvalid(BoneScaleIndices) ||
		UE::AnimDatabase::Math::AnyBoneIndicesInvalid(BoneLinearVelocityIndices) ||
		UE::AnimDatabase::Math::AnyBoneIndicesInvalid(BoneAngularVelocityIndices) ||
		UE::AnimDatabase::Math::AnyBoneIndicesInvalid(BoneScalarVelocityIndices))
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneLocalFrameAttributesFromArrayViews: Invalid Bone Index.");
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutLocationFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutRotationFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutScaleFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutLinearVelocityFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutAngularVelocityFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutScalarVelocityFrameAttributes, FAnimDatabaseFrameAttribute());
		return;
	}

	const int32 BoneLocationIndexNum = BoneLocationIndices.Num();
	const int32 BoneRotationIndexNum = BoneRotationIndices.Num();
	const int32 BoneScaleIndexNum = BoneScaleIndices.Num();
	const int32 BoneLinearVelocityIndexNum = BoneLinearVelocityIndices.Num();
	const int32 BoneAngularVelocityIndexNum = BoneAngularVelocityIndices.Num();
	const int32 BoneScalarVelocityIndexNum = BoneScalarVelocityIndices.Num();
	const int32 TotalFrameNum = FrameRanges.FrameRangeSet->GetTotalFrameNum();

	check(OutLocationFrameAttributes.Num() == BoneLocationIndexNum);
	check(OutRotationFrameAttributes.Num() == BoneRotationIndexNum);
	check(OutScaleFrameAttributes.Num() == BoneScaleIndexNum);
	check(OutLinearVelocityFrameAttributes.Num() == BoneLinearVelocityIndexNum);
	check(OutAngularVelocityFrameAttributes.Num() == BoneAngularVelocityIndexNum);
	check(OutScalarVelocityFrameAttributes.Num() == BoneScalarVelocityIndexNum);

	for (int32 BoneIdx = 0; BoneIdx < BoneLocationIndexNum; BoneIdx++)
	{
		OutLocationFrameAttributes[BoneIdx] = MakeEmptyFrameAttribute();
		OutLocationFrameAttributes[BoneIdx].Type = EAnimDatabaseAttributeType::Location;
		OutLocationFrameAttributes[BoneIdx].FrameAttribute->FrameRangeSet = *FrameRanges.FrameRangeSet;
		OutLocationFrameAttributes[BoneIdx].FrameAttribute->AttributeData.SetNumUninitialized({ AttributeTypeSize(EAnimDatabaseAttributeType::Location), TotalFrameNum });
	}

	for (int32 BoneIdx = 0; BoneIdx < BoneRotationIndexNum; BoneIdx++)
	{
		OutRotationFrameAttributes[BoneIdx] = MakeEmptyFrameAttribute();
		OutRotationFrameAttributes[BoneIdx].Type = EAnimDatabaseAttributeType::Rotation;
		OutRotationFrameAttributes[BoneIdx].FrameAttribute->FrameRangeSet = *FrameRanges.FrameRangeSet;
		OutRotationFrameAttributes[BoneIdx].FrameAttribute->AttributeData.SetNumUninitialized({ AttributeTypeSize(EAnimDatabaseAttributeType::Rotation), TotalFrameNum });
	}

	for (int32 BoneIdx = 0; BoneIdx < BoneScaleIndexNum; BoneIdx++)
	{
		OutScaleFrameAttributes[BoneIdx] = MakeEmptyFrameAttribute();
		OutScaleFrameAttributes[BoneIdx].Type = EAnimDatabaseAttributeType::Scale;
		OutScaleFrameAttributes[BoneIdx].FrameAttribute->FrameRangeSet = *FrameRanges.FrameRangeSet;
		OutScaleFrameAttributes[BoneIdx].FrameAttribute->AttributeData.SetNumUninitialized({ AttributeTypeSize(EAnimDatabaseAttributeType::Scale), TotalFrameNum });
	}

	for (int32 BoneIdx = 0; BoneIdx < BoneLinearVelocityIndexNum; BoneIdx++)
	{
		OutLinearVelocityFrameAttributes[BoneIdx] = MakeEmptyFrameAttribute();
		OutLinearVelocityFrameAttributes[BoneIdx].Type = EAnimDatabaseAttributeType::LinearVelocity;
		OutLinearVelocityFrameAttributes[BoneIdx].FrameAttribute->FrameRangeSet = *FrameRanges.FrameRangeSet;
		OutLinearVelocityFrameAttributes[BoneIdx].FrameAttribute->AttributeData.SetNumUninitialized({ AttributeTypeSize(EAnimDatabaseAttributeType::LinearVelocity), TotalFrameNum });
	}

	for (int32 BoneIdx = 0; BoneIdx < BoneAngularVelocityIndexNum; BoneIdx++)
	{
		OutAngularVelocityFrameAttributes[BoneIdx] = MakeEmptyFrameAttribute();
		OutAngularVelocityFrameAttributes[BoneIdx].Type = EAnimDatabaseAttributeType::AngularVelocity;
		OutAngularVelocityFrameAttributes[BoneIdx].FrameAttribute->FrameRangeSet = *FrameRanges.FrameRangeSet;
		OutAngularVelocityFrameAttributes[BoneIdx].FrameAttribute->AttributeData.SetNumUninitialized({ AttributeTypeSize(EAnimDatabaseAttributeType::AngularVelocity), TotalFrameNum });
	}

	for (int32 BoneIdx = 0; BoneIdx < BoneScalarVelocityIndexNum; BoneIdx++)
	{
		OutScalarVelocityFrameAttributes[BoneIdx] = MakeEmptyFrameAttribute();
		OutScalarVelocityFrameAttributes[BoneIdx].Type = EAnimDatabaseAttributeType::ScalarVelocity;
		OutScalarVelocityFrameAttributes[BoneIdx].FrameAttribute->FrameRangeSet = *FrameRanges.FrameRangeSet;
		OutScalarVelocityFrameAttributes[BoneIdx].FrameAttribute->AttributeData.SetNumUninitialized({ AttributeTypeSize(EAnimDatabaseAttributeType::ScalarVelocity), TotalFrameNum });
	}

	// Find required Indices

	TArray<int32> BoneParents;
	BoneParents.SetNumUninitialized(Database->GetBoneNum());
	Database->GetBoneParents(BoneParents);

	TArray<int32> RequiredIndices;
	TArray<int32> CurrentIndices;

	for (int32 BoneIdx = 0; BoneIdx < BoneLocationIndexNum; BoneIdx++)
	{
		CurrentIndices = RequiredIndices;
		UE::AnimDatabase::Math::BoneUnion(RequiredIndices, CurrentIndices, BoneLocationIndices[BoneIdx]);
	}

	for (int32 BoneIdx = 0; BoneIdx < BoneRotationIndexNum; BoneIdx++)
	{
		CurrentIndices = RequiredIndices;
		UE::AnimDatabase::Math::BoneUnion(RequiredIndices, CurrentIndices, BoneRotationIndices[BoneIdx]);
	}

	for (int32 BoneIdx = 0; BoneIdx < BoneScaleIndexNum; BoneIdx++)
	{
		CurrentIndices = RequiredIndices;
		UE::AnimDatabase::Math::BoneUnion(RequiredIndices, CurrentIndices, BoneScaleIndices[BoneIdx]);
	}

	for (int32 BoneIdx = 0; BoneIdx < BoneLinearVelocityIndexNum; BoneIdx++)
	{
		CurrentIndices = RequiredIndices;
		UE::AnimDatabase::Math::BoneUnion(RequiredIndices, CurrentIndices, BoneLinearVelocityIndices[BoneIdx]);
	}

	for (int32 BoneIdx = 0; BoneIdx < BoneAngularVelocityIndexNum; BoneIdx++)
	{
		CurrentIndices = RequiredIndices;
		UE::AnimDatabase::Math::BoneUnion(RequiredIndices, CurrentIndices, BoneAngularVelocityIndices[BoneIdx]);
	}

	for (int32 BoneIdx = 0; BoneIdx < BoneScalarVelocityIndexNum; BoneIdx++)
	{
		CurrentIndices = RequiredIndices;
		UE::AnimDatabase::Math::BoneUnion(RequiredIndices, CurrentIndices, BoneScalarVelocityIndices[BoneIdx]);
	}

	// Extract Pose Data

	const int32 RequiredBoneNum = RequiredIndices.Num();

	UE::AnimDatabase::FPoseData PoseData;
	PoseData.Resize(TotalFrameNum, RequiredBoneNum, {}, {});

	Database->WaitForCompressionOnAnimSequencesFromArrayView(FrameRanges.FrameRangeSet->GetEntrySequences());

	UE::Learning::FrameRangeSet::ParallelForEachRange(*FrameRanges.FrameRangeSet, [&FrameRanges, &PoseData, &Database, &RequiredIndices, RelativeTime](
		const int32 TotalRangeIdx,
		const int32 EntryIdx,
		const int32 RangeIdx) {

			const int32 FrameOffset = FrameRanges.FrameRangeSet->GetEntryRangeOffset(EntryIdx, RangeIdx);
			const int32 FrameNum = FrameRanges.FrameRangeSet->GetEntryRangeLength(EntryIdx, RangeIdx);
			const int32 Sequence = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);
			const int32 StartFrame = FrameRanges.FrameRangeSet->GetEntryRangeStart(EntryIdx, RangeIdx);

			Database->GetPoseSubsetData(
				PoseData.Slice(FrameOffset, FrameNum),
				Sequence,
				StartFrame + FMath::RoundToInt(RelativeTime * Database->GetFrameRate().AsDecimal()),
				RequiredIndices);
		});

	TArray<int32, TInlineAllocator<8>> RequiredBoneLocationIndices;
	TArray<int32, TInlineAllocator<8>> RequiredBoneRotationIndices;
	TArray<int32, TInlineAllocator<8>> RequiredBoneScaleIndices;
	TArray<int32, TInlineAllocator<8>> RequiredBoneLinearVelocityIndices;
	TArray<int32, TInlineAllocator<8>> RequiredBoneAngularVelocityIndices;
	TArray<int32, TInlineAllocator<8>> RequiredBoneScalarVelocityIndices;

	RequiredBoneLocationIndices.SetNumUninitialized(BoneLocationIndexNum);
	RequiredBoneRotationIndices.SetNumUninitialized(BoneRotationIndexNum);
	RequiredBoneScaleIndices.SetNumUninitialized(BoneScaleIndexNum);
	RequiredBoneLinearVelocityIndices.SetNumUninitialized(BoneLinearVelocityIndexNum);
	RequiredBoneAngularVelocityIndices.SetNumUninitialized(BoneAngularVelocityIndexNum);
	RequiredBoneScalarVelocityIndices.SetNumUninitialized(BoneScalarVelocityIndexNum);

	UE::AnimDatabase::Math::BoneFindIndicesOf(RequiredBoneLocationIndices, BoneLocationIndices, RequiredIndices);
	UE::AnimDatabase::Math::BoneFindIndicesOf(RequiredBoneRotationIndices, BoneRotationIndices, RequiredIndices);
	UE::AnimDatabase::Math::BoneFindIndicesOf(RequiredBoneScaleIndices, BoneScaleIndices, RequiredIndices);
	UE::AnimDatabase::Math::BoneFindIndicesOf(RequiredBoneLinearVelocityIndices, BoneLinearVelocityIndices, RequiredIndices);
	UE::AnimDatabase::Math::BoneFindIndicesOf(RequiredBoneAngularVelocityIndices, BoneAngularVelocityIndices, RequiredIndices);
	UE::AnimDatabase::Math::BoneFindIndicesOf(RequiredBoneScalarVelocityIndices, BoneScalarVelocityIndices, RequiredIndices);

	UE::Learning::SlicedParallelFor(TotalFrameNum, 512, [
			&PoseData,
			&RequiredBoneLocationIndices,
			&RequiredBoneRotationIndices,
			&RequiredBoneScaleIndices,
			&RequiredBoneLinearVelocityIndices,
			&RequiredBoneAngularVelocityIndices,
			&RequiredBoneScalarVelocityIndices,
			&OutLocationFrameAttributes,
			&OutRotationFrameAttributes,
			&OutScaleFrameAttributes,
			&OutLinearVelocityFrameAttributes,
			&OutAngularVelocityFrameAttributes,
			&OutScalarVelocityFrameAttributes](const int32 SliceStart, const int32 SliceLength)
		{
			for (int32 FrameIdx = SliceStart; FrameIdx < SliceStart + SliceLength; FrameIdx++)
			{
				const int32 BoneLocationIndexNum = RequiredBoneLocationIndices.Num();
				const int32 BoneRotationIndexNum = RequiredBoneRotationIndices.Num();
				const int32 BoneScaleIndexNum = RequiredBoneScaleIndices.Num();
				const int32 BoneLinearVelocityIndexNum = RequiredBoneLinearVelocityIndices.Num();
				const int32 BoneAngularVelocityIndexNum = RequiredBoneAngularVelocityIndices.Num();
				const int32 BoneScalarVelocityIndexNum = RequiredBoneScalarVelocityIndices.Num();

				for (int32 BoneIdx = 0; BoneIdx < BoneLocationIndexNum; BoneIdx++)
				{
					const FVector3f Location = PoseData.LocalBoneData.BoneLocations[FrameIdx][RequiredBoneLocationIndices[BoneIdx]];
					OutLocationFrameAttributes[BoneIdx].FrameAttribute->AttributeData[0][FrameIdx] = Location.X;
					OutLocationFrameAttributes[BoneIdx].FrameAttribute->AttributeData[1][FrameIdx] = Location.Y;
					OutLocationFrameAttributes[BoneIdx].FrameAttribute->AttributeData[2][FrameIdx] = Location.Z;
				}

				for (int32 BoneIdx = 0; BoneIdx < BoneRotationIndexNum; BoneIdx++)
				{
					const FQuat4f Rotation = PoseData.LocalBoneData.BoneRotations[FrameIdx][RequiredBoneRotationIndices[BoneIdx]];
					OutRotationFrameAttributes[BoneIdx].FrameAttribute->AttributeData[0][FrameIdx] = Rotation.X;
					OutRotationFrameAttributes[BoneIdx].FrameAttribute->AttributeData[1][FrameIdx] = Rotation.Y;
					OutRotationFrameAttributes[BoneIdx].FrameAttribute->AttributeData[2][FrameIdx] = Rotation.Z;
					OutRotationFrameAttributes[BoneIdx].FrameAttribute->AttributeData[3][FrameIdx] = Rotation.W;
				}

				for (int32 BoneIdx = 0; BoneIdx < BoneScaleIndexNum; BoneIdx++)
				{
					const FVector3f Scale = PoseData.LocalBoneData.BoneScales[FrameIdx][RequiredBoneScaleIndices[BoneIdx]];
					OutScaleFrameAttributes[BoneIdx].FrameAttribute->AttributeData[0][FrameIdx] = Scale.X;
					OutScaleFrameAttributes[BoneIdx].FrameAttribute->AttributeData[1][FrameIdx] = Scale.Y;
					OutScaleFrameAttributes[BoneIdx].FrameAttribute->AttributeData[2][FrameIdx] = Scale.Z;
				}

				for (int32 BoneIdx = 0; BoneIdx < BoneLinearVelocityIndexNum; BoneIdx++)
				{
					const FVector3f LinearVelocity = PoseData.LocalBoneData.BoneLinearVelocities[FrameIdx][RequiredBoneLinearVelocityIndices[BoneIdx]];
					OutLinearVelocityFrameAttributes[BoneIdx].FrameAttribute->AttributeData[0][FrameIdx] = LinearVelocity.X;
					OutLinearVelocityFrameAttributes[BoneIdx].FrameAttribute->AttributeData[1][FrameIdx] = LinearVelocity.Y;
					OutLinearVelocityFrameAttributes[BoneIdx].FrameAttribute->AttributeData[2][FrameIdx] = LinearVelocity.Z;
				}

				for (int32 BoneIdx = 0; BoneIdx < BoneAngularVelocityIndexNum; BoneIdx++)
				{
					const FVector3f AngularVelocity = PoseData.LocalBoneData.BoneAngularVelocities[FrameIdx][RequiredBoneAngularVelocityIndices[BoneIdx]];
					OutAngularVelocityFrameAttributes[BoneIdx].FrameAttribute->AttributeData[0][FrameIdx] = AngularVelocity.X;
					OutAngularVelocityFrameAttributes[BoneIdx].FrameAttribute->AttributeData[1][FrameIdx] = AngularVelocity.Y;
					OutAngularVelocityFrameAttributes[BoneIdx].FrameAttribute->AttributeData[2][FrameIdx] = AngularVelocity.Z;
				}

				for (int32 BoneIdx = 0; BoneIdx < BoneScalarVelocityIndexNum; BoneIdx++)
				{
					const FVector3f ScalarVelocity = PoseData.LocalBoneData.BoneScalarVelocities[FrameIdx][RequiredBoneScalarVelocityIndices[BoneIdx]];
					OutScalarVelocityFrameAttributes[BoneIdx].FrameAttribute->AttributeData[0][FrameIdx] = ScalarVelocity.X;
					OutScalarVelocityFrameAttributes[BoneIdx].FrameAttribute->AttributeData[1][FrameIdx] = ScalarVelocity.Y;
					OutScalarVelocityFrameAttributes[BoneIdx].FrameAttribute->AttributeData[2][FrameIdx] = ScalarVelocity.Z;
				}
			}
		});

		for (int32 BoneIdx = 0; BoneIdx < BoneLocationIndexNum; BoneIdx++)
		{
			UE::Learning::Array::Check(OutLocationFrameAttributes[BoneIdx].FrameAttribute->AttributeData);
		}

		for (int32 BoneIdx = 0; BoneIdx < BoneRotationIndexNum; BoneIdx++)
		{
			UE::Learning::Array::Check(OutRotationFrameAttributes[BoneIdx].FrameAttribute->AttributeData);
		}

		for (int32 BoneIdx = 0; BoneIdx < BoneScaleIndexNum; BoneIdx++)
		{
			UE::Learning::Array::Check(OutScaleFrameAttributes[BoneIdx].FrameAttribute->AttributeData);
		}

		for (int32 BoneIdx = 0; BoneIdx < BoneLinearVelocityIndexNum; BoneIdx++)
		{
			UE::Learning::Array::Check(OutLinearVelocityFrameAttributes[BoneIdx].FrameAttribute->AttributeData);
		}

		for (int32 BoneIdx = 0; BoneIdx < BoneAngularVelocityIndexNum; BoneIdx++)
		{
			UE::Learning::Array::Check(OutAngularVelocityFrameAttributes[BoneIdx].FrameAttribute->AttributeData);
		}

		for (int32 BoneIdx = 0; BoneIdx < BoneScalarVelocityIndexNum; BoneIdx++)
		{
			UE::Learning::Array::Check(OutScalarVelocityFrameAttributes[BoneIdx].FrameAttribute->AttributeData);
		}
}

void UAnimDatabaseFrameAttributeLibrary::MakeBoneLocalTransformFrameAttributes(
	TArray<FAnimDatabaseFrameAttribute>& OutTransformFrameAttributes,
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const TArray<int32>& BoneIndices,
	const float RelativeTime)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::MakeBoneLocalTransformFrameAttributes);

	for (const int32 BoneIndex : BoneIndices)
	{
		if (BoneIndex == INDEX_NONE)
		{
			UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneLocalTransformFrameAttributes: Invalid Bone Index.");
			OutTransformFrameAttributes.Empty();
			return;
		}
	}

	OutTransformFrameAttributes.SetNum(BoneIndices.Num());

	MakeBoneLocalTransformFrameAttributesFromArrayViews(
		OutTransformFrameAttributes,
		Database,
		FrameRanges,
		BoneIndices,
		RelativeTime);
}

void UAnimDatabaseFrameAttributeLibrary::MakeBoneLocalTransformFrameAttributesFromNames(
	TArray<FAnimDatabaseFrameAttribute>& OutTransformFrameAttributes,
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const TArray<FName>& BoneNames,
	const float RelativeTime)
{
	OutTransformFrameAttributes.SetNum(BoneNames.Num());
	MakeBoneLocalTransformFrameAttributesFromNamesArrayView(OutTransformFrameAttributes, Database, FrameRanges, BoneNames, RelativeTime);
}

void UAnimDatabaseFrameAttributeLibrary::MakeBoneLocalTransformFrameAttributesFromNamesArrayView(
	const TArrayView<FAnimDatabaseFrameAttribute> OutTransformFrameAttributes,
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const TArrayView<const FName> BoneNames,
	const float RelativeTime)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneLocalTransformFrameAttributesFromNamesArrayView: Database is nullptr.");
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutTransformFrameAttributes, FAnimDatabaseFrameAttribute());
		return;
	}

	check(OutTransformFrameAttributes.Num() == BoneNames.Num());

	TArray<int32, TInlineAllocator<32>> BoneIndices;
	const int32 BoneNum = BoneNames.Num();
	BoneIndices.SetNumUninitialized(BoneNum);
	Database->FindBoneIndicesFromArrayViews(BoneIndices, BoneNames);

	MakeBoneLocalTransformFrameAttributesFromArrayViews(
		OutTransformFrameAttributes,
		Database,
		FrameRanges,
		BoneIndices,
		RelativeTime);
}

void UAnimDatabaseFrameAttributeLibrary::MakeBoneLocalTransformFrameAttributesFromArrayViews(
	const TArrayView<FAnimDatabaseFrameAttribute> OutTransformFrameAttributes,
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const TArrayView<const int32> BoneIndices,
	const float RelativeTime)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneLocalTransformFrameAttributesFromArrayViews: Database is nullptr.");
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutTransformFrameAttributes, FAnimDatabaseFrameAttribute());
		return;
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneLocalTransformFrameAttributesFromArrayViews: Invalid FrameRanges.");
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutTransformFrameAttributes, FAnimDatabaseFrameAttribute());
		return;
	}

	const int32 BoneIndexNum = BoneIndices.Num();
	const int32 TotalFrameNum = FrameRanges.FrameRangeSet->GetTotalFrameNum();

	check(OutTransformFrameAttributes.Num() == BoneIndexNum);

	for (int32 BoneIdx = 0; BoneIdx < BoneIndexNum; BoneIdx++)
	{
		if (BoneIndices[BoneIdx] == INDEX_NONE)
		{
			UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneLocalTransformFrameAttributesFromArrayViews: Invalid Bone Index.");
			UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutTransformFrameAttributes, FAnimDatabaseFrameAttribute());
			return;
		}

		OutTransformFrameAttributes[BoneIdx] = MakeEmptyFrameAttribute();
		OutTransformFrameAttributes[BoneIdx].Type = EAnimDatabaseAttributeType::Transform;
		OutTransformFrameAttributes[BoneIdx].FrameAttribute->FrameRangeSet = *FrameRanges.FrameRangeSet;
		OutTransformFrameAttributes[BoneIdx].FrameAttribute->AttributeData.SetNumUninitialized({ AttributeTypeSize(EAnimDatabaseAttributeType::Transform), TotalFrameNum });
	}

	// Extract Pose Data

	UE::AnimDatabase::FPoseData PoseData;
	PoseData.Resize(TotalFrameNum, BoneIndexNum, {}, {});

	Database->WaitForCompressionOnAnimSequencesFromArrayView(FrameRanges.FrameRangeSet->GetEntrySequences());

	UE::Learning::FrameRangeSet::ParallelForEachRange(*FrameRanges.FrameRangeSet, [&FrameRanges, &PoseData, &Database, &BoneIndices, RelativeTime](
		const int32 TotalRangeIdx,
		const int32 EntryIdx,
		const int32 RangeIdx) {

			const int32 FrameOffset = FrameRanges.FrameRangeSet->GetEntryRangeOffset(EntryIdx, RangeIdx);
			const int32 FrameNum = FrameRanges.FrameRangeSet->GetEntryRangeLength(EntryIdx, RangeIdx);
			const int32 Sequence = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);
			const int32 StartFrame = FrameRanges.FrameRangeSet->GetEntryRangeStart(EntryIdx, RangeIdx);

			Database->GetPoseSubsetData(
				PoseData.Slice(FrameOffset, FrameNum),
				Sequence,
				StartFrame + FMath::RoundToInt(RelativeTime * Database->GetFrameRate().AsDecimal()),
				BoneIndices);
		});

	UE::Learning::SlicedParallelFor(TotalFrameNum, 512, [
			&PoseData,
			&OutTransformFrameAttributes](const int32 SliceStart, const int32 SliceLength)
		{
			const int32 BoneIndexNum = PoseData.LocalBoneData.GetBoneNum();
			check(OutTransformFrameAttributes.Num() == BoneIndexNum);

			for (int32 FrameIdx = SliceStart; FrameIdx < SliceStart + SliceLength; FrameIdx++)
			{
				for (int32 BoneIdx = 0; BoneIdx < BoneIndexNum; BoneIdx++)
				{
					const FVector3f Location = PoseData.LocalBoneData.BoneLocations[FrameIdx][BoneIdx];
					const FQuat4f Rotation = PoseData.LocalBoneData.BoneRotations[FrameIdx][BoneIdx];
					const FVector3f Scale = PoseData.LocalBoneData.BoneScales[FrameIdx][BoneIdx];
					OutTransformFrameAttributes[BoneIdx].FrameAttribute->AttributeData[0][FrameIdx] = Location.X;
					OutTransformFrameAttributes[BoneIdx].FrameAttribute->AttributeData[1][FrameIdx] = Location.Y;
					OutTransformFrameAttributes[BoneIdx].FrameAttribute->AttributeData[2][FrameIdx] = Location.Z;
					OutTransformFrameAttributes[BoneIdx].FrameAttribute->AttributeData[3][FrameIdx] = Rotation.X;
					OutTransformFrameAttributes[BoneIdx].FrameAttribute->AttributeData[4][FrameIdx] = Rotation.Y;
					OutTransformFrameAttributes[BoneIdx].FrameAttribute->AttributeData[5][FrameIdx] = Rotation.Z;
					OutTransformFrameAttributes[BoneIdx].FrameAttribute->AttributeData[6][FrameIdx] = Rotation.W;
					OutTransformFrameAttributes[BoneIdx].FrameAttribute->AttributeData[7][FrameIdx] = Scale.X;
					OutTransformFrameAttributes[BoneIdx].FrameAttribute->AttributeData[8][FrameIdx] = Scale.Y;
					OutTransformFrameAttributes[BoneIdx].FrameAttribute->AttributeData[9][FrameIdx] = Scale.Z;
				}

			}
		});

	for (int32 BoneIdx = 0; BoneIdx < BoneIndexNum; BoneIdx++)
	{
		UE::Learning::Array::Check(OutTransformFrameAttributes[BoneIdx].FrameAttribute->AttributeData);
	}
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeBoneLocalTransformFrameAttribute(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const int32 BoneIndex, const float RelativeTime)
{
	FAnimDatabaseFrameAttribute Out;
	MakeBoneLocalTransformFrameAttributesFromArrayViews(MakeArrayView(&Out, 1), Database, FrameRanges, { BoneIndex }, RelativeTime);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeBoneLocalTransformFrameAttributeFromName(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName BoneName, const float RelativeTime)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneLocalTransformFrameAttributeFromName: Database is nullptr.");
		return FAnimDatabaseFrameAttribute();
	}

	const int32 BoneIdx = Database->FindBoneIndex(BoneName);
	if (BoneIdx == INDEX_NONE)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneLocalTransformFrameAttributeFromName: Bone {Name} not found.", *BoneName.ToString());
		return FAnimDatabaseFrameAttribute();
	}

	return MakeBoneLocalTransformFrameAttribute(Database, FrameRanges, BoneIdx, RelativeTime);
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeBoneLocalLocationFrameAttribute(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const int32 BoneIndex, const float RelativeTime)
{
	FAnimDatabaseFrameAttribute Out;
	MakeBoneLocalFrameAttributesFromArrayViews(MakeArrayView(&Out, 1), {}, {}, {}, {}, {}, Database, FrameRanges, { BoneIndex }, {}, {}, {}, {}, {}, RelativeTime);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeBoneLocalLocationFrameAttributeFromName(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName BoneName, const float RelativeTime)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneLocalTransformFrameAttributeFromName: Database is nullptr.");
		return FAnimDatabaseFrameAttribute();
	}

	const int32 BoneIdx = Database->FindBoneIndex(BoneName);
	if (BoneIdx == INDEX_NONE)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneLocalLocationFrameAttributeFromName: Bone {Name} not found.", *BoneName.ToString());
		return FAnimDatabaseFrameAttribute();
	}

	return MakeBoneLocalLocationFrameAttribute(Database, FrameRanges, BoneIdx, RelativeTime);
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeBoneLocalRotationFrameAttribute(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const int32 BoneIndex, const float RelativeTime)
{
	FAnimDatabaseFrameAttribute Out;
	MakeBoneLocalFrameAttributesFromArrayViews({}, MakeArrayView(&Out, 1), {}, {}, {}, {}, Database, FrameRanges, {}, { BoneIndex }, {}, {}, {}, {}, RelativeTime);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeBoneLocalRotationFrameAttributeFromName(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName BoneName, const float RelativeTime)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneLocalTransformFrameAttributeFromName: Database is nullptr.");
		return FAnimDatabaseFrameAttribute();
	}

	const int32 BoneIdx = Database->FindBoneIndex(BoneName);
	if (BoneIdx == INDEX_NONE)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneLocalRotationFrameAttributeFromName: Bone {Name} not found.", *BoneName.ToString());
		return FAnimDatabaseFrameAttribute();
	}

	return MakeBoneLocalRotationFrameAttribute(Database, FrameRanges, BoneIdx, RelativeTime);
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeBoneLocalScaleFrameAttribute(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const int32 BoneIndex, const float RelativeTime)
{
	FAnimDatabaseFrameAttribute Out;
	MakeBoneLocalFrameAttributesFromArrayViews({}, {}, MakeArrayView(&Out, 1), {}, {}, {}, Database, FrameRanges, {}, {}, { BoneIndex }, {}, {}, {}, RelativeTime);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeBoneLocalScaleFrameAttributeFromName(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName BoneName, const float RelativeTime)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneLocalTransformFrameAttributeFromName: Database is nullptr.");
		return FAnimDatabaseFrameAttribute();
	}

	const int32 BoneIdx = Database->FindBoneIndex(BoneName);
	if (BoneIdx == INDEX_NONE)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneLocalScaleFrameAttributeFromName: Bone {Name} not found.", *BoneName.ToString());
		return FAnimDatabaseFrameAttribute();
	}

	return MakeBoneLocalScaleFrameAttribute(Database, FrameRanges, BoneIdx, RelativeTime);
}

void UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalFrameAttributes(
	TArray<FAnimDatabaseFrameAttribute>& OutLocationFrameAttributes,
	TArray<FAnimDatabaseFrameAttribute>& OutRotationFrameAttributes,
	TArray<FAnimDatabaseFrameAttribute>& OutScaleFrameAttributes,
	TArray<FAnimDatabaseFrameAttribute>& OutLinearVelocityFrameAttributes,
	TArray<FAnimDatabaseFrameAttribute>& OutAngularVelocityFrameAttributes,
	TArray<FAnimDatabaseFrameAttribute>& OutScalarVelocityFrameAttributes,
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const TArray<int32>& BoneLocationIndices,
	const TArray<int32>& BoneRotationIndices,
	const TArray<int32>& BoneScaleIndices,
	const TArray<int32>& BoneLinearVelocityIndices,
	const TArray<int32>& BoneAngularVelocityIndices,
	const TArray<int32>& BoneScalarVelocityIndices,
	const float RelativeTime)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalFrameAttributes);

	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneGlobalFrameAttributes: Database is nullptr.");
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutLocationFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutRotationFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutScaleFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutLinearVelocityFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutAngularVelocityFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutScalarVelocityFrameAttributes, FAnimDatabaseFrameAttribute());
		return;
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneGlobalFrameAttributes: Invalid FrameRanges.");
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutLocationFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutRotationFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutScaleFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutLinearVelocityFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutAngularVelocityFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutScalarVelocityFrameAttributes, FAnimDatabaseFrameAttribute());
		return;
	}

	if (UE::AnimDatabase::Math::AnyBoneIndicesInvalid(BoneLocationIndices) ||
		UE::AnimDatabase::Math::AnyBoneIndicesInvalid(BoneRotationIndices) ||
		UE::AnimDatabase::Math::AnyBoneIndicesInvalid(BoneScaleIndices) ||
		UE::AnimDatabase::Math::AnyBoneIndicesInvalid(BoneLinearVelocityIndices) ||
		UE::AnimDatabase::Math::AnyBoneIndicesInvalid(BoneAngularVelocityIndices) ||
		UE::AnimDatabase::Math::AnyBoneIndicesInvalid(BoneScalarVelocityIndices))
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneGlobalFrameAttributes: Invalid Bone Indices.");
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutLocationFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutRotationFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutScaleFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutLinearVelocityFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutAngularVelocityFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutScalarVelocityFrameAttributes, FAnimDatabaseFrameAttribute());
		return;
	}

	OutLocationFrameAttributes.SetNum(BoneLocationIndices.Num());
	OutRotationFrameAttributes.SetNum(BoneRotationIndices.Num());
	OutScaleFrameAttributes.SetNum(BoneScaleIndices.Num());
	OutLinearVelocityFrameAttributes.SetNum(BoneLinearVelocityIndices.Num());
	OutAngularVelocityFrameAttributes.SetNum(BoneAngularVelocityIndices.Num());
	OutScalarVelocityFrameAttributes.SetNum(BoneScalarVelocityIndices.Num());

	MakeBoneGlobalFrameAttributesFromArrayViews(
		OutLocationFrameAttributes,
		OutRotationFrameAttributes,
		OutScaleFrameAttributes,
		OutLinearVelocityFrameAttributes,
		OutAngularVelocityFrameAttributes,
		OutScalarVelocityFrameAttributes,
		Database,
		FrameRanges,
		BoneLocationIndices,
		BoneRotationIndices,
		BoneScaleIndices,
		BoneLinearVelocityIndices,
		BoneAngularVelocityIndices,
		BoneScalarVelocityIndices,
		RelativeTime);
}

void UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalFrameAttributesFromArrayViews(
	const TArrayView<FAnimDatabaseFrameAttribute> OutLocationFrameAttributes,
	const TArrayView<FAnimDatabaseFrameAttribute> OutRotationFrameAttributes,
	const TArrayView<FAnimDatabaseFrameAttribute> OutScaleFrameAttributes,
	const TArrayView<FAnimDatabaseFrameAttribute> OutLinearVelocityFrameAttributes,
	const TArrayView<FAnimDatabaseFrameAttribute> OutAngularVelocityFrameAttributes,
	const TArrayView<FAnimDatabaseFrameAttribute> OutScalarVelocityFrameAttributes,
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const TArrayView<const int32> BoneLocationIndices,
	const TArrayView<const int32> BoneRotationIndices,
	const TArrayView<const int32> BoneScaleIndices,
	const TArrayView<const int32> BoneLinearVelocityIndices,
	const TArrayView<const int32> BoneAngularVelocityIndices,
	const TArrayView<const int32> BoneScalarVelocityIndices,
	const float RelativeTime)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneGlobalFrameAttributesFromArrayViews: Database is nullptr.");
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutLocationFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutRotationFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutScaleFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutLinearVelocityFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutAngularVelocityFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutScalarVelocityFrameAttributes, FAnimDatabaseFrameAttribute());
		return;
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneGlobalFrameAttributesFromArrayViews: Invalid FrameRanges.");
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutLocationFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutRotationFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutScaleFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutLinearVelocityFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutAngularVelocityFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutScalarVelocityFrameAttributes, FAnimDatabaseFrameAttribute());
		return;
	}

	if (UE::AnimDatabase::Math::AnyBoneIndicesInvalid(BoneLocationIndices) ||
		UE::AnimDatabase::Math::AnyBoneIndicesInvalid(BoneRotationIndices) ||
		UE::AnimDatabase::Math::AnyBoneIndicesInvalid(BoneScaleIndices) ||
		UE::AnimDatabase::Math::AnyBoneIndicesInvalid(BoneLinearVelocityIndices) ||
		UE::AnimDatabase::Math::AnyBoneIndicesInvalid(BoneAngularVelocityIndices) ||
		UE::AnimDatabase::Math::AnyBoneIndicesInvalid(BoneScalarVelocityIndices))
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneGlobalFrameAttributesFromArrayViews: Invalid Bone Indices.");
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutLocationFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutRotationFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutScaleFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutLinearVelocityFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutAngularVelocityFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutScalarVelocityFrameAttributes, FAnimDatabaseFrameAttribute());
		return;
	}

	const int32 BoneLocationIndexNum = BoneLocationIndices.Num();
	const int32 BoneRotationIndexNum = BoneRotationIndices.Num();
	const int32 BoneScaleIndexNum = BoneScaleIndices.Num();
	const int32 BoneLinearVelocityIndexNum = BoneLinearVelocityIndices.Num();
	const int32 BoneAngularVelocityIndexNum = BoneAngularVelocityIndices.Num();
	const int32 BoneScalarVelocityIndexNum = BoneScalarVelocityIndices.Num();
	const int32 TotalFrameNum = FrameRanges.FrameRangeSet->GetTotalFrameNum();

	check(OutLocationFrameAttributes.Num() == BoneLocationIndexNum);
	check(OutRotationFrameAttributes.Num() == BoneRotationIndexNum);
	check(OutScaleFrameAttributes.Num() == BoneScaleIndexNum);
	check(OutLinearVelocityFrameAttributes.Num() == BoneLinearVelocityIndexNum);
	check(OutAngularVelocityFrameAttributes.Num() == BoneAngularVelocityIndexNum);
	check(OutScalarVelocityFrameAttributes.Num() == BoneScalarVelocityIndexNum);

	for (int32 BoneIdx = 0; BoneIdx < BoneLocationIndexNum; BoneIdx++)
	{
		OutLocationFrameAttributes[BoneIdx] = MakeEmptyFrameAttribute();
		OutLocationFrameAttributes[BoneIdx].Type = EAnimDatabaseAttributeType::Location;
		OutLocationFrameAttributes[BoneIdx].FrameAttribute->FrameRangeSet = *FrameRanges.FrameRangeSet;
		OutLocationFrameAttributes[BoneIdx].FrameAttribute->AttributeData.SetNumUninitialized({ AttributeTypeSize(EAnimDatabaseAttributeType::Location), TotalFrameNum });
	}

	for (int32 BoneIdx = 0; BoneIdx < BoneRotationIndexNum; BoneIdx++)
	{
		OutRotationFrameAttributes[BoneIdx] = MakeEmptyFrameAttribute();
		OutRotationFrameAttributes[BoneIdx].Type = EAnimDatabaseAttributeType::Rotation;
		OutRotationFrameAttributes[BoneIdx].FrameAttribute->FrameRangeSet = *FrameRanges.FrameRangeSet;
		OutRotationFrameAttributes[BoneIdx].FrameAttribute->AttributeData.SetNumUninitialized({ AttributeTypeSize(EAnimDatabaseAttributeType::Rotation), TotalFrameNum });
	}

	for (int32 BoneIdx = 0; BoneIdx < BoneScaleIndexNum; BoneIdx++)
	{
		OutScaleFrameAttributes[BoneIdx] = MakeEmptyFrameAttribute();
		OutScaleFrameAttributes[BoneIdx].Type = EAnimDatabaseAttributeType::Scale;
		OutScaleFrameAttributes[BoneIdx].FrameAttribute->FrameRangeSet = *FrameRanges.FrameRangeSet;
		OutScaleFrameAttributes[BoneIdx].FrameAttribute->AttributeData.SetNumUninitialized({ AttributeTypeSize(EAnimDatabaseAttributeType::Scale), TotalFrameNum });
	}

	for (int32 BoneIdx = 0; BoneIdx < BoneLinearVelocityIndexNum; BoneIdx++)
	{
		OutLinearVelocityFrameAttributes[BoneIdx] = MakeEmptyFrameAttribute();
		OutLinearVelocityFrameAttributes[BoneIdx].Type = EAnimDatabaseAttributeType::LinearVelocity;
		OutLinearVelocityFrameAttributes[BoneIdx].FrameAttribute->FrameRangeSet = *FrameRanges.FrameRangeSet;
		OutLinearVelocityFrameAttributes[BoneIdx].FrameAttribute->AttributeData.SetNumUninitialized({ AttributeTypeSize(EAnimDatabaseAttributeType::LinearVelocity), TotalFrameNum });
	}

	for (int32 BoneIdx = 0; BoneIdx < BoneAngularVelocityIndexNum; BoneIdx++)
	{
		OutAngularVelocityFrameAttributes[BoneIdx] = MakeEmptyFrameAttribute();
		OutAngularVelocityFrameAttributes[BoneIdx].Type = EAnimDatabaseAttributeType::AngularVelocity;
		OutAngularVelocityFrameAttributes[BoneIdx].FrameAttribute->FrameRangeSet = *FrameRanges.FrameRangeSet;
		OutAngularVelocityFrameAttributes[BoneIdx].FrameAttribute->AttributeData.SetNumUninitialized({ AttributeTypeSize(EAnimDatabaseAttributeType::AngularVelocity), TotalFrameNum });
	}

	for (int32 BoneIdx = 0; BoneIdx < BoneScalarVelocityIndexNum; BoneIdx++)
	{
		OutScalarVelocityFrameAttributes[BoneIdx] = MakeEmptyFrameAttribute();
		OutScalarVelocityFrameAttributes[BoneIdx].Type = EAnimDatabaseAttributeType::ScalarVelocity;
		OutScalarVelocityFrameAttributes[BoneIdx].FrameAttribute->FrameRangeSet = *FrameRanges.FrameRangeSet;
		OutScalarVelocityFrameAttributes[BoneIdx].FrameAttribute->AttributeData.SetNumUninitialized({ AttributeTypeSize(EAnimDatabaseAttributeType::ScalarVelocity), TotalFrameNum });
	}

	// Find required Indices

	TArray<int32> BoneParents;
	BoneParents.SetNumUninitialized(Database->GetBoneNum());
	Database->GetBoneParents(BoneParents);

	TArray<int32> RequiredIndices;
	TArray<int32> CurrentIndices;
	TArray<int32> Ascendants;

	for (int32 BoneIdx = 0; BoneIdx < BoneLocationIndexNum; BoneIdx++)
	{
		UE::AnimDatabase::Math::BoneAscendantsInclusive(Ascendants, BoneLocationIndices[BoneIdx], BoneParents);
		CurrentIndices = RequiredIndices;
		UE::AnimDatabase::Math::BoneUnion(RequiredIndices, CurrentIndices, Ascendants);
	}

	for (int32 BoneIdx = 0; BoneIdx < BoneRotationIndexNum; BoneIdx++)
	{
		UE::AnimDatabase::Math::BoneAscendantsInclusive(Ascendants, BoneRotationIndices[BoneIdx], BoneParents);
		CurrentIndices = RequiredIndices;
		UE::AnimDatabase::Math::BoneUnion(RequiredIndices, CurrentIndices, Ascendants);
	}

	for (int32 BoneIdx = 0; BoneIdx < BoneScaleIndexNum; BoneIdx++)
	{
		UE::AnimDatabase::Math::BoneAscendantsInclusive(Ascendants, BoneScaleIndices[BoneIdx], BoneParents);
		CurrentIndices = RequiredIndices;
		UE::AnimDatabase::Math::BoneUnion(RequiredIndices, CurrentIndices, Ascendants);
	}

	for (int32 BoneIdx = 0; BoneIdx < BoneLinearVelocityIndexNum; BoneIdx++)
	{
		UE::AnimDatabase::Math::BoneAscendantsInclusive(Ascendants, BoneLinearVelocityIndices[BoneIdx], BoneParents);
		CurrentIndices = RequiredIndices;
		UE::AnimDatabase::Math::BoneUnion(RequiredIndices, CurrentIndices, Ascendants);
	}

	for (int32 BoneIdx = 0; BoneIdx < BoneAngularVelocityIndexNum; BoneIdx++)
	{
		UE::AnimDatabase::Math::BoneAscendantsInclusive(Ascendants, BoneAngularVelocityIndices[BoneIdx], BoneParents);
		CurrentIndices = RequiredIndices;
		UE::AnimDatabase::Math::BoneUnion(RequiredIndices, CurrentIndices, Ascendants);
	}

	for (int32 BoneIdx = 0; BoneIdx < BoneScalarVelocityIndexNum; BoneIdx++)
	{
		UE::AnimDatabase::Math::BoneAscendantsInclusive(Ascendants, BoneScalarVelocityIndices[BoneIdx], BoneParents);
		CurrentIndices = RequiredIndices;
		UE::AnimDatabase::Math::BoneUnion(RequiredIndices, CurrentIndices, Ascendants);
	}

	// Extract Pose Data

	const int32 RequiredBoneNum = RequiredIndices.Num();

	UE::AnimDatabase::FPoseData PoseData;
	PoseData.Resize(TotalFrameNum, RequiredBoneNum, {}, {});

	Database->WaitForCompressionOnAnimSequencesFromArrayView(FrameRanges.FrameRangeSet->GetEntrySequences());

	UE::Learning::FrameRangeSet::ParallelForEachRange(*FrameRanges.FrameRangeSet, [&FrameRanges, &PoseData, &Database, &RequiredIndices, RelativeTime](
		const int32 TotalRangeIdx,
		const int32 EntryIdx,
		const int32 RangeIdx) {

			const int32 FrameOffset = FrameRanges.FrameRangeSet->GetEntryRangeOffset(EntryIdx, RangeIdx);
			const int32 FrameNum = FrameRanges.FrameRangeSet->GetEntryRangeLength(EntryIdx, RangeIdx);
			const int32 Sequence = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);
			const int32 StartFrame = FrameRanges.FrameRangeSet->GetEntryRangeStart(EntryIdx, RangeIdx);

			Database->GetPoseSubsetData(
				PoseData.Slice(FrameOffset, FrameNum),
				Sequence,
				StartFrame + FMath::RoundToInt(RelativeTime * Database->GetFrameRate().AsDecimal()),
				RequiredIndices);
		});

	UE::AnimDatabase::FPoseGlobalBoneData GlobalPoseData;
	GlobalPoseData.Resize(TotalFrameNum, RequiredBoneNum);

	UE::Learning::SlicedParallelFor(TotalFrameNum, 512, [&GlobalPoseData, &PoseData, &BoneParents, &RequiredIndices](const int32 SliceStart, const int32 SliceLength)
		{
			UE::AnimDatabase::PoseData::ForwardKinematics(
				GlobalPoseData.Slice(SliceStart, SliceLength),
				PoseData.LocalBoneData.ConstSlice(SliceStart, SliceLength),
				PoseData.RootData.ConstSlice(SliceStart, SliceLength),
				BoneParents,
				RequiredIndices);
		});

	TArray<int32, TInlineAllocator<8>> RequiredBoneLocationIndices;
	TArray<int32, TInlineAllocator<8>> RequiredBoneRotationIndices;
	TArray<int32, TInlineAllocator<8>> RequiredBoneScaleIndices;
	TArray<int32, TInlineAllocator<8>> RequiredBoneLinearVelocityIndices;
	TArray<int32, TInlineAllocator<8>> RequiredBoneAngularVelocityIndices;
	TArray<int32, TInlineAllocator<8>> RequiredBoneScalarVelocityIndices;

	RequiredBoneLocationIndices.SetNumUninitialized(BoneLocationIndexNum);
	RequiredBoneRotationIndices.SetNumUninitialized(BoneRotationIndexNum);
	RequiredBoneScaleIndices.SetNumUninitialized(BoneScaleIndexNum);
	RequiredBoneLinearVelocityIndices.SetNumUninitialized(BoneLinearVelocityIndexNum);
	RequiredBoneAngularVelocityIndices.SetNumUninitialized(BoneAngularVelocityIndexNum);
	RequiredBoneScalarVelocityIndices.SetNumUninitialized(BoneScalarVelocityIndexNum);

	UE::AnimDatabase::Math::BoneFindIndicesOf(RequiredBoneLocationIndices, BoneLocationIndices, RequiredIndices);
	UE::AnimDatabase::Math::BoneFindIndicesOf(RequiredBoneRotationIndices, BoneRotationIndices, RequiredIndices);
	UE::AnimDatabase::Math::BoneFindIndicesOf(RequiredBoneScaleIndices, BoneScaleIndices, RequiredIndices);
	UE::AnimDatabase::Math::BoneFindIndicesOf(RequiredBoneLinearVelocityIndices, BoneLinearVelocityIndices, RequiredIndices);
	UE::AnimDatabase::Math::BoneFindIndicesOf(RequiredBoneAngularVelocityIndices, BoneAngularVelocityIndices, RequiredIndices);
	UE::AnimDatabase::Math::BoneFindIndicesOf(RequiredBoneScalarVelocityIndices, BoneScalarVelocityIndices, RequiredIndices);

	UE::Learning::SlicedParallelFor(TotalFrameNum, 512, [
		&GlobalPoseData,
			&RequiredBoneLocationIndices,
			&RequiredBoneRotationIndices,
			&RequiredBoneScaleIndices,
			&RequiredBoneLinearVelocityIndices,
			&RequiredBoneAngularVelocityIndices,
			&RequiredBoneScalarVelocityIndices,
			&OutLocationFrameAttributes,
			&OutRotationFrameAttributes,
			&OutScaleFrameAttributes,
			&OutLinearVelocityFrameAttributes,
			&OutAngularVelocityFrameAttributes,
			&OutScalarVelocityFrameAttributes](const int32 SliceStart, const int32 SliceLength)
		{
			for (int32 FrameIdx = SliceStart; FrameIdx < SliceStart + SliceLength; FrameIdx++)
			{
				const int32 BoneLocationIndexNum = RequiredBoneLocationIndices.Num();
				const int32 BoneRotationIndexNum = RequiredBoneRotationIndices.Num();
				const int32 BoneScaleIndexNum = RequiredBoneScaleIndices.Num();
				const int32 BoneLinearVelocityIndexNum = RequiredBoneLinearVelocityIndices.Num();
				const int32 BoneAngularVelocityIndexNum = RequiredBoneAngularVelocityIndices.Num();
				const int32 BoneScalarVelocityIndexNum = RequiredBoneScalarVelocityIndices.Num();

				for (int32 BoneIdx = 0; BoneIdx < BoneLocationIndexNum; BoneIdx++)
				{
					const FVector Location = GlobalPoseData.BoneLocations[FrameIdx][RequiredBoneLocationIndices[BoneIdx]];
					OutLocationFrameAttributes[BoneIdx].FrameAttribute->AttributeData[0][FrameIdx] = Location.X;
					OutLocationFrameAttributes[BoneIdx].FrameAttribute->AttributeData[1][FrameIdx] = Location.Y;
					OutLocationFrameAttributes[BoneIdx].FrameAttribute->AttributeData[2][FrameIdx] = Location.Z;
				}

				for (int32 BoneIdx = 0; BoneIdx < BoneRotationIndexNum; BoneIdx++)
				{
					const FQuat4f Rotation = GlobalPoseData.BoneRotations[FrameIdx][RequiredBoneRotationIndices[BoneIdx]];
					OutRotationFrameAttributes[BoneIdx].FrameAttribute->AttributeData[0][FrameIdx] = Rotation.X;
					OutRotationFrameAttributes[BoneIdx].FrameAttribute->AttributeData[1][FrameIdx] = Rotation.Y;
					OutRotationFrameAttributes[BoneIdx].FrameAttribute->AttributeData[2][FrameIdx] = Rotation.Z;
					OutRotationFrameAttributes[BoneIdx].FrameAttribute->AttributeData[3][FrameIdx] = Rotation.W;
				}

				for (int32 BoneIdx = 0; BoneIdx < BoneScaleIndexNum; BoneIdx++)
				{
					const FVector3f Scale = GlobalPoseData.BoneScales[FrameIdx][RequiredBoneScaleIndices[BoneIdx]];
					OutScaleFrameAttributes[BoneIdx].FrameAttribute->AttributeData[0][FrameIdx] = Scale.X;
					OutScaleFrameAttributes[BoneIdx].FrameAttribute->AttributeData[1][FrameIdx] = Scale.Y;
					OutScaleFrameAttributes[BoneIdx].FrameAttribute->AttributeData[2][FrameIdx] = Scale.Z;
				}

				for (int32 BoneIdx = 0; BoneIdx < BoneLinearVelocityIndexNum; BoneIdx++)
				{
					const FVector3f LinearVelocity = GlobalPoseData.BoneLinearVelocities[FrameIdx][RequiredBoneLinearVelocityIndices[BoneIdx]];
					OutLinearVelocityFrameAttributes[BoneIdx].FrameAttribute->AttributeData[0][FrameIdx] = LinearVelocity.X;
					OutLinearVelocityFrameAttributes[BoneIdx].FrameAttribute->AttributeData[1][FrameIdx] = LinearVelocity.Y;
					OutLinearVelocityFrameAttributes[BoneIdx].FrameAttribute->AttributeData[2][FrameIdx] = LinearVelocity.Z;
				}

				for (int32 BoneIdx = 0; BoneIdx < BoneAngularVelocityIndexNum; BoneIdx++)
				{
					const FVector3f AngularVelocity = GlobalPoseData.BoneAngularVelocities[FrameIdx][RequiredBoneAngularVelocityIndices[BoneIdx]];
					OutAngularVelocityFrameAttributes[BoneIdx].FrameAttribute->AttributeData[0][FrameIdx] = AngularVelocity.X;
					OutAngularVelocityFrameAttributes[BoneIdx].FrameAttribute->AttributeData[1][FrameIdx] = AngularVelocity.Y;
					OutAngularVelocityFrameAttributes[BoneIdx].FrameAttribute->AttributeData[2][FrameIdx] = AngularVelocity.Z;
				}

				for (int32 BoneIdx = 0; BoneIdx < BoneScalarVelocityIndexNum; BoneIdx++)
				{
					const FVector3f ScalarVelocity = GlobalPoseData.BoneScalarVelocities[FrameIdx][RequiredBoneScalarVelocityIndices[BoneIdx]];
					OutScalarVelocityFrameAttributes[BoneIdx].FrameAttribute->AttributeData[0][FrameIdx] = ScalarVelocity.X;
					OutScalarVelocityFrameAttributes[BoneIdx].FrameAttribute->AttributeData[1][FrameIdx] = ScalarVelocity.Y;
					OutScalarVelocityFrameAttributes[BoneIdx].FrameAttribute->AttributeData[2][FrameIdx] = ScalarVelocity.Z;
				}
			}
		});

		for (int32 BoneIdx = 0; BoneIdx < BoneLocationIndexNum; BoneIdx++)
		{
			UE::Learning::Array::Check(OutLocationFrameAttributes[BoneIdx].FrameAttribute->AttributeData);
		}

		for (int32 BoneIdx = 0; BoneIdx < BoneRotationIndexNum; BoneIdx++)
		{
			UE::Learning::Array::Check(OutRotationFrameAttributes[BoneIdx].FrameAttribute->AttributeData);
		}

		for (int32 BoneIdx = 0; BoneIdx < BoneScaleIndexNum; BoneIdx++)
		{
			UE::Learning::Array::Check(OutScaleFrameAttributes[BoneIdx].FrameAttribute->AttributeData);
		}

		for (int32 BoneIdx = 0; BoneIdx < BoneLinearVelocityIndexNum; BoneIdx++)
		{
			UE::Learning::Array::Check(OutLinearVelocityFrameAttributes[BoneIdx].FrameAttribute->AttributeData);
		}

		for (int32 BoneIdx = 0; BoneIdx < BoneAngularVelocityIndexNum; BoneIdx++)
		{
			UE::Learning::Array::Check(OutAngularVelocityFrameAttributes[BoneIdx].FrameAttribute->AttributeData);
		}

		for (int32 BoneIdx = 0; BoneIdx < BoneScalarVelocityIndexNum; BoneIdx++)
		{
			UE::Learning::Array::Check(OutScalarVelocityFrameAttributes[BoneIdx].FrameAttribute->AttributeData);
		}
}

void UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalTransformFrameAttributes(
	TArray<FAnimDatabaseFrameAttribute>& OutTransformFrameAttributes,
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const TArray<int32>& BoneIndices,
	const float RelativeTime)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalFrameAttributes);

	OutTransformFrameAttributes.SetNum(BoneIndices.Num());

	MakeBoneGlobalFrameAttributesFromArrayViews(
		OutTransformFrameAttributes,
		Database,
		FrameRanges,
		BoneIndices,
		RelativeTime);
}

void UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalFrameAttributesFromArrayViews(
	const TArrayView<FAnimDatabaseFrameAttribute> OutTransformFrameAttributes,
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const TArrayView<const int32> BoneIndices,
	const float RelativeTime)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneGlobalFrameAttributesFromArrayViews: Database is nullptr.");
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutTransformFrameAttributes, FAnimDatabaseFrameAttribute());
		return;
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneGlobalFrameAttributesFromArrayViews: Invalid FrameRanges.");
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutTransformFrameAttributes, FAnimDatabaseFrameAttribute());
		return;
	}

	if (UE::AnimDatabase::Math::AnyBoneIndicesInvalid(BoneIndices))
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneGlobalFrameAttributesFromArrayViews: Invalid Bone Indices.");
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutTransformFrameAttributes, FAnimDatabaseFrameAttribute());
		return;
	}

	const int32 BoneIndexNum = BoneIndices.Num();
	const int32 TotalFrameNum = FrameRanges.FrameRangeSet->GetTotalFrameNum();

	check(OutTransformFrameAttributes.Num() == BoneIndexNum);

	for (int32 BoneIdx = 0; BoneIdx < BoneIndexNum; BoneIdx++)
	{
		OutTransformFrameAttributes[BoneIdx] = MakeEmptyFrameAttribute();
		OutTransformFrameAttributes[BoneIdx].Type = EAnimDatabaseAttributeType::Transform;
		OutTransformFrameAttributes[BoneIdx].FrameAttribute->FrameRangeSet = *FrameRanges.FrameRangeSet;
		OutTransformFrameAttributes[BoneIdx].FrameAttribute->AttributeData.SetNumUninitialized({ AttributeTypeSize(EAnimDatabaseAttributeType::Transform), TotalFrameNum });
	}

	// Find required Indices

	TArray<int32> BoneParents;
	BoneParents.SetNumUninitialized(Database->GetBoneNum());
	Database->GetBoneParents(BoneParents);

	TArray<int32> RequiredIndices;
	TArray<int32> CurrentIndices;
	TArray<int32> Ascendants;

	for (int32 BoneIdx = 0; BoneIdx < BoneIndexNum; BoneIdx++)
	{
		UE::AnimDatabase::Math::BoneAscendantsInclusive(Ascendants, BoneIndices[BoneIdx], BoneParents);
		CurrentIndices = RequiredIndices;
		UE::AnimDatabase::Math::BoneUnion(RequiredIndices, CurrentIndices, Ascendants);
	}

	// Extract Pose Data

	const int32 RequiredBoneNum = RequiredIndices.Num();

	UE::AnimDatabase::FPoseData PoseData;
	PoseData.Resize(TotalFrameNum, RequiredBoneNum, {}, {});

	Database->WaitForCompressionOnAnimSequencesFromArrayView(FrameRanges.FrameRangeSet->GetEntrySequences());

	UE::Learning::FrameRangeSet::ParallelForEachRange(*FrameRanges.FrameRangeSet, [&FrameRanges, &PoseData, &Database, &RequiredIndices, RelativeTime](
		const int32 TotalRangeIdx,
		const int32 EntryIdx,
		const int32 RangeIdx) {

			const int32 FrameOffset = FrameRanges.FrameRangeSet->GetEntryRangeOffset(EntryIdx, RangeIdx);
			const int32 FrameNum = FrameRanges.FrameRangeSet->GetEntryRangeLength(EntryIdx, RangeIdx);
			const int32 Sequence = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);
			const int32 StartFrame = FrameRanges.FrameRangeSet->GetEntryRangeStart(EntryIdx, RangeIdx);

			Database->GetPoseSubsetData(
				PoseData.Slice(FrameOffset, FrameNum),
				Sequence,
				StartFrame + FMath::RoundToInt(RelativeTime * Database->GetFrameRate().AsDecimal()),
				RequiredIndices);
		});

	UE::AnimDatabase::FPoseGlobalBoneData GlobalPoseData;
	GlobalPoseData.Resize(TotalFrameNum, RequiredBoneNum);

	UE::Learning::SlicedParallelFor(TotalFrameNum, 512, [&GlobalPoseData, &PoseData, &BoneParents, &RequiredIndices](const int32 SliceStart, const int32 SliceLength)
		{
			UE::AnimDatabase::PoseData::ForwardKinematics(
				GlobalPoseData.Slice(SliceStart, SliceLength),
				PoseData.LocalBoneData.ConstSlice(SliceStart, SliceLength),
				PoseData.RootData.ConstSlice(SliceStart, SliceLength),
				BoneParents,
				RequiredIndices);
		});

	TArray<int32, TInlineAllocator<8>> RequiredBoneIndices;
	RequiredBoneIndices.SetNumUninitialized(BoneIndexNum);
	UE::AnimDatabase::Math::BoneFindIndicesOf(RequiredBoneIndices, BoneIndices, RequiredIndices);

	UE::Learning::SlicedParallelFor(TotalFrameNum, 512, [
		&GlobalPoseData,
			&RequiredBoneIndices,
			&OutTransformFrameAttributes](const int32 SliceStart, const int32 SliceLength)
		{
			for (int32 FrameIdx = SliceStart; FrameIdx < SliceStart + SliceLength; FrameIdx++)
			{
				const int32 BoneIndexNum = RequiredBoneIndices.Num();

				for (int32 BoneIdx = 0; BoneIdx < BoneIndexNum; BoneIdx++)
				{
					const FVector Location = GlobalPoseData.BoneLocations[FrameIdx][RequiredBoneIndices[BoneIdx]];
					const FQuat4f Rotation = GlobalPoseData.BoneRotations[FrameIdx][RequiredBoneIndices[BoneIdx]];
					const FVector3f Scale = GlobalPoseData.BoneScales[FrameIdx][RequiredBoneIndices[BoneIdx]];
					OutTransformFrameAttributes[BoneIdx].FrameAttribute->AttributeData[0][FrameIdx] = Location.X;
					OutTransformFrameAttributes[BoneIdx].FrameAttribute->AttributeData[1][FrameIdx] = Location.Y;
					OutTransformFrameAttributes[BoneIdx].FrameAttribute->AttributeData[2][FrameIdx] = Location.Z;
					OutTransformFrameAttributes[BoneIdx].FrameAttribute->AttributeData[3][FrameIdx] = Rotation.X;
					OutTransformFrameAttributes[BoneIdx].FrameAttribute->AttributeData[4][FrameIdx] = Rotation.Y;
					OutTransformFrameAttributes[BoneIdx].FrameAttribute->AttributeData[5][FrameIdx] = Rotation.Z;
					OutTransformFrameAttributes[BoneIdx].FrameAttribute->AttributeData[6][FrameIdx] = Rotation.W;
					OutTransformFrameAttributes[BoneIdx].FrameAttribute->AttributeData[7][FrameIdx] = Scale.X;
					OutTransformFrameAttributes[BoneIdx].FrameAttribute->AttributeData[8][FrameIdx] = Scale.Y;
					OutTransformFrameAttributes[BoneIdx].FrameAttribute->AttributeData[9][FrameIdx] = Scale.Z;
				}

			}
		});

	for (int32 BoneIdx = 0; BoneIdx < BoneIndexNum; BoneIdx++)
	{
		UE::Learning::Array::Check(OutTransformFrameAttributes[BoneIdx].FrameAttribute->AttributeData);
	}
}

void UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalTransformFrameAttributesFromNames(
	TArray<FAnimDatabaseFrameAttribute>& OutTransformFrameAttributes,
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const TArray<FName>& BoneNames,
	const float RelativeTime)
{
	OutTransformFrameAttributes.SetNum(BoneNames.Num());
	MakeBoneGlobalTransformFrameAttributesFromNamesArrayView(
		OutTransformFrameAttributes,
		Database,
		FrameRanges,
		BoneNames,
		RelativeTime);
}

void UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalTransformFrameAttributesFromNamesArrayView(
	const TArrayView<FAnimDatabaseFrameAttribute> OutTransformFrameAttributes,
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const TArrayView<const FName> BoneNames,
	const float RelativeTime)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneGlobalTransformFrameAttributesFromNamesArrayView: Database is nullptr.");
		return;
	}

	TArray<int32, TInlineAllocator<32>> BoneIndices;
	const int32 BoneNum = BoneNames.Num();
	BoneIndices.SetNumUninitialized(BoneNum);
	Database->FindBoneIndicesFromArrayViews(BoneIndices, BoneNames);

	MakeBoneGlobalFrameAttributesFromArrayViews(
		OutTransformFrameAttributes,
		Database,
		FrameRanges,
		BoneIndices,
		RelativeTime);
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalLocationFrameAttribute(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const int32 BoneIndex, const float RelativeTime)
{
	FAnimDatabaseFrameAttribute OutFrameAttribute;

	MakeBoneGlobalFrameAttributesFromArrayViews(
		MakeArrayView(&OutFrameAttribute, 1), {}, {},
		{}, {}, {},
		Database,
		FrameRanges,
		{ BoneIndex }, {}, {},
		{}, {}, {},
		RelativeTime);

	return OutFrameAttribute;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalLocationFrameAttributeFromName(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName BoneName, const float RelativeTime)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneGlobalLocationFrameAttributeFromName: Database is nullptr.");
		return FAnimDatabaseFrameAttribute();
	}

	const int32 BoneIdx = Database->FindBoneIndex(BoneName);
	if (BoneIdx == INDEX_NONE)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneGlobalLocationFrameAttributeFromName: Bone {Name} not found.", *BoneName.ToString());
		return FAnimDatabaseFrameAttribute();
	}

	return MakeBoneGlobalLocationFrameAttribute(Database, FrameRanges, BoneIdx, RelativeTime);
}

void UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalLocationFrameAttributes(TArray<FAnimDatabaseFrameAttribute>& OutLocationFrameAttributes, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArray<int32>& BoneIndices, const float RelativeTime)
{
	OutLocationFrameAttributes.SetNum(BoneIndices.Num());
	MakeBoneGlobalLocationFrameAttributesFromArrayViews(OutLocationFrameAttributes, Database, FrameRanges, BoneIndices, RelativeTime);
}

void UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalLocationFrameAttributesFromArrayViews(const TArrayView<FAnimDatabaseFrameAttribute> OutLocationFrameAttributes, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const int32> BoneIndices, const float RelativeTime)
{
	MakeBoneGlobalFrameAttributesFromArrayViews(
		OutLocationFrameAttributes, {}, {},
		{}, {}, {},
		Database,
		FrameRanges,
		BoneIndices, {}, {},
		{}, {}, {},
		RelativeTime);
}

void UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalLocationFrameAttributesFromNames(TArray<FAnimDatabaseFrameAttribute>& OutLocationFrameAttributes, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArray<FName>& BoneNames, const float RelativeTime)
{
	OutLocationFrameAttributes.SetNum(BoneNames.Num());
	MakeBoneGlobalLocationFrameAttributesFromNamesArrayView(OutLocationFrameAttributes, Database, FrameRanges, BoneNames, RelativeTime);
}

void UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalLocationFrameAttributesFromNamesArrayView(const TArrayView<FAnimDatabaseFrameAttribute> OutLocationFrameAttributes, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const FName> BoneNames, const float RelativeTime)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneGlobalLocationFrameAttributesFromNamesArrayView: Database is nullptr.");
		return;
	}

	TArray<int32, TInlineAllocator<32>> BoneIndices;
	const int32 BoneNum = BoneNames.Num();
	BoneIndices.SetNumUninitialized(BoneNum);
	Database->FindBoneIndicesFromArrayViews(BoneIndices, BoneNames);

	MakeBoneGlobalLocationFrameAttributesFromArrayViews(
		OutLocationFrameAttributes,
		Database,
		FrameRanges,
		BoneIndices,
		RelativeTime);
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalRotationFrameAttribute(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const int32 BoneIndex, const float RelativeTime)
{
	FAnimDatabaseFrameAttribute OutFrameAttribute;

	MakeBoneGlobalFrameAttributesFromArrayViews(
		{}, MakeArrayView(&OutFrameAttribute, 1), {},
		{}, {}, {},
		Database,
		FrameRanges,
		{}, { BoneIndex }, {},
		{}, {}, {},
		RelativeTime);

	return OutFrameAttribute;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalRotationFrameAttributeFromName(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName BoneName, const float RelativeTime)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneGlobalRotationFrameAttributeFromName: Database is nullptr.");
		return FAnimDatabaseFrameAttribute();
	}

	const int32 BoneIdx = Database->FindBoneIndex(BoneName);
	if (BoneIdx == INDEX_NONE)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneGlobalRotationFrameAttributeFromName: Bone {Name} not found.", *BoneName.ToString());
		return FAnimDatabaseFrameAttribute();
	}

	return MakeBoneGlobalRotationFrameAttribute(Database, FrameRanges, BoneIdx, RelativeTime);
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalDirectionFrameAttribute(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const int32 BoneIndex, const FVector LocalDirection, const float RelativeTime)
{
	FAnimDatabaseFrameAttribute OutFrameAttribute;

	MakeBoneGlobalFrameAttributesFromArrayViews(
		{}, MakeArrayView(&OutFrameAttribute, 1), {},
		{}, {}, {},
		Database,
		FrameRanges,
		{}, { BoneIndex }, {},
		{}, {}, {},
		RelativeTime);

	return FrameAttributeRotateDirection(OutFrameAttribute, LocalDirection);
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalDirectionFrameAttributeFromName(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName BoneName, const FVector LocalDirection, const float RelativeTime)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneGlobalDirectionFrameAttributeFromName: Database is nullptr.");
		return FAnimDatabaseFrameAttribute();
	}

	const int32 BoneIdx = Database->FindBoneIndex(BoneName);
	if (BoneIdx == INDEX_NONE)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneGlobalDirectionFrameAttributeFromName: Bone {Name} not found.", *BoneName.ToString());
		return FAnimDatabaseFrameAttribute();
	}

	return MakeBoneGlobalDirectionFrameAttribute(Database, FrameRanges, BoneIdx, LocalDirection, RelativeTime);
}

void UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalDirectionFrameAttributes(TArray<FAnimDatabaseFrameAttribute>& OutDirectionFrameAttributes, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArray<int32>& BoneIndices, const FVector LocalDirection, const float RelativeTime)
{
	OutDirectionFrameAttributes.SetNum(BoneIndices.Num());
	MakeBoneGlobalDirectionFrameAttributesFromArrayViews(OutDirectionFrameAttributes, Database, FrameRanges, BoneIndices, LocalDirection, RelativeTime);
}

void UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalDirectionFrameAttributesFromArrayViews(TArrayView<FAnimDatabaseFrameAttribute> OutDirectionFrameAttributes, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const int32> BoneIndices, const FVector LocalDirection, const float RelativeTime)
{
	if (BoneIndices.Num() != OutDirectionFrameAttributes.Num())
	{
		return;
	}

	TArray<FAnimDatabaseFrameAttribute, TInlineAllocator<32>> RotationAttributes;
	RotationAttributes.SetNum(BoneIndices.Num());

	MakeBoneGlobalFrameAttributesFromArrayViews(
		{}, RotationAttributes, {},
		{}, {}, {},
		Database,
		FrameRanges,
		{}, BoneIndices, {},
		{}, {}, {},
		RelativeTime);

	const int32 AttrNum = OutDirectionFrameAttributes.Num();
	for (int32 AttrIdx = 0; AttrIdx < AttrNum; AttrIdx++)
	{
		OutDirectionFrameAttributes[AttrIdx] = FrameAttributeRotateDirection(RotationAttributes[AttrIdx], LocalDirection);
	}
}

void UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalDirectionFrameAttributesFromNames(TArray<FAnimDatabaseFrameAttribute>& OutDirectionFrameAttributes, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArray<FName>& BoneNames, const FVector LocalDirection, const float RelativeTime)
{
	OutDirectionFrameAttributes.SetNum(BoneNames.Num());
	MakeBoneGlobalDirectionFrameAttributesFromNamesArrayView(OutDirectionFrameAttributes, Database, FrameRanges, BoneNames, LocalDirection, RelativeTime);
}

void UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalDirectionFrameAttributesFromNamesArrayView(const TArrayView<FAnimDatabaseFrameAttribute> OutDirectionFrameAttributes, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const FName> BoneNames, const FVector LocalDirection, const float RelativeTime)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneGlobalDirectionFrameAttributesFromNamesArrayView: Database is nullptr.");
		return;
	}

	TArray<int32, TInlineAllocator<32>> BoneIndices;
	const int32 BoneNum = BoneNames.Num();
	BoneIndices.SetNumUninitialized(BoneNum);
	Database->FindBoneIndicesFromArrayViews(BoneIndices, BoneNames);

	MakeBoneGlobalDirectionFrameAttributesFromArrayViews(
		OutDirectionFrameAttributes,
		Database,
		FrameRanges,
		BoneIndices,
		LocalDirection,
		RelativeTime);
}


FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalLinearVelocityFrameAttribute(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const int32 BoneIndex, const float RelativeTime)
{
	FAnimDatabaseFrameAttribute OutFrameAttribute;

	MakeBoneGlobalFrameAttributesFromArrayViews(
		{}, {}, {},
		MakeArrayView(&OutFrameAttribute, 1), {}, {},
		Database,
		FrameRanges,
		{}, {}, {},
		{ BoneIndex }, {}, {},
		RelativeTime);

	return OutFrameAttribute;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalLinearVelocityFrameAttributeFromName(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName BoneName, const float RelativeTime)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneGlobalLinearVelocityFrameAttributeFromName: Database is nullptr.");
		return FAnimDatabaseFrameAttribute();
	}

	const int32 BoneIdx = Database->FindBoneIndex(BoneName);
	if (BoneIdx == INDEX_NONE)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneGlobalLinearVelocityFrameAttributeFromName: Bone {Name} not found.", *BoneName.ToString());
		return FAnimDatabaseFrameAttribute();
	}

	return MakeBoneGlobalLinearVelocityFrameAttribute(Database, FrameRanges, BoneIdx, RelativeTime);
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalAngularVelocityFrameAttribute(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const int32 BoneIndex, const float RelativeTime)
{
	FAnimDatabaseFrameAttribute OutFrameAttribute;

	MakeBoneGlobalFrameAttributesFromArrayViews(
		{}, {}, {},
		{}, MakeArrayView(&OutFrameAttribute, 1), {},
		Database,
		FrameRanges,
		{}, {}, {},
		{}, { BoneIndex }, {},
		RelativeTime);

	return OutFrameAttribute;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalAngularVelocityFrameAttributeFromName(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName BoneName, const float RelativeTime)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneGlobalAngularVelocityFrameAttributeFromName: Database is nullptr.");
		return FAnimDatabaseFrameAttribute();
	}

	const int32 BoneIdx = Database->FindBoneIndex(BoneName);
	if (BoneIdx == INDEX_NONE)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneGlobalAngularVelocityFrameAttributeFromName: Bone {Name} not found.", *BoneName.ToString());
		return FAnimDatabaseFrameAttribute();
	}

	return MakeBoneGlobalAngularVelocityFrameAttribute(Database, FrameRanges, BoneIdx, RelativeTime);
}

void UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalLocationAndLinearVelocityFrameAttribute(FAnimDatabaseFrameAttribute& OutLocationFrameAttribute, FAnimDatabaseFrameAttribute& OutLinearVelocityFrameAttribute, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const int32 BoneIndex, const float RelativeTime)
{
	MakeBoneGlobalFrameAttributesFromArrayViews(
		MakeArrayView(&OutLocationFrameAttribute, 1), {}, {},
		MakeArrayView(&OutLinearVelocityFrameAttribute, 1), {}, {},
		Database,
		FrameRanges,
		{ BoneIndex }, {}, {},
		{ BoneIndex }, {}, {},
		RelativeTime);
}

void UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalLocationAndLinearVelocityFrameAttributeFromName(FAnimDatabaseFrameAttribute& OutLocationFrameAttribute, FAnimDatabaseFrameAttribute& OutLinearVelocityFrameAttribute, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName BoneName, const float RelativeTime)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneGlobalLocationAndLinearVelocityFrameAttributeFromName: Database is nullptr.");
		OutLocationFrameAttribute = FAnimDatabaseFrameAttribute();
		OutLinearVelocityFrameAttribute = FAnimDatabaseFrameAttribute();
		return;
	}

	const int32 BoneIdx = Database->FindBoneIndex(BoneName);
	if (BoneIdx == INDEX_NONE)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneGlobalLocationAndLinearVelocityFrameAttributeFromName: Bone {Name} not found.", *BoneName.ToString());
		OutLocationFrameAttribute = FAnimDatabaseFrameAttribute();
		OutLinearVelocityFrameAttribute = FAnimDatabaseFrameAttribute();
		return;
	}

	MakeBoneGlobalLocationAndLinearVelocityFrameAttribute(OutLocationFrameAttribute, OutLinearVelocityFrameAttribute, Database, FrameRanges, BoneIdx, RelativeTime);
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalTransformFrameAttribute(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const int32 BoneIndex, const float RelativeTime)
{
	FAnimDatabaseFrameAttribute Location, Rotation, Scale;

	MakeBoneGlobalFrameAttributesFromArrayViews(
		MakeArrayView(&Location, 1), MakeArrayView(&Rotation, 1), MakeArrayView(&Scale, 1),
		{}, {}, {},
		Database,
		FrameRanges,
		{ BoneIndex }, { BoneIndex }, { BoneIndex },
		{ }, {}, {},
		RelativeTime);

	return MakeTransformFrameAttribute(Location, Rotation, Scale);
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalTransformFrameAttributeFromName(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName BoneName, const float RelativeTime)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneGlobalTransformFrameAttributeFromName: Database is nullptr.");
		return FAnimDatabaseFrameAttribute();
	}

	const int32 BoneIdx = Database->FindBoneIndex(BoneName);
	if (BoneIdx == INDEX_NONE)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneGlobalTransformFrameAttributeFromName: Bone {Name} not found.", *BoneName.ToString());
		return FAnimDatabaseFrameAttribute();
	}

	return MakeBoneGlobalTransformFrameAttribute(Database, FrameRanges, BoneIdx, RelativeTime);
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeGlobalDirectionBetweenBonesFrameAttribute(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const int32 BoneIndexFrom, const int32 BoneIndexTo)
{
	TArray<FAnimDatabaseFrameAttribute> Locations;
	Locations.SetNum(2);
	MakeBoneGlobalLocationFrameAttributesFromArrayViews(Locations, Database, FrameRanges, { BoneIndexFrom, BoneIndexTo });
	return FrameAttributeLocationToDirection(FrameAttributeSubtract(Locations[1], Locations[0]));
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeGlobalDirectionBetweenBonesFrameAttributeFromNames(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName BoneNameFrom, const FName BoneNameTo)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeGlobalDirectionBetweenBonesFrameAttribute: Database is nullptr.");
		return FAnimDatabaseFrameAttribute();
	}

	const int32 BoneIdxFrom = Database->FindBoneIndex(BoneNameFrom);
	if (BoneIdxFrom == INDEX_NONE)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeGlobalDirectionBetweenBonesFrameAttribute: Bone {Name} not found.", *BoneNameFrom.ToString());
		return FAnimDatabaseFrameAttribute();
	}

	const int32 BoneIdxTo = Database->FindBoneIndex(BoneNameTo);
	if (BoneIdxTo == INDEX_NONE)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeGlobalDirectionBetweenBonesFrameAttribute: Bone {Name} not found.", *BoneNameTo.ToString());
		return FAnimDatabaseFrameAttribute();
	}

	return MakeGlobalDirectionBetweenBonesFrameAttribute(Database, FrameRanges, BoneIdxFrom, BoneIdxTo);
}

void UAnimDatabaseFrameAttributeLibrary::MakePoseFrameAttribute(
	TArray<FAnimDatabaseFrameAttribute>& OutLocationFrameAttributes,
	TArray<FAnimDatabaseFrameAttribute>& OutLinearVelocityFrameAttributes,
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const TArray<int32>& BoneIndices,
	const float RelativeTime)
{
	OutLocationFrameAttributes.SetNum(BoneIndices.Num());
	OutLinearVelocityFrameAttributes.SetNum(BoneIndices.Num());

	MakeBoneGlobalFrameAttributesFromArrayViews(
		OutLocationFrameAttributes, {}, {},
		OutLinearVelocityFrameAttributes, {}, {},
		Database,
		FrameRanges,
		BoneIndices, {}, {},
		BoneIndices, {}, {},
		RelativeTime);
}

void UAnimDatabaseFrameAttributeLibrary::MakeRootLocalPoseFrameAttributeAndScale(
	TArray<FAnimDatabaseFrameAttribute>& OutLocationFrameAttributes,
	TArray<FAnimDatabaseFrameAttribute>& OutLinearVelocityFrameAttributes,
	float& OutLocationScale,
	float& OutLinearVelocityScale,
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const TArray<int32>& BoneIndices)
{
	FAnimDatabaseFrameAttribute RootLocation, RootRotation;
	MakeRootLocationAndRotationFrameAttribute(RootLocation, RootRotation, Database, FrameRanges);

	TArray<FAnimDatabaseFrameAttribute> PoseLocationFrameAttributes;
	TArray<FAnimDatabaseFrameAttribute> PoseLinearVelocityFrameAttributes;
	MakePoseFrameAttribute(PoseLocationFrameAttributes, PoseLinearVelocityFrameAttributes, Database, FrameRanges, BoneIndices);
	FrameAttributesSubtractAndUnrotate(OutLocationFrameAttributes, PoseLocationFrameAttributes, RootLocation, RootRotation);
	FrameAttributesUnrotate(OutLinearVelocityFrameAttributes, RootRotation, PoseLinearVelocityFrameAttributes);
	OutLocationScale = FrameAttributesAverageStdFromArrayView(OutLocationFrameAttributes);
	OutLinearVelocityScale = FrameAttributesAverageStdFromArrayView(OutLinearVelocityFrameAttributes);
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeRootLocationAtRangeStartFrameAttribute(
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::MakeRootLocationAtRangeStartFrameAttribute);

	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeRootLocationAtRangeStartFrameAttribute: Database is nullptr.");
		return FAnimDatabaseFrameAttribute();
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeRootLocationAtRangeStartFrameAttribute: FrameRanges is invalid.");
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Location;
	Out.FrameAttribute->FrameRangeSet = *FrameRanges.FrameRangeSet;
	Out.FrameAttribute->AttributeData.SetNumUninitialized({ AttributeTypeSize(Out.Type), FrameRanges.FrameRangeSet->GetTotalFrameNum() });

	Database->WaitForCompressionOnAnimSequencesFromArrayView(FrameRanges.FrameRangeSet->GetEntrySequences());

	UE::Learning::FrameRangeSet::ParallelForEachRange(*FrameRanges.FrameRangeSet, [&FrameRanges, &Out, &Database](
		const int32 TotalRangeIdx,
		const int32 EntryIdx,
		const int32 RangeIdx) {

			const int32 FrameOffset = FrameRanges.FrameRangeSet->GetEntryRangeOffset(EntryIdx, RangeIdx);
			const int32 FrameNum = FrameRanges.FrameRangeSet->GetEntryRangeLength(EntryIdx, RangeIdx);
			const int32 Sequence = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);
			const int32 StartFrame = FrameRanges.FrameRangeSet->GetEntryRangeStart(EntryIdx, RangeIdx);
			FVector RootLocation; Database->GetRootLocation(MakeArrayView(&RootLocation, 1), Sequence, StartFrame);

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				Out.FrameAttribute->AttributeData[0][FrameOffset + FrameIdx] = RootLocation.X;
				Out.FrameAttribute->AttributeData[1][FrameOffset + FrameIdx] = RootLocation.Y;
				Out.FrameAttribute->AttributeData[2][FrameOffset + FrameIdx] = RootLocation.Z;
			}
		});

	UE::Learning::Array::Check(Out.FrameAttribute->AttributeData);

	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeRootDirectionAtRangeStartFrameAttribute(
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const FVector ForwardVector)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::MakeRootDirectionAtRangeStartFrameAttribute);

	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeRootDirectionAtRangeStartFrameAttribute: Database is nullptr.");
		return FAnimDatabaseFrameAttribute();
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeRootDirectionAtRangeStartFrameAttribute: FrameRanges is invalid.");
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Direction;
	Out.FrameAttribute->FrameRangeSet = *FrameRanges.FrameRangeSet;
	Out.FrameAttribute->AttributeData.SetNumUninitialized({ AttributeTypeSize(Out.Type), FrameRanges.FrameRangeSet->GetTotalFrameNum() });

	Database->WaitForCompressionOnAnimSequencesFromArrayView(FrameRanges.FrameRangeSet->GetEntrySequences());

	UE::Learning::FrameRangeSet::ParallelForEachRange(*FrameRanges.FrameRangeSet, [&FrameRanges, &Out, &Database, &ForwardVector](
		const int32 TotalRangeIdx,
		const int32 EntryIdx,
		const int32 RangeIdx) {

			const int32 FrameOffset = FrameRanges.FrameRangeSet->GetEntryRangeOffset(EntryIdx, RangeIdx);
			const int32 FrameNum = FrameRanges.FrameRangeSet->GetEntryRangeLength(EntryIdx, RangeIdx);
			const int32 Sequence = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);
			const int32 StartFrame = FrameRanges.FrameRangeSet->GetEntryRangeStart(EntryIdx, RangeIdx);
			FVector3f RootDirection; Database->GetRootDirection(MakeArrayView(&RootDirection, 1), Sequence, StartFrame, (FVector3f)ForwardVector);

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				Out.FrameAttribute->AttributeData[0][FrameOffset + FrameIdx] = RootDirection.X;
				Out.FrameAttribute->AttributeData[1][FrameOffset + FrameIdx] = RootDirection.Y;
				Out.FrameAttribute->AttributeData[2][FrameOffset + FrameIdx] = RootDirection.Z;
			}
		});

	UE::Learning::Array::Check(Out.FrameAttribute->AttributeData);

	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeRootLocationAtRangeEndFrameAttribute(
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::MakeRootLocationAtRangeEndFrameAttribute);

	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeRootLocationAtRangeEndFrameAttribute: Database is nullptr.");
		return FAnimDatabaseFrameAttribute();
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeRootLocationAtRangeEndFrameAttribute: FrameRanges is invalid.");
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Location;
	Out.FrameAttribute->FrameRangeSet = *FrameRanges.FrameRangeSet;
	Out.FrameAttribute->AttributeData.SetNumUninitialized({ AttributeTypeSize(Out.Type), FrameRanges.FrameRangeSet->GetTotalFrameNum() });

	Database->WaitForCompressionOnAnimSequencesFromArrayView(FrameRanges.FrameRangeSet->GetEntrySequences());

	UE::Learning::FrameRangeSet::ParallelForEachRange(*FrameRanges.FrameRangeSet, [&FrameRanges, &Out, &Database](
		const int32 TotalRangeIdx,
		const int32 EntryIdx,
		const int32 RangeIdx) {

			const int32 FrameOffset = FrameRanges.FrameRangeSet->GetEntryRangeOffset(EntryIdx, RangeIdx);
			const int32 FrameNum = FrameRanges.FrameRangeSet->GetEntryRangeLength(EntryIdx, RangeIdx);
			const int32 Sequence = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);
			const int32 StartFrame = FrameRanges.FrameRangeSet->GetEntryRangeStart(EntryIdx, RangeIdx);
			const int32 EndOfFrameRangesFrame = StartFrame + FrameNum - 1;
			FVector RootLocation; Database->GetRootLocation(MakeArrayView(&RootLocation, 1), Sequence, EndOfFrameRangesFrame);

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				Out.FrameAttribute->AttributeData[0][FrameOffset + FrameIdx] = RootLocation.X;
				Out.FrameAttribute->AttributeData[1][FrameOffset + FrameIdx] = RootLocation.Y;
				Out.FrameAttribute->AttributeData[2][FrameOffset + FrameIdx] = RootLocation.Z;
			}
		});

	UE::Learning::Array::Check(Out.FrameAttribute->AttributeData);

	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeRootDirectionAtRangeEndFrameAttribute(
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const FVector ForwardVector)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::MakeRootDirectionAtRangeEndFrameAttribute);

	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeRootDirectionAtRangeEndFrameAttribute: Database is nullptr.");
		return FAnimDatabaseFrameAttribute();
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeRootDirectionAtRangeEndFrameAttribute: FrameRanges is invalid.");
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Direction;
	Out.FrameAttribute->FrameRangeSet = *FrameRanges.FrameRangeSet;
	Out.FrameAttribute->AttributeData.SetNumUninitialized({ AttributeTypeSize(Out.Type), FrameRanges.FrameRangeSet->GetTotalFrameNum() });

	Database->WaitForCompressionOnAnimSequencesFromArrayView(FrameRanges.FrameRangeSet->GetEntrySequences());

	UE::Learning::FrameRangeSet::ParallelForEachRange(*FrameRanges.FrameRangeSet, [&FrameRanges, &Out, &Database, ForwardVector](
		const int32 TotalRangeIdx,
		const int32 EntryIdx,
		const int32 RangeIdx) {

			const int32 FrameOffset = FrameRanges.FrameRangeSet->GetEntryRangeOffset(EntryIdx, RangeIdx);
			const int32 FrameNum = FrameRanges.FrameRangeSet->GetEntryRangeLength(EntryIdx, RangeIdx);
			const int32 Sequence = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);
			const int32 StartFrame = FrameRanges.FrameRangeSet->GetEntryRangeStart(EntryIdx, RangeIdx);
			FVector3f RootDirection; Database->GetRootDirection(MakeArrayView(&RootDirection, 1), Sequence, StartFrame + FrameNum - 1, (FVector3f)ForwardVector);

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				Out.FrameAttribute->AttributeData[0][FrameOffset + FrameIdx] = RootDirection.X;
				Out.FrameAttribute->AttributeData[1][FrameOffset + FrameIdx] = RootDirection.Y;
				Out.FrameAttribute->AttributeData[2][FrameOffset + FrameIdx] = RootDirection.Z;
			}
		});

	UE::Learning::Array::Check(Out.FrameAttribute->AttributeData);

	return Out;
}

void UAnimDatabaseFrameAttributeLibrary::MakeRootLocationAndDirectionAtRangeEndFrameAttribute(
	FAnimDatabaseFrameAttribute& OutLocationFrameAttribute,
	FAnimDatabaseFrameAttribute& OutDirectionFrameAttribute,
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const FVector ForwardVector)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::MakeRootLocationAndDirectionAtRangeEndFrameAttribute);

	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeRootLocationAndDirectionAtRangeEndFrameAttribute: Database is nullptr.");
		OutLocationFrameAttribute = FAnimDatabaseFrameAttribute();
		OutDirectionFrameAttribute = FAnimDatabaseFrameAttribute();
		return;
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeRootLocationAndDirectionAtRangeEndFrameAttribute: FrameRanges is invalid.");
		OutLocationFrameAttribute = FAnimDatabaseFrameAttribute();
		OutDirectionFrameAttribute = FAnimDatabaseFrameAttribute();
		return;
	}

	OutLocationFrameAttribute = MakeEmptyFrameAttribute();
	OutLocationFrameAttribute.Type = EAnimDatabaseAttributeType::Location;
	OutLocationFrameAttribute.FrameAttribute->FrameRangeSet = *FrameRanges.FrameRangeSet;
	OutLocationFrameAttribute.FrameAttribute->AttributeData.SetNumUninitialized({ AttributeTypeSize(OutLocationFrameAttribute.Type), FrameRanges.FrameRangeSet->GetTotalFrameNum() });

	OutDirectionFrameAttribute = MakeEmptyFrameAttribute();
	OutDirectionFrameAttribute.Type = EAnimDatabaseAttributeType::Direction;
	OutDirectionFrameAttribute.FrameAttribute->FrameRangeSet = *FrameRanges.FrameRangeSet;
	OutDirectionFrameAttribute.FrameAttribute->AttributeData.SetNumUninitialized({ AttributeTypeSize(OutDirectionFrameAttribute.Type), FrameRanges.FrameRangeSet->GetTotalFrameNum() });

	Database->WaitForCompressionOnAnimSequencesFromArrayView(FrameRanges.FrameRangeSet->GetEntrySequences());

	UE::Learning::FrameRangeSet::ParallelForEachRange(*FrameRanges.FrameRangeSet, [&FrameRanges, &OutLocationFrameAttribute, OutDirectionFrameAttribute, &Database, ForwardVector](
		const int32 TotalRangeIdx,
		const int32 EntryIdx,
		const int32 RangeIdx) {

			const int32 FrameOffset = FrameRanges.FrameRangeSet->GetEntryRangeOffset(EntryIdx, RangeIdx);
			const int32 FrameNum = FrameRanges.FrameRangeSet->GetEntryRangeLength(EntryIdx, RangeIdx);
			const int32 Sequence = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);
			const int32 StartFrame = FrameRanges.FrameRangeSet->GetEntryRangeStart(EntryIdx, RangeIdx);
			FVector RootLocation; Database->GetRootLocation(MakeArrayView(&RootLocation, 1), Sequence, StartFrame + FrameNum - 1);
			FVector3f RootDirection; Database->GetRootDirection(MakeArrayView(&RootDirection, 1), Sequence, StartFrame + FrameNum - 1, (FVector3f)ForwardVector);

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				OutLocationFrameAttribute.FrameAttribute->AttributeData[0][FrameOffset + FrameIdx] = RootLocation.X;
				OutLocationFrameAttribute.FrameAttribute->AttributeData[1][FrameOffset + FrameIdx] = RootLocation.Y;
				OutLocationFrameAttribute.FrameAttribute->AttributeData[2][FrameOffset + FrameIdx] = RootLocation.Z;

				OutDirectionFrameAttribute.FrameAttribute->AttributeData[0][FrameOffset + FrameIdx] = RootDirection.X;
				OutDirectionFrameAttribute.FrameAttribute->AttributeData[1][FrameOffset + FrameIdx] = RootDirection.Y;
				OutDirectionFrameAttribute.FrameAttribute->AttributeData[2][FrameOffset + FrameIdx] = RootDirection.Z;
			}
		});

	UE::Learning::Array::Check(OutLocationFrameAttribute.FrameAttribute->AttributeData);
	UE::Learning::Array::Check(OutDirectionFrameAttribute.FrameAttribute->AttributeData);
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalLocationAtNearestFramesFrameAttribute(
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const int32 BoneIndex,
	const FAnimDatabaseFrames& Frames)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalLocationAtNearestFramesFrameAttribute);

	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneGlobalLocationAtNearestFramesFrameAttribute: Database is nullptr.");
		return FAnimDatabaseFrameAttribute();
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneGlobalLocationAtNearestFramesFrameAttribute: FrameRanges is invalid.");
		return FAnimDatabaseFrameAttribute();
	}

	if (!Frames.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneGlobalLocationAtNearestFramesFrameAttribute: Frames is invalid.");
		return FAnimDatabaseFrameAttribute();
	}

	const int32 BoneNum = Database->GetBoneNum();

	if (BoneIndex < 0 || BoneIndex >= BoneNum)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneGlobalLocationAtNearestFramesFrameAttribute: Invalid Bone Index.");
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Location;
	Out.FrameAttribute->FrameRangeSet = *FrameRanges.FrameRangeSet;
	Out.FrameAttribute->AttributeData.SetNumUninitialized({ AttributeTypeSize(Out.Type), FrameRanges.FrameRangeSet->GetTotalFrameNum() });

	TArray<int32> BoneParents;
	BoneParents.SetNumUninitialized(BoneNum);
	Database->GetBoneParents(BoneParents);

	// TODO: Don't load all bones.

	UE::AnimDatabase::FPoseData PoseData;
	UE::AnimDatabase::FPoseGlobalBoneData PoseGlobalBoneData;
	PoseData.Resize(FrameRanges.FrameRangeSet->GetTotalFrameNum(), BoneNum, {}, {});
	PoseGlobalBoneData.Resize(FrameRanges.FrameRangeSet->GetTotalFrameNum(), BoneNum);

	Database->WaitForCompressionOnAnimSequencesFromArrayView(FrameRanges.FrameRangeSet->GetEntrySequences());

	UE::Learning::FrameRangeSet::ParallelForEachRange(*FrameRanges.FrameRangeSet, [&FrameRanges, &Frames, &Out, &PoseData, &PoseGlobalBoneData, &BoneParents, &Database, BoneIndex](
		const int32 TotalRangeIdx,
		const int32 EntryIdx,
		const int32 RangeIdx) {

			const int32 FrameOffset = FrameRanges.FrameRangeSet->GetEntryRangeOffset(EntryIdx, RangeIdx);
			const int32 FrameNum = FrameRanges.FrameRangeSet->GetEntryRangeLength(EntryIdx, RangeIdx);
			const int32 Sequence = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);
			const int32 StartFrame = FrameRanges.FrameRangeSet->GetEntryRangeStart(EntryIdx, RangeIdx);

			Database->GetPoseData(
				PoseData.Slice(FrameOffset, FrameNum),
				Sequence,
				StartFrame);

			UE::AnimDatabase::PoseData::ForwardKinematics(
				PoseGlobalBoneData.Slice(FrameOffset, FrameNum),
				PoseData.LocalBoneData.ConstSlice(FrameOffset, FrameNum),
				PoseData.RootData.ConstSlice(FrameOffset, FrameNum),
				BoneParents);

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				int32 NearestEntryIdx = INDEX_NONE;
				int32 NearestFrameIdx = INDEX_NONE;
				int32 NearestFrameDifference = INT32_MAX;
				if (Frames.FrameSet->FindNearestInRange(
					NearestEntryIdx,
					NearestFrameIdx,
					NearestFrameDifference,
					Sequence,
					StartFrame + FrameIdx,
					StartFrame,
					FrameNum))
				{
					const int32 NearestRangeFrame = FMath::Clamp(Frames.FrameSet->GetEntryFrame(NearestEntryIdx, NearestFrameIdx) - StartFrame, 0, FrameNum - 1);
					const FVector BoneLocation = PoseGlobalBoneData.BoneLocations[FrameOffset + NearestRangeFrame][BoneIndex];

					Out.FrameAttribute->AttributeData[0][FrameOffset + FrameIdx] = BoneLocation.X;
					Out.FrameAttribute->AttributeData[1][FrameOffset + FrameIdx] = BoneLocation.Y;
					Out.FrameAttribute->AttributeData[2][FrameOffset + FrameIdx] = BoneLocation.Z;
				}
				else
				{
					Out.FrameAttribute->AttributeData[0][FrameOffset + FrameIdx] = 0.0f;
					Out.FrameAttribute->AttributeData[1][FrameOffset + FrameIdx] = 0.0f;
					Out.FrameAttribute->AttributeData[2][FrameOffset + FrameIdx] = 0.0f;
				}
			}
		});

	UE::Learning::Array::Check(Out.FrameAttribute->AttributeData);

	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalDirectionAtNearestFramesFrameAttribute(
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const int32 BoneIndex,
	const FAnimDatabaseFrames& Frames,
	const FVector LocalDirection)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalDirectionAtNearestFramesFrameAttribute);

	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneGlobalDirectionAtNearestFramesFrameAttribute: Database is nullptr.");
		return FAnimDatabaseFrameAttribute();
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneGlobalDirectionAtNearestFramesFrameAttribute: FrameRanges is invalid.");
		return FAnimDatabaseFrameAttribute();
	}

	if (!Frames.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneGlobalDirectionAtNearestFramesFrameAttribute: Frames is invalid.");
		return FAnimDatabaseFrameAttribute();
	}

	const int32 BoneNum = Database->GetBoneNum();

	if (BoneIndex < 0 || BoneIndex >= BoneNum)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneGlobalDirectionAtNearestFramesFrameAttribute: Invalid Bone Index.");
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Direction;
	Out.FrameAttribute->FrameRangeSet = *FrameRanges.FrameRangeSet;
	Out.FrameAttribute->AttributeData.SetNumUninitialized({ AttributeTypeSize(Out.Type), FrameRanges.FrameRangeSet->GetTotalFrameNum() });

	TArray<int32> BoneParents;
	BoneParents.SetNumUninitialized(BoneNum);
	Database->GetBoneParents(BoneParents);

	// TODO: Don't load all bones.

	UE::AnimDatabase::FPoseData PoseData;
	UE::AnimDatabase::FPoseGlobalBoneData PoseGlobalBoneData;
	PoseData.Resize(FrameRanges.FrameRangeSet->GetTotalFrameNum(), BoneNum, {}, {});
	PoseGlobalBoneData.Resize(FrameRanges.FrameRangeSet->GetTotalFrameNum(), BoneNum);

	Database->WaitForCompressionOnAnimSequencesFromArrayView(FrameRanges.FrameRangeSet->GetEntrySequences());

	UE::Learning::FrameRangeSet::ParallelForEachRange(*FrameRanges.FrameRangeSet, [&FrameRanges, &Frames, &Out, &PoseData, &PoseGlobalBoneData, &BoneParents, &Database, BoneIndex, LocalDirection](
		const int32 TotalRangeIdx,
		const int32 EntryIdx,
		const int32 RangeIdx) {

			const int32 FrameOffset = FrameRanges.FrameRangeSet->GetEntryRangeOffset(EntryIdx, RangeIdx);
			const int32 FrameNum = FrameRanges.FrameRangeSet->GetEntryRangeLength(EntryIdx, RangeIdx);
			const int32 Sequence = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);
			const int32 StartFrame = FrameRanges.FrameRangeSet->GetEntryRangeStart(EntryIdx, RangeIdx);

			Database->GetPoseData(
				PoseData.Slice(FrameOffset, FrameNum),
				Sequence,
				StartFrame);

			UE::AnimDatabase::PoseData::ForwardKinematics(
				PoseGlobalBoneData.Slice(FrameOffset, FrameNum),
				PoseData.LocalBoneData.ConstSlice(FrameOffset, FrameNum),
				PoseData.RootData.ConstSlice(FrameOffset, FrameNum),
				BoneParents);

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				int32 NearestEntryIdx = INDEX_NONE;
				int32 NearestFrameIdx = INDEX_NONE;
				int32 NearestFrameDifference = INT32_MAX;
				if (Frames.FrameSet->FindNearestInRange(
					NearestEntryIdx,
					NearestFrameIdx,
					NearestFrameDifference,
					Sequence,
					StartFrame + FrameIdx,
					StartFrame,
					FrameNum))
				{
					const int32 NearestRangeFrame = FMath::Clamp(Frames.FrameSet->GetEntryFrame(NearestEntryIdx, NearestFrameIdx) - StartFrame, 0, FrameNum - 1);

					const FVector BoneDirection = (FVector)PoseGlobalBoneData.BoneRotations[FrameOffset + NearestRangeFrame][BoneIndex].RotateVector((FVector3f)LocalDirection);
					Out.FrameAttribute->AttributeData[0][FrameOffset + FrameIdx] = BoneDirection.X;
					Out.FrameAttribute->AttributeData[1][FrameOffset + FrameIdx] = BoneDirection.Y;
					Out.FrameAttribute->AttributeData[2][FrameOffset + FrameIdx] = BoneDirection.Z;
				}
				else
				{
					Out.FrameAttribute->AttributeData[0][FrameOffset + FrameIdx] = LocalDirection.X;
					Out.FrameAttribute->AttributeData[1][FrameOffset + FrameIdx] = LocalDirection.Y;
					Out.FrameAttribute->AttributeData[2][FrameOffset + FrameIdx] = LocalDirection.Z;
				}
			}
		});

	UE::Learning::Array::Check(Out.FrameAttribute->AttributeData);

	return Out;
}

void UAnimDatabaseFrameAttributeLibrary::MakeFrameAttributesFromCurves(
	TArray<FAnimDatabaseFrameAttribute>& OutFloatCurveValueFrameAttributes,
	TArray<FAnimDatabaseFrameAttribute>& OutFloatCurveVelocityFrameAttributes,
	TArray<FAnimDatabaseFrameAttribute>& OutBoolCurveActiveFrameAttributes,
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const TArray<FName>& CurveNames)
{
	OutFloatCurveValueFrameAttributes.SetNum(CurveNames.Num());
	OutFloatCurveVelocityFrameAttributes.SetNum(CurveNames.Num());
	OutBoolCurveActiveFrameAttributes.SetNum(CurveNames.Num());
	MakeFrameAttributesFromCurvesArrayViews(
		OutFloatCurveValueFrameAttributes,
		OutFloatCurveVelocityFrameAttributes,
		OutBoolCurveActiveFrameAttributes,
		Database,
		FrameRanges,
		CurveNames);
}

void UAnimDatabaseFrameAttributeLibrary::MakeFrameAttributesFromCurvesArrayViews(
	const TArrayView<FAnimDatabaseFrameAttribute> OutFloatCurveValueFrameAttributes,
	const TArrayView<FAnimDatabaseFrameAttribute> OutFloatCurveVelocityFrameAttributes,
	const TArrayView<FAnimDatabaseFrameAttribute> OutBoolCurveActiveFrameAttributes,
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const TArrayView<const FName> CurveNames)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::MakeFrameAttributesFromCurvesArrayViews);

	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneGlobalLocationAtNearestFramesFrameAttribute: Database is nullptr.");
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutFloatCurveValueFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutFloatCurveVelocityFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutBoolCurveActiveFrameAttributes, FAnimDatabaseFrameAttribute());
		return;
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeBoneGlobalLocationAtNearestFramesFrameAttribute: FrameRanges is invalid.");
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutFloatCurveValueFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutFloatCurveVelocityFrameAttributes, FAnimDatabaseFrameAttribute());
		UE::Learning::Array::Set<1, FAnimDatabaseFrameAttribute>(OutBoolCurveActiveFrameAttributes, FAnimDatabaseFrameAttribute());
		return;
	}

	const int32 CurveNum = CurveNames.Num();
	check(OutFloatCurveValueFrameAttributes.Num() == CurveNum);
	check(OutFloatCurveVelocityFrameAttributes.Num() == CurveNum);
	check(OutBoolCurveActiveFrameAttributes.Num() == CurveNum);

	const int32 TotalFrameNum = FrameRanges.FrameRangeSet->GetTotalFrameNum();

	for (int32 CurveIdx = 0; CurveIdx < CurveNum; CurveIdx++)
	{
		OutFloatCurveValueFrameAttributes[CurveIdx] = MakeEmptyFrameAttribute();
		OutFloatCurveValueFrameAttributes[CurveIdx].Type = EAnimDatabaseAttributeType::Float;
		OutFloatCurveValueFrameAttributes[CurveIdx].FrameAttribute->FrameRangeSet = *FrameRanges.FrameRangeSet;
		OutFloatCurveValueFrameAttributes[CurveIdx].FrameAttribute->AttributeData.SetNumUninitialized({ AttributeTypeSize(OutFloatCurveValueFrameAttributes[CurveIdx].Type), TotalFrameNum });

		OutFloatCurveVelocityFrameAttributes[CurveIdx] = MakeEmptyFrameAttribute();
		OutFloatCurveVelocityFrameAttributes[CurveIdx].Type = EAnimDatabaseAttributeType::Float;
		OutFloatCurveVelocityFrameAttributes[CurveIdx].FrameAttribute->FrameRangeSet = *FrameRanges.FrameRangeSet;
		OutFloatCurveVelocityFrameAttributes[CurveIdx].FrameAttribute->AttributeData.SetNumUninitialized({ AttributeTypeSize(OutFloatCurveVelocityFrameAttributes[CurveIdx].Type), TotalFrameNum });

		OutBoolCurveActiveFrameAttributes[CurveIdx] = MakeEmptyFrameAttribute();
		OutBoolCurveActiveFrameAttributes[CurveIdx].Type = EAnimDatabaseAttributeType::Bool;
		OutBoolCurveActiveFrameAttributes[CurveIdx].FrameAttribute->FrameRangeSet = *FrameRanges.FrameRangeSet;
		OutBoolCurveActiveFrameAttributes[CurveIdx].FrameAttribute->AttributeData.SetNumUninitialized({ AttributeTypeSize(OutBoolCurveActiveFrameAttributes[CurveIdx].Type), TotalFrameNum });
	}

	TLearningArray<2, float> CurveValues;
	TLearningArray<2, float> CurveVelocities;
	TLearningArray<2, bool> CurveActives;
	CurveValues.SetNumUninitialized({ TotalFrameNum, CurveNum });
	CurveVelocities.SetNumUninitialized({ TotalFrameNum, CurveNum });
	CurveActives.SetNumUninitialized({ TotalFrameNum, CurveNum });

	Database->WaitForCompressionOnAnimSequencesFromArrayView(FrameRanges.FrameRangeSet->GetEntrySequences());

	UE::Learning::FrameRangeSet::ParallelForEachRange(*FrameRanges.FrameRangeSet, [&OutFloatCurveValueFrameAttributes, &OutFloatCurveVelocityFrameAttributes, &OutBoolCurveActiveFrameAttributes, &FrameRanges, &Database, &CurveActives, &CurveValues, &CurveVelocities, &CurveNames, CurveNum](
		const int32 TotalRangeIdx,
		const int32 EntryIdx,
		const int32 RangeIdx) {

			const int32 FrameOffset = FrameRanges.FrameRangeSet->GetEntryRangeOffset(EntryIdx, RangeIdx);
			const int32 FrameNum = FrameRanges.FrameRangeSet->GetEntryRangeLength(EntryIdx, RangeIdx);
			const int32 Sequence = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);
			const int32 StartFrame = FrameRanges.FrameRangeSet->GetEntryRangeStart(EntryIdx, RangeIdx);

			Database->GetCurveData(
				CurveValues.Slice(FrameOffset, FrameNum),
				CurveVelocities.Slice(FrameOffset, FrameNum),
				CurveActives.Slice(FrameOffset, FrameNum),
				Sequence,
				StartFrame,
				CurveNames);

			for (int32 FrameIdx = FrameOffset; FrameIdx < FrameOffset + FrameNum; FrameIdx++)
			{
				for (int32 CurveIdx = 0; CurveIdx < CurveNum; CurveIdx++)
				{
					OutFloatCurveValueFrameAttributes[CurveIdx].FrameAttribute->AttributeData[0][FrameIdx] = CurveValues[FrameIdx][CurveIdx];
					OutFloatCurveVelocityFrameAttributes[CurveIdx].FrameAttribute->AttributeData[0][FrameIdx] = CurveVelocities[FrameIdx][CurveIdx];
					OutBoolCurveActiveFrameAttributes[CurveIdx].FrameAttribute->AttributeData[0][FrameIdx] = CurveActives[FrameIdx][CurveIdx] ? 1.0f : 0.0f;
				}
			}
		});

	for (int32 CurveIdx = 0; CurveIdx < CurveNum; CurveIdx++)
	{
		UE::Learning::Array::Check(OutFloatCurveValueFrameAttributes[CurveIdx].FrameAttribute->AttributeData);
		UE::Learning::Array::Check(OutFloatCurveVelocityFrameAttributes[CurveIdx].FrameAttribute->AttributeData);
		UE::Learning::Array::Check(OutBoolCurveActiveFrameAttributes[CurveIdx].FrameAttribute->AttributeData);
	}
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeBoolAtFrame(bool& OutBool, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx)
{
	if (!FrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeBoolAtFrame: Invalid FrameAttribute.");
		OutBool = false;
		return false;
	}

	if (FrameAttribute.Type != EAnimDatabaseAttributeType::Bool)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeBoolAtFrame: FrameAttribute Types don't match. Got {Type}. Expected Bool.", AttributeTypeNameInternal(FrameAttribute.Type));
		OutBool = false;
		return false;
	}

	int32 EntryIdx = INDEX_NONE;
	int32 RangeIdx = INDEX_NONE;
	int32 RangeFrame = INDEX_NONE;
	if (!FrameAttribute.FrameAttribute->FrameRangeSet.Find(EntryIdx, RangeIdx, RangeFrame, SequenceIdx, FrameIdx))
	{
		OutBool = false;
		return false;
	}

	const int32 RangeOffset = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeOffset(EntryIdx, RangeIdx) + RangeFrame;

	OutBool = FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset] == 1.0f;
	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeFloatAtFrame(float& OutValue, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx)
{
	if (!FrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeFloatAtFrame: Invalid FrameAttribute.");
		OutValue = 0.0f;
		return false;
	}

	if (FrameAttribute.Type != EAnimDatabaseAttributeType::Float)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeFloatAtFrame: FrameAttribute Types don't match. Got {Type}. Expected Float.", AttributeTypeNameInternal(FrameAttribute.Type));
		OutValue = 0.0f;
		return false;
	}

	int32 EntryIdx = INDEX_NONE;
	int32 RangeIdx = INDEX_NONE;
	int32 RangeFrame = INDEX_NONE;
	if (!FrameAttribute.FrameAttribute->FrameRangeSet.Find(EntryIdx, RangeIdx, RangeFrame, SequenceIdx, FrameIdx))
	{
		OutValue = 0.0f;
		return false;
	}

	const int32 RangeOffset = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeOffset(EntryIdx, RangeIdx) + RangeFrame;

	OutValue = FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset];
	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeLocationAtFrame(FVector& OutLocation, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx)
{
	FVector3f OutLocationFloat = FVector3f::ZeroVector;
	if (!FrameAttributeLocationAtFrameFloat(OutLocationFloat, FrameAttribute, SequenceIdx, FrameIdx))
	{
		OutLocation = FVector::ZeroVector;
		return false;
	}

	OutLocation = (FVector)OutLocationFloat;
	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeLocationAtFrameFloat(FVector3f& OutLocation, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx)
{
	if (!FrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeLocationAtFrameFloat: Invalid FrameAttribute.");
		OutLocation = FVector3f::ZeroVector;
		return false;
	}

	if (FrameAttribute.Type != EAnimDatabaseAttributeType::Location)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeLocationAtFrameFloat: FrameAttribute Types don't match. Got {Type}. Expected Location.", AttributeTypeNameInternal(FrameAttribute.Type));
		OutLocation = FVector3f::ZeroVector;
		return false;
	}

	int32 EntryIdx = INDEX_NONE;
	int32 RangeIdx = INDEX_NONE;
	int32 RangeFrame = INDEX_NONE;
	if (!FrameAttribute.FrameAttribute->FrameRangeSet.Find(EntryIdx, RangeIdx, RangeFrame, SequenceIdx, FrameIdx))
	{
		OutLocation = FVector3f::ZeroVector;
		return false;
	}

	const int32 RangeOffset = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeOffset(EntryIdx, RangeIdx) + RangeFrame;

	OutLocation = FVector3f(
		FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset],
		FrameAttribute.FrameAttribute->AttributeData[1][RangeOffset],
		FrameAttribute.FrameAttribute->AttributeData[2][RangeOffset]);
	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeRotationAtFrame(FRotator& OutRotation, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx)
{
	FQuat OutQuat = FQuat::Identity;
	if (!FrameAttributeRotationAtFrameAsQuat(OutQuat, FrameAttribute, SequenceIdx, FrameIdx))
	{
		OutRotation = FRotator::ZeroRotator;
		return false;
	}

	OutRotation = OutQuat.Rotator();
	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeRotationAtFrameAsQuat(FQuat& OutRotation, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx)
{
	FQuat4f OutRotationFloat = FQuat4f::Identity;
	if (!FrameAttributeRotationAtFrameAsQuatFloat(OutRotationFloat, FrameAttribute, SequenceIdx, FrameIdx))
	{
		OutRotation = FQuat::Identity;
		return false;
	}

	OutRotation = ((FQuat)OutRotationFloat).GetNormalized();
	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeRotationAtFrameAsQuatFloat(FQuat4f& OutRotation, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx)
{
	if (!FrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeRotationAtFrameAsQuatFloat: Invalid FrameAttribute.");
		OutRotation = FQuat4f::Identity;
		return false;
	}

	if (FrameAttribute.Type != EAnimDatabaseAttributeType::Rotation)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeRotationAtFrameAsQuatFloat: FrameAttribute Types don't match. Got {Type}. Expected Rotation.", AttributeTypeNameInternal(FrameAttribute.Type));
		OutRotation = FQuat4f::Identity;
		return false;
	}

	int32 EntryIdx = INDEX_NONE;
	int32 RangeIdx = INDEX_NONE;
	int32 RangeFrame = INDEX_NONE;
	if (!FrameAttribute.FrameAttribute->FrameRangeSet.Find(EntryIdx, RangeIdx, RangeFrame, SequenceIdx, FrameIdx))
	{
		OutRotation = FQuat4f::Identity;
		return false;
	}

	const int32 RangeOffset = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeOffset(EntryIdx, RangeIdx) + RangeFrame;

	OutRotation = FQuat4f(
		FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset],
		FrameAttribute.FrameAttribute->AttributeData[1][RangeOffset],
		FrameAttribute.FrameAttribute->AttributeData[2][RangeOffset],
		FrameAttribute.FrameAttribute->AttributeData[3][RangeOffset]);
	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeScaleAtFrame(FVector& OutScale, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx)
{
	FVector3f OutScaleFloat = FVector3f::OneVector;
	if (!FrameAttributeScaleAtFrameFloat(OutScaleFloat, FrameAttribute, SequenceIdx, FrameIdx))
	{
		OutScale = FVector::OneVector;
		return false;
	}

	OutScale = (FVector)OutScaleFloat;
	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeScaleAtFrameFloat(FVector3f& OutScale, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx)
{
	if (!FrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeScaleAtFrameFloat: Invalid FrameAttribute.");
		OutScale = FVector3f::OneVector;
		return false;
	}

	if (FrameAttribute.Type != EAnimDatabaseAttributeType::Scale)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeScaleAtFrameFloat: FrameAttribute Types don't match. Got {Type}. Expected Scale.", AttributeTypeNameInternal(FrameAttribute.Type));
		OutScale = FVector3f::OneVector;
		return false;
	}

	int32 EntryIdx = INDEX_NONE;
	int32 RangeIdx = INDEX_NONE;
	int32 RangeFrame = INDEX_NONE;
	if (!FrameAttribute.FrameAttribute->FrameRangeSet.Find(EntryIdx, RangeIdx, RangeFrame, SequenceIdx, FrameIdx))
	{
		OutScale = FVector3f::OneVector;
		return false;
	}

	const int32 RangeOffset = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeOffset(EntryIdx, RangeIdx) + RangeFrame;

	OutScale = FVector3f(
		FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset],
		FrameAttribute.FrameAttribute->AttributeData[1][RangeOffset],
		FrameAttribute.FrameAttribute->AttributeData[2][RangeOffset]);
	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeLinearVelocityAtFrame(FVector& OutLinearVelocity, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx)
{
	FVector3f OutLinearVelocityFloat = FVector3f::ZeroVector;
	if (!FrameAttributeLinearVelocityAtFrameFloat(OutLinearVelocityFloat, FrameAttribute, SequenceIdx, FrameIdx))
	{
		OutLinearVelocity = FVector::ZeroVector;
		return false;
	}

	OutLinearVelocity = (FVector)OutLinearVelocityFloat;
	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeLinearVelocityAtFrameFloat(FVector3f& OutLinearVelocity, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx)
{
	if (!FrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeLinearVelocityAtFrameFloat: Invalid FrameAttribute.");
		OutLinearVelocity = FVector3f::ZeroVector;
		return false;
	}

	if (FrameAttribute.Type != EAnimDatabaseAttributeType::LinearVelocity)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeLinearVelocityAtFrameFloat: FrameAttribute Types don't match. Got {Type}. Expected LinearVelocity.", AttributeTypeNameInternal(FrameAttribute.Type));
		OutLinearVelocity = FVector3f::ZeroVector;
		return false;
	}

	int32 EntryIdx = INDEX_NONE;
	int32 RangeIdx = INDEX_NONE;
	int32 RangeFrame = INDEX_NONE;
	if (!FrameAttribute.FrameAttribute->FrameRangeSet.Find(EntryIdx, RangeIdx, RangeFrame, SequenceIdx, FrameIdx))
	{
		OutLinearVelocity = FVector3f::ZeroVector;
		return false;
	}

	const int32 RangeOffset = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeOffset(EntryIdx, RangeIdx) + RangeFrame;

	OutLinearVelocity = FVector3f(
		FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset],
		FrameAttribute.FrameAttribute->AttributeData[1][RangeOffset],
		FrameAttribute.FrameAttribute->AttributeData[2][RangeOffset]);
	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeAngularVelocityAtFrame(FVector& OutAngularVelocity, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx)
{
	FVector3f OutAngularVelocityFloat = FVector3f::ZeroVector;
	if (!FrameAttributeAngularVelocityAtFrameFloat(OutAngularVelocityFloat, FrameAttribute, SequenceIdx, FrameIdx))
	{
		OutAngularVelocity = FVector::ZeroVector;
		return false;
	}

	OutAngularVelocity = (FVector)OutAngularVelocityFloat;
	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeAngularVelocityAtFrameFloat(FVector3f& OutAngularVelocity, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx)
{
	if (!FrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeAngularVelocityAtFrameFloat: Invalid FrameAttribute.");
		OutAngularVelocity = FVector3f::ZeroVector;
		return false;
	}

	if (FrameAttribute.Type != EAnimDatabaseAttributeType::AngularVelocity)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeAngularVelocityAtFrameFloat: FrameAttribute Types don't match. Got {Type}. Expected AngularVelocity.", AttributeTypeNameInternal(FrameAttribute.Type));
		OutAngularVelocity = FVector3f::ZeroVector;
		return false;
	}

	int32 EntryIdx = INDEX_NONE;
	int32 RangeIdx = INDEX_NONE;
	int32 RangeFrame = INDEX_NONE;
	if (!FrameAttribute.FrameAttribute->FrameRangeSet.Find(EntryIdx, RangeIdx, RangeFrame, SequenceIdx, FrameIdx))
	{
		OutAngularVelocity = FVector3f::ZeroVector;
		return false;
	}

	const int32 RangeOffset = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeOffset(EntryIdx, RangeIdx) + RangeFrame;

	OutAngularVelocity = FVector3f(
		FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset],
		FrameAttribute.FrameAttribute->AttributeData[1][RangeOffset],
		FrameAttribute.FrameAttribute->AttributeData[2][RangeOffset]);
	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeScalarVelocityAtFrame(FVector& OutScalarVelocity, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx)
{
	FVector3f OutScalarVelocityFloat = FVector3f::ZeroVector;
	if (!FrameAttributeScalarVelocityAtFrameFloat(OutScalarVelocityFloat, FrameAttribute, SequenceIdx, FrameIdx))
	{
		OutScalarVelocity = FVector::ZeroVector;
		return false;
	}

	OutScalarVelocity = (FVector)OutScalarVelocityFloat;
	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeScalarVelocityAtFrameFloat(FVector3f& OutScalarVelocity, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx)
{
	if (!FrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeScalarVelocityAtFrameFloat: Invalid FrameAttribute.");
		OutScalarVelocity = FVector3f::ZeroVector;
		return false;
	}

	if (FrameAttribute.Type != EAnimDatabaseAttributeType::ScalarVelocity)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeScalarVelocityAtFrameFloat: FrameAttribute Types don't match. Got {Type}. Expected ScalarVelocity.", AttributeTypeNameInternal(FrameAttribute.Type));
		OutScalarVelocity = FVector3f::ZeroVector;
		return false;
	}

	int32 EntryIdx = INDEX_NONE;
	int32 RangeIdx = INDEX_NONE;
	int32 RangeFrame = INDEX_NONE;
	if (!FrameAttribute.FrameAttribute->FrameRangeSet.Find(EntryIdx, RangeIdx, RangeFrame, SequenceIdx, FrameIdx))
	{
		OutScalarVelocity = FVector3f::ZeroVector;
		return false;
	}

	const int32 RangeOffset = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeOffset(EntryIdx, RangeIdx) + RangeFrame;

	OutScalarVelocity = FVector3f(
		FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset],
		FrameAttribute.FrameAttribute->AttributeData[1][RangeOffset],
		FrameAttribute.FrameAttribute->AttributeData[2][RangeOffset]);
	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeDirectionAtFrame(FVector& OutDirection, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx)
{
	FVector3f OutDirectionFloat = FVector3f::ForwardVector;
	if (!FrameAttributeDirectionAtFrameFloat(OutDirectionFloat, FrameAttribute, SequenceIdx, FrameIdx))
	{
		OutDirection = FVector::ForwardVector;
		return false;
	}

	OutDirection = ((FVector)OutDirectionFloat).GetSafeNormal();
	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeDirectionAtFrameFloat(FVector3f& OutDirection, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx)
{
	if (!FrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeDirectionAtFrameFloat: Invalid FrameAttribute.");
		OutDirection = FVector3f::ForwardVector;
		return false;
	}

	if (FrameAttribute.Type != EAnimDatabaseAttributeType::Direction)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeDirectionAtFrameFloat: FrameAttribute Types don't match. Got {Type}. Expected Direction.", AttributeTypeNameInternal(FrameAttribute.Type));
		OutDirection = FVector3f::ForwardVector;
		return false;
	}

	int32 EntryIdx = INDEX_NONE;
	int32 RangeIdx = INDEX_NONE;
	int32 RangeFrame = INDEX_NONE;
	if (!FrameAttribute.FrameAttribute->FrameRangeSet.Find(EntryIdx, RangeIdx, RangeFrame, SequenceIdx, FrameIdx))
	{
		OutDirection = FVector3f::ForwardVector;
		return false;
	}

	const int32 RangeOffset = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeOffset(EntryIdx, RangeIdx) + RangeFrame;

	OutDirection = FVector3f(
		FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset],
		FrameAttribute.FrameAttribute->AttributeData[1][RangeOffset],
		FrameAttribute.FrameAttribute->AttributeData[2][RangeOffset]);
	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeTransformAtFrame(FTransform& OutTransform, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx)
{
	FTransform3f OutTransformFloat = FTransform3f::Identity;
	if (!FrameAttributeTransformAtFrameFloat(OutTransformFloat, FrameAttribute, SequenceIdx, FrameIdx))
	{
		OutTransform = FTransform::Identity;
		return false;
	}

	OutTransform = FTransform(
		((FQuat)OutTransformFloat.GetRotation()).GetNormalized(),
		(FVector)OutTransformFloat.GetLocation(),
		(FVector)OutTransformFloat.GetScale3D());
	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeTransformAtFrameFloat(FTransform3f& OutTransform, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx)
{
	if (!FrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeTransformAtFrameFloat: Invalid FrameAttribute.");
		OutTransform = FTransform3f::Identity;
		return false;
	}

	if (FrameAttribute.Type != EAnimDatabaseAttributeType::Transform)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeTransformAtFrameFloat: FrameAttribute Types don't match. Got {Type}. Expected Transform.", AttributeTypeNameInternal(FrameAttribute.Type));
		OutTransform = FTransform3f::Identity;
		return false;
	}

	int32 EntryIdx = INDEX_NONE;
	int32 RangeIdx = INDEX_NONE;
	int32 RangeFrame = INDEX_NONE;
	if (!FrameAttribute.FrameAttribute->FrameRangeSet.Find(EntryIdx, RangeIdx, RangeFrame, SequenceIdx, FrameIdx))
	{
		OutTransform = FTransform3f::Identity;
		return false;
	}

	const int32 RangeOffset = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeOffset(EntryIdx, RangeIdx) + RangeFrame;

	OutTransform = FTransform3f(
		FQuat4f(
			FrameAttribute.FrameAttribute->AttributeData[3][RangeOffset], 
			FrameAttribute.FrameAttribute->AttributeData[4][RangeOffset], 
			FrameAttribute.FrameAttribute->AttributeData[5][RangeOffset], 
			FrameAttribute.FrameAttribute->AttributeData[6][RangeOffset]),
		FVector3f(
			FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset], 
			FrameAttribute.FrameAttribute->AttributeData[1][RangeOffset], 
			FrameAttribute.FrameAttribute->AttributeData[2][RangeOffset]),
		FVector3f(
			FrameAttribute.FrameAttribute->AttributeData[7][RangeOffset], 
			FrameAttribute.FrameAttribute->AttributeData[8][RangeOffset], 
			FrameAttribute.FrameAttribute->AttributeData[9][RangeOffset]));
	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeEventAtFrame(bool& OutTimeUntilEventKnown, float& OutTimeUntilEvent, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx)
{
	if (!FrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeEventAtFrame: Invalid FrameAttribute.");
		OutTimeUntilEventKnown = false;
		OutTimeUntilEvent = UE_MAX_FLT;
		return false;
	}

	if (FrameAttribute.Type != EAnimDatabaseAttributeType::Event)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeEventAtFrame: FrameAttribute Types don't match. Got {Type}. Expected Event.", AttributeTypeNameInternal(FrameAttribute.Type));
		OutTimeUntilEventKnown = false;
		OutTimeUntilEvent = UE_MAX_FLT;
		return false;
	}

	int32 EntryIdx = INDEX_NONE;
	int32 RangeIdx = INDEX_NONE;
	int32 RangeFrame = INDEX_NONE;
	if (!FrameAttribute.FrameAttribute->FrameRangeSet.Find(EntryIdx, RangeIdx, RangeFrame, SequenceIdx, FrameIdx))
	{
		OutTimeUntilEventKnown = false;
		OutTimeUntilEvent = UE_MAX_FLT;
		return false;
	}

	const int32 RangeOffset = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeOffset(EntryIdx, RangeIdx) + RangeFrame;

	OutTimeUntilEventKnown = FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset] == 1.0f;
	OutTimeUntilEvent = OutTimeUntilEventKnown ? FrameAttribute.FrameAttribute->AttributeData[1][RangeOffset] : UE_MAX_FLT;
	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeAngleAtFrameDegrees(float& OutValue, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx)
{
	if (!FrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeAngleAtFrameDegrees: Invalid FrameAttribute.");
		OutValue = 0.0f;
		return false;
	}

	if (FrameAttribute.Type != EAnimDatabaseAttributeType::Angle)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeAngleAtFrameDegrees: FrameAttribute Types don't match. Got {Type}. Expected Angle.", AttributeTypeNameInternal(FrameAttribute.Type));
		OutValue = 0.0f;
		return false;
	}

	int32 EntryIdx = INDEX_NONE;
	int32 RangeIdx = INDEX_NONE;
	int32 RangeFrame = INDEX_NONE;
	if (!FrameAttribute.FrameAttribute->FrameRangeSet.Find(EntryIdx, RangeIdx, RangeFrame, SequenceIdx, FrameIdx))
	{
		OutValue = 0.0f;
		return false;
	}

	const int32 RangeOffset = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeOffset(EntryIdx, RangeIdx) + RangeFrame;

	OutValue = FMath::RadiansToDegrees(FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset]);
	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeAngleAtFrameRadians(float& OutValue, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx)
{
	if (!FrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeAngleAtFrameRadians: Invalid FrameAttribute.");
		OutValue = 0.0f;
		return false;
	}

	if (FrameAttribute.Type != EAnimDatabaseAttributeType::Angle)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeAngleAtFrameRadians: FrameAttribute Types don't match. Got {Type}. Expected Angle.", AttributeTypeNameInternal(FrameAttribute.Type));
		OutValue = 0.0f;
		return false;
	}

	int32 EntryIdx = INDEX_NONE;
	int32 RangeIdx = INDEX_NONE;
	int32 RangeFrame = INDEX_NONE;
	if (!FrameAttribute.FrameAttribute->FrameRangeSet.Find(EntryIdx, RangeIdx, RangeFrame, SequenceIdx, FrameIdx))
	{
		OutValue = 0.0f;
		return false;
	}

	const int32 RangeOffset = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeOffset(EntryIdx, RangeIdx) + RangeFrame;

	OutValue = FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset];
	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeLocationsAtFrame(TArray<FVector>& OutLocations, const TArray<FAnimDatabaseFrameAttribute>& FrameAttributes, const int32 SequenceIdx, const int32 FrameIdx)
{
	OutLocations.SetNum(FrameAttributes.Num());
	return FrameAttributeLocationsAtFrameToArrayView(OutLocations, FrameAttributes, SequenceIdx, FrameIdx);
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeLocationsAtFrameToArrayView(const TArrayView<FVector> OutLocations, const TArrayView<const FAnimDatabaseFrameAttribute> FrameAttributes, const int32 SequenceIdx, const int32 FrameIdx)
{
	check(OutLocations.Num() == FrameAttributes.Num());
	for (int32 Idx = 0; Idx < FrameAttributes.Num(); Idx++)
	{
		if (!FrameAttributeLocationAtFrame(OutLocations[Idx], FrameAttributes[Idx], SequenceIdx, FrameIdx))
		{
			UE::Learning::Array::Set<1, FVector>(OutLocations, FVector::ZeroVector);
			return false;
		}
	}

	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeDirectionsAtFrame(TArray<FVector>& OutDirections, const TArray<FAnimDatabaseFrameAttribute>& FrameAttributes, const int32 SequenceIdx, const int32 FrameIdx)
{
	OutDirections.SetNum(FrameAttributes.Num());
	return FrameAttributeDirectionsAtFrameToArrayView(OutDirections, FrameAttributes, SequenceIdx, FrameIdx);
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeDirectionsAtFrameToArrayView(const TArrayView<FVector> OutDirections, const TArrayView<const FAnimDatabaseFrameAttribute> FrameAttributes, const int32 SequenceIdx, const int32 FrameIdx)
{
	check(OutDirections.Num() == FrameAttributes.Num());
	for (int32 Idx = 0; Idx < FrameAttributes.Num(); Idx++)
	{
		if (!FrameAttributeDirectionAtFrame(OutDirections[Idx], FrameAttributes[Idx], SequenceIdx, FrameIdx))
		{
			UE::Learning::Array::Set<1, FVector>(OutDirections, FVector::ForwardVector);
			return false;
		}
	}

	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeLinearVelocitiesAtFrame(TArray<FVector>& OutLinearVelocities, const TArray<FAnimDatabaseFrameAttribute>& FrameAttributes, const int32 SequenceIdx, const int32 FrameIdx)
{
	OutLinearVelocities.SetNum(FrameAttributes.Num());
	return FrameAttributeLinearVelocitiesAtFrameToArrayView(OutLinearVelocities, FrameAttributes, SequenceIdx, FrameIdx);
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeLinearVelocitiesAtFrameToArrayView(const TArrayView<FVector> OutLinearVelocities, const TArrayView<const FAnimDatabaseFrameAttribute> FrameAttributes, const int32 SequenceIdx, const int32 FrameIdx)
{
	check(OutLinearVelocities.Num() == FrameAttributes.Num());
	for (int32 Idx = 0; Idx < FrameAttributes.Num(); Idx++)
	{
		if (!FrameAttributeLinearVelocityAtFrame(OutLinearVelocities[Idx], FrameAttributes[Idx], SequenceIdx, FrameIdx))
		{
			UE::Learning::Array::Set<1, FVector>(OutLinearVelocities, FVector::ZeroVector);
			return false;
		}
	}

	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleBool(bool& OutBool, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate)
{
	if (!FrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeSampleBool: Invalid FrameAttribute.");
		OutBool = false;
		return false;
	}

	if (FrameAttribute.Type != EAnimDatabaseAttributeType::Bool)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeSampleBool: FrameAttribute Types don't match. Got {Type}. Expected Bool.", AttributeTypeNameInternal(FrameAttribute.Type));
		OutBool = false;
		return false;
	}

	int32 EntryIdx = INDEX_NONE;
	int32 RangeIdx = INDEX_NONE;
	float RangeTime = 1.0f;
	if (!FrameAttribute.FrameAttribute->FrameRangeSet.FindTime(EntryIdx, RangeIdx, RangeTime, SequenceIdx, SequenceTime, 1.0f / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER)))
	{
		OutBool = false;
		return false;
	}

	const int32 RangeOffset = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeOffset(EntryIdx, RangeIdx);
	const int32 RangeLength = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeLength(EntryIdx, RangeIdx);
	const float RangeFrameTime = 1.0f / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER);
	const int64 SampleFrameNearest = UE::AnimDatabase::Math::ComputeNearestSampleFrame(RangeTime / RangeFrameTime, RangeLength);

	OutBool = FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset + SampleFrameNearest] == 1.0f;
	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleFloat(float& OutValue, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate)
{
	if (!FrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeSampleFloat: Invalid FrameAttribute.");
		OutValue = 0.0f;
		return false;
	}

	if (FrameAttribute.Type != EAnimDatabaseAttributeType::Float)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeSampleFloat: FrameAttribute Types don't match. Got {Type}. Expected Float.", AttributeTypeNameInternal(FrameAttribute.Type));
		OutValue = 0.0f;
		return false;
	}

	int32 EntryIdx = INDEX_NONE;
	int32 RangeIdx = INDEX_NONE;
	float RangeTime = 1.0f;
	if (!FrameAttribute.FrameAttribute->FrameRangeSet.FindTime(EntryIdx, RangeIdx, RangeTime, SequenceIdx, SequenceTime, 1.0f / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER)))
	{
		OutValue = 0.0f;
		return false;
	}

	const int32 RangeOffset = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeOffset(EntryIdx, RangeIdx);
	const int32 RangeLength = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeLength(EntryIdx, RangeIdx);
	const float RangeFrameTime = 1.0f / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER);

	int64 SampleFrame0, SampleFrame1, SampleFrame2, SampleFrame3;
	float SampleAlpha;
	UE::AnimDatabase::Math::ComputeCubicSampleFramesAndAlpha(
		SampleFrame0,
		SampleFrame1,
		SampleFrame2,
		SampleFrame3,
		SampleAlpha,
		RangeTime /	RangeFrameTime,
		RangeLength);

	if (SampleFrame0 == SampleFrame1 && SampleFrame2 == SampleFrame3)
	{
		UE::AnimDatabase::Math::ValueInterpolateLinear(
			OutValue,
			FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset + SampleFrame1],
			FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset + SampleFrame2],
			SampleAlpha);
	}
	else if (SampleFrame0 == SampleFrame1)
	{
		UE::AnimDatabase::Math::ValueInterpolateCubicMonoStart(
			OutValue,
			FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset + SampleFrame1],
			FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset + SampleFrame2],
			FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset + SampleFrame3],
			SampleAlpha);
	}
	else if (SampleFrame2 == SampleFrame3)
	{
		UE::AnimDatabase::Math::ValueInterpolateCubicMonoEnd(
			OutValue,
			FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset + SampleFrame0],
			FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset + SampleFrame1],
			FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset + SampleFrame2],
			SampleAlpha);
	}
	else
	{
		UE::AnimDatabase::Math::ValueInterpolateCubicMono(
			OutValue,
			FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset + SampleFrame0],
			FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset + SampleFrame1],
			FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset + SampleFrame2],
			FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset + SampleFrame3],
			SampleAlpha);
	}

	return true;
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleFloatRange(TArray<float>& OutValues, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float StartTime, const float EndTime, const int32 Num, const FFrameRate& FrameRate)
{
	OutValues.Init(0.0, Num);
	FrameAttributeSampleFloatRangeToArrayView(OutValues, FrameAttribute, SequenceIdx, StartTime, EndTime, Num, FrameRate);
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleFloatRangeToArrayView(TArrayView<float> OutValues, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float StartTime, const float EndTime, const int32 Num, const FFrameRate& FrameRate)
{
	check(OutValues.Num() == Num);
	
	if (Num == 1)
	{
		FrameAttributeSampleFloat(OutValues[0], FrameAttribute, SequenceIdx, StartTime, FrameRate);
		return;
	}
	
	for (int32 Idx = 0; Idx < Num; Idx++)
	{
		FrameAttributeSampleFloat(OutValues[Idx], FrameAttribute, SequenceIdx, StartTime + (EndTime - StartTime) * ((float)Idx) / (Num - 1), FrameRate);
	}
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleLocation(FVector& OutLocation, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate)
{
	FVector3f OutLocationFloat = FVector3f::ZeroVector;
	if (!FrameAttributeSampleLocationFloat(OutLocationFloat, FrameAttribute, SequenceIdx, SequenceTime, FrameRate))
	{
		OutLocation = FVector::ZeroVector;
		return false;
	}

	OutLocation = (FVector)OutLocationFloat;
	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleLocationFloat(FVector3f& OutLocation, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate)
{
	if (!FrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeSampleLocationFloat: Invalid FrameAttribute.");
		OutLocation = FVector3f::ZeroVector;
		return false;
	}

	if (FrameAttribute.Type != EAnimDatabaseAttributeType::Location)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeSampleLocationFloat: FrameAttribute Types don't match. Got {Type}. Expected Location.", AttributeTypeNameInternal(FrameAttribute.Type));
		OutLocation = FVector3f::ZeroVector;
		return false;
	}

	int32 EntryIdx = INDEX_NONE;
	int32 RangeIdx = INDEX_NONE;
	float RangeTime = 1.0f;
	if (!FrameAttribute.FrameAttribute->FrameRangeSet.FindTime(EntryIdx, RangeIdx, RangeTime, SequenceIdx, SequenceTime, 1.0f / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER)))
	{
		OutLocation = FVector3f::ZeroVector;
		return false;
	}

	const int32 RangeOffset = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeOffset(EntryIdx, RangeIdx);
	const int32 RangeLength = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeLength(EntryIdx, RangeIdx);
	const float RangeFrameTime = 1.0f / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER);

	int64 SampleFrame0, SampleFrame1, SampleFrame2, SampleFrame3;
	float SampleAlpha;
	UE::AnimDatabase::Math::ComputeCubicSampleFramesAndAlpha(
		SampleFrame0,
		SampleFrame1,
		SampleFrame2,
		SampleFrame3,
		SampleAlpha,
		RangeTime /	RangeFrameTime,
		RangeLength);

	const TLearningArrayView<2, const float> FrameAttributeData = FrameAttribute.FrameAttribute->AttributeData;

	if (SampleFrame0 == SampleFrame1 && SampleFrame2 == SampleFrame3)
	{
		UE::AnimDatabase::Math::LocationInterpolateLinear(
			OutLocation,
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame1], FrameAttributeData[1][RangeOffset + SampleFrame1], FrameAttributeData[2][RangeOffset + SampleFrame1]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame2], FrameAttributeData[1][RangeOffset + SampleFrame2], FrameAttributeData[2][RangeOffset + SampleFrame2]),
			SampleAlpha);
	}
	else if (SampleFrame0 == SampleFrame1)
	{
		UE::AnimDatabase::Math::LocationInterpolateCubicMonoStart(
			OutLocation,
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame1], FrameAttributeData[1][RangeOffset + SampleFrame1], FrameAttributeData[2][RangeOffset + SampleFrame1]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame2], FrameAttributeData[1][RangeOffset + SampleFrame2], FrameAttributeData[2][RangeOffset + SampleFrame2]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame3], FrameAttributeData[1][RangeOffset + SampleFrame3], FrameAttributeData[2][RangeOffset + SampleFrame3]),
			SampleAlpha);
	}
	else if (SampleFrame2 == SampleFrame3)
	{
		UE::AnimDatabase::Math::LocationInterpolateCubicMonoEnd(
			OutLocation,
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame0], FrameAttributeData[1][RangeOffset + SampleFrame0], FrameAttributeData[2][RangeOffset + SampleFrame0]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame1], FrameAttributeData[1][RangeOffset + SampleFrame1], FrameAttributeData[2][RangeOffset + SampleFrame1]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame2], FrameAttributeData[1][RangeOffset + SampleFrame2], FrameAttributeData[2][RangeOffset + SampleFrame2]),
			SampleAlpha);
	}
	else
	{
		UE::AnimDatabase::Math::LocationInterpolateCubicMono(
			OutLocation,
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame0], FrameAttributeData[1][RangeOffset + SampleFrame0], FrameAttributeData[2][RangeOffset + SampleFrame0]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame1], FrameAttributeData[1][RangeOffset + SampleFrame1], FrameAttributeData[2][RangeOffset + SampleFrame1]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame2], FrameAttributeData[1][RangeOffset + SampleFrame2], FrameAttributeData[2][RangeOffset + SampleFrame2]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame3], FrameAttributeData[1][RangeOffset + SampleFrame3], FrameAttributeData[2][RangeOffset + SampleFrame3]),
			SampleAlpha);
	}

	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleRotation(FRotator& OutRotation, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate)
{
	FQuat OutQuat = FQuat::Identity;
	if (!FrameAttributeSampleRotationAsQuat(OutQuat, FrameAttribute, SequenceIdx, SequenceTime, FrameRate))
	{
		OutRotation = FRotator::ZeroRotator;
		return false;
	}

	OutRotation = OutQuat.Rotator();
	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleRotationAsQuat(FQuat& OutRotation, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate)
{
	FQuat4f OutRotationFloat = FQuat4f::Identity;
	if (!FrameAttributeSampleRotationAsQuatFloat(OutRotationFloat, FrameAttribute, SequenceIdx, SequenceTime, FrameRate))
	{
		OutRotation = FQuat::Identity;
		return false;
	}

	OutRotation = ((FQuat)OutRotationFloat).GetNormalized();
	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleRotationAsQuatFloat(FQuat4f& OutRotation, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate)
{
	if (!FrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeSampleRotationAsQuatFloat: Invalid FrameAttribute.");
		OutRotation = FQuat4f::Identity;
		return false;
	}

	if (FrameAttribute.Type != EAnimDatabaseAttributeType::Rotation)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeSampleRotationAsQuatFloat: FrameAttribute Types don't match. Got {Type}. Expected Rotation.", AttributeTypeNameInternal(FrameAttribute.Type));
		OutRotation = FQuat4f::Identity;
		return false;
	}

	int32 EntryIdx = INDEX_NONE;
	int32 RangeIdx = INDEX_NONE;
	float RangeTime = 1.0f;
	if (!FrameAttribute.FrameAttribute->FrameRangeSet.FindTime(EntryIdx, RangeIdx, RangeTime, SequenceIdx, SequenceTime, 1.0f / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER)))
	{
		OutRotation = FQuat4f::Identity;
		return false;
	}

	const int32 RangeOffset = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeOffset(EntryIdx, RangeIdx);
	const int32 RangeLength = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeLength(EntryIdx, RangeIdx);
	const float RangeFrameTime = 1.0f / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER);

	int64 SampleFrame0, SampleFrame1, SampleFrame2, SampleFrame3;
	float SampleAlpha;
	UE::AnimDatabase::Math::ComputeCubicSampleFramesAndAlpha(
		SampleFrame0,
		SampleFrame1,
		SampleFrame2,
		SampleFrame3,
		SampleAlpha,
		RangeTime /	RangeFrameTime,
		RangeLength);

	const TLearningArrayView<2, const float> FrameAttributeData = FrameAttribute.FrameAttribute->AttributeData;

	if (SampleFrame0 == SampleFrame1 && SampleFrame2 == SampleFrame3)
	{
		UE::AnimDatabase::Math::RotationInterpolateLinear(
			OutRotation,
			FQuat4f(FrameAttributeData[0][RangeOffset + SampleFrame1], FrameAttributeData[1][RangeOffset + SampleFrame1], FrameAttributeData[2][RangeOffset + SampleFrame1], FrameAttributeData[3][RangeOffset + SampleFrame1]),
			FQuat4f(FrameAttributeData[0][RangeOffset + SampleFrame2], FrameAttributeData[1][RangeOffset + SampleFrame2], FrameAttributeData[2][RangeOffset + SampleFrame2], FrameAttributeData[3][RangeOffset + SampleFrame2]),
			SampleAlpha);
	}
	else if (SampleFrame0 == SampleFrame1)
	{
		UE::AnimDatabase::Math::RotationInterpolateCubicMonoStart(
			OutRotation,
			FQuat4f(FrameAttributeData[0][RangeOffset + SampleFrame1], FrameAttributeData[1][RangeOffset + SampleFrame1], FrameAttributeData[2][RangeOffset + SampleFrame1], FrameAttributeData[3][RangeOffset + SampleFrame1]),
			FQuat4f(FrameAttributeData[0][RangeOffset + SampleFrame2], FrameAttributeData[1][RangeOffset + SampleFrame2], FrameAttributeData[2][RangeOffset + SampleFrame2], FrameAttributeData[3][RangeOffset + SampleFrame2]),
			FQuat4f(FrameAttributeData[0][RangeOffset + SampleFrame3], FrameAttributeData[1][RangeOffset + SampleFrame3], FrameAttributeData[2][RangeOffset + SampleFrame3], FrameAttributeData[3][RangeOffset + SampleFrame3]),
			SampleAlpha);
	}
	else if (SampleFrame2 == SampleFrame3)
	{
		UE::AnimDatabase::Math::RotationInterpolateCubicMonoEnd(
			OutRotation,
			FQuat4f(FrameAttributeData[0][RangeOffset + SampleFrame0], FrameAttributeData[1][RangeOffset + SampleFrame0], FrameAttributeData[2][RangeOffset + SampleFrame0], FrameAttributeData[3][RangeOffset + SampleFrame0]),
			FQuat4f(FrameAttributeData[0][RangeOffset + SampleFrame1], FrameAttributeData[1][RangeOffset + SampleFrame1], FrameAttributeData[2][RangeOffset + SampleFrame1], FrameAttributeData[3][RangeOffset + SampleFrame1]),
			FQuat4f(FrameAttributeData[0][RangeOffset + SampleFrame2], FrameAttributeData[1][RangeOffset + SampleFrame2], FrameAttributeData[2][RangeOffset + SampleFrame2], FrameAttributeData[3][RangeOffset + SampleFrame2]),
			SampleAlpha);
	}
	else
	{
		UE::AnimDatabase::Math::RotationInterpolateCubicMono(
			OutRotation,
			FQuat4f(FrameAttributeData[0][RangeOffset + SampleFrame0], FrameAttributeData[1][RangeOffset + SampleFrame0], FrameAttributeData[2][RangeOffset + SampleFrame0], FrameAttributeData[3][RangeOffset + SampleFrame0]),
			FQuat4f(FrameAttributeData[0][RangeOffset + SampleFrame1], FrameAttributeData[1][RangeOffset + SampleFrame1], FrameAttributeData[2][RangeOffset + SampleFrame1], FrameAttributeData[3][RangeOffset + SampleFrame1]),
			FQuat4f(FrameAttributeData[0][RangeOffset + SampleFrame2], FrameAttributeData[1][RangeOffset + SampleFrame2], FrameAttributeData[2][RangeOffset + SampleFrame2], FrameAttributeData[3][RangeOffset + SampleFrame2]),
			FQuat4f(FrameAttributeData[0][RangeOffset + SampleFrame3], FrameAttributeData[1][RangeOffset + SampleFrame3], FrameAttributeData[2][RangeOffset + SampleFrame3], FrameAttributeData[3][RangeOffset + SampleFrame3]),
			SampleAlpha);
	}

	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleScale(FVector& OutScale, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate)
{
	FVector3f OutScaleFloat = FVector3f::OneVector;
	if (!FrameAttributeSampleScaleFloat(OutScaleFloat, FrameAttribute, SequenceIdx, SequenceTime, FrameRate))
	{
		OutScale = FVector::OneVector;
		return false;
	}

	OutScale = (FVector)OutScaleFloat;
	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleScaleFloat(FVector3f& OutScale, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate)
{
	if (!FrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeSampleScaleFloat: Invalid FrameAttribute.");
		OutScale = FVector3f::OneVector;
		return false;
	}

	if (FrameAttribute.Type != EAnimDatabaseAttributeType::Scale)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeSampleScaleFloat: FrameAttribute Types don't match. Got {Type}. Expected Scale.", AttributeTypeNameInternal(FrameAttribute.Type));
		OutScale = FVector3f::OneVector;
		return false;
	}

	int32 EntryIdx = INDEX_NONE;
	int32 RangeIdx = INDEX_NONE;
	float RangeTime = 1.0f;
	if (!FrameAttribute.FrameAttribute->FrameRangeSet.FindTime(EntryIdx, RangeIdx, RangeTime, SequenceIdx, SequenceTime, 1.0f / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER)))
	{
		OutScale = FVector3f::OneVector;
		return false;
	}

	const int32 RangeOffset = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeOffset(EntryIdx, RangeIdx);
	const int32 RangeLength = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeLength(EntryIdx, RangeIdx);
	const float RangeFrameTime = 1.0f / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER);

	int64 SampleFrame0, SampleFrame1, SampleFrame2, SampleFrame3;
	float SampleAlpha;
	UE::AnimDatabase::Math::ComputeCubicSampleFramesAndAlpha(
		SampleFrame0,
		SampleFrame1,
		SampleFrame2,
		SampleFrame3,
		SampleAlpha,
		RangeTime /	RangeFrameTime,
		RangeLength);

	const TLearningArrayView<2, const float> FrameAttributeData = FrameAttribute.FrameAttribute->AttributeData;

	if (SampleFrame0 == SampleFrame1 && SampleFrame2 == SampleFrame3)
	{
		UE::AnimDatabase::Math::ScaleInterpolateLinear(
			OutScale,
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame1], FrameAttributeData[1][RangeOffset + SampleFrame1], FrameAttributeData[2][RangeOffset + SampleFrame1]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame2], FrameAttributeData[1][RangeOffset + SampleFrame2], FrameAttributeData[2][RangeOffset + SampleFrame2]),
			SampleAlpha);
	}
	else if (SampleFrame0 == SampleFrame1)
	{
		UE::AnimDatabase::Math::ScaleInterpolateCubicMonoStart(
			OutScale,
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame1], FrameAttributeData[1][RangeOffset + SampleFrame1], FrameAttributeData[2][RangeOffset + SampleFrame1]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame2], FrameAttributeData[1][RangeOffset + SampleFrame2], FrameAttributeData[2][RangeOffset + SampleFrame2]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame3], FrameAttributeData[1][RangeOffset + SampleFrame3], FrameAttributeData[2][RangeOffset + SampleFrame3]),
			SampleAlpha);
	}
	else if (SampleFrame2 == SampleFrame3)
	{
		UE::AnimDatabase::Math::ScaleInterpolateCubicMonoEnd(
			OutScale,
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame0], FrameAttributeData[1][RangeOffset + SampleFrame0], FrameAttributeData[2][RangeOffset + SampleFrame0]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame1], FrameAttributeData[1][RangeOffset + SampleFrame1], FrameAttributeData[2][RangeOffset + SampleFrame1]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame2], FrameAttributeData[1][RangeOffset + SampleFrame2], FrameAttributeData[2][RangeOffset + SampleFrame2]),
			SampleAlpha);
	}
	else
	{
		UE::AnimDatabase::Math::ScaleInterpolateCubicMono(
			OutScale,
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame0], FrameAttributeData[1][RangeOffset + SampleFrame0], FrameAttributeData[2][RangeOffset + SampleFrame0]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame1], FrameAttributeData[1][RangeOffset + SampleFrame1], FrameAttributeData[2][RangeOffset + SampleFrame1]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame2], FrameAttributeData[1][RangeOffset + SampleFrame2], FrameAttributeData[2][RangeOffset + SampleFrame2]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame3], FrameAttributeData[1][RangeOffset + SampleFrame3], FrameAttributeData[2][RangeOffset + SampleFrame3]),
			SampleAlpha);
	}

	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleLinearVelocity(FVector& OutLinearVelocity, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate)
{
	FVector3f OutLinearVelocityFloat = FVector3f::ZeroVector;
	if (!FrameAttributeSampleLinearVelocityFloat(OutLinearVelocityFloat, FrameAttribute, SequenceIdx, SequenceTime, FrameRate))
	{
		OutLinearVelocity = FVector::ZeroVector;
		return false;
	}

	OutLinearVelocity = (FVector)OutLinearVelocityFloat;
	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleLinearVelocityFloat(FVector3f& OutLinearVelocity, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate)
{
	if (!FrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeSampleLinearVelocityFloat: Invalid FrameAttribute.");
		OutLinearVelocity = FVector3f::ZeroVector;
		return false;
	}

	if (FrameAttribute.Type != EAnimDatabaseAttributeType::LinearVelocity)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeSampleLinearVelocityFloat: FrameAttribute Types don't match. Got {Type}. Expected LinearVelocity.", AttributeTypeNameInternal(FrameAttribute.Type));
		OutLinearVelocity = FVector3f::ZeroVector;
		return false;
	}

	int32 EntryIdx = INDEX_NONE;
	int32 RangeIdx = INDEX_NONE;
	float RangeTime = 1.0f;
	if (!FrameAttribute.FrameAttribute->FrameRangeSet.FindTime(EntryIdx, RangeIdx, RangeTime, SequenceIdx, SequenceTime, 1.0f / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER)))
	{
		OutLinearVelocity = FVector3f::ZeroVector;
		return false;
	}

	const int32 RangeOffset = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeOffset(EntryIdx, RangeIdx);
	const int32 RangeLength = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeLength(EntryIdx, RangeIdx);
	const float RangeFrameTime = 1.0f / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER);

	int64 SampleFrame0, SampleFrame1, SampleFrame2, SampleFrame3;
	float SampleAlpha;
	UE::AnimDatabase::Math::ComputeCubicSampleFramesAndAlpha(
		SampleFrame0,
		SampleFrame1,
		SampleFrame2,
		SampleFrame3,
		SampleAlpha,
		RangeTime /	RangeFrameTime,
		RangeLength);

	const TLearningArrayView<2, const float> FrameAttributeData = FrameAttribute.FrameAttribute->AttributeData;

	if (SampleFrame0 == SampleFrame1 && SampleFrame2 == SampleFrame3)
	{
		UE::AnimDatabase::Math::LocationInterpolateLinear(
			OutLinearVelocity,
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame1], FrameAttributeData[1][RangeOffset + SampleFrame1], FrameAttributeData[2][RangeOffset + SampleFrame1]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame2], FrameAttributeData[1][RangeOffset + SampleFrame2], FrameAttributeData[2][RangeOffset + SampleFrame2]),
			SampleAlpha);
	}
	else if (SampleFrame0 == SampleFrame1)
	{
		UE::AnimDatabase::Math::LocationInterpolateCubicMonoStart(
			OutLinearVelocity,
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame1], FrameAttributeData[1][RangeOffset + SampleFrame1], FrameAttributeData[2][RangeOffset + SampleFrame1]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame2], FrameAttributeData[1][RangeOffset + SampleFrame2], FrameAttributeData[2][RangeOffset + SampleFrame2]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame3], FrameAttributeData[1][RangeOffset + SampleFrame3], FrameAttributeData[2][RangeOffset + SampleFrame3]),
			SampleAlpha);
	}
	else if (SampleFrame2 == SampleFrame3)
	{
		UE::AnimDatabase::Math::LocationInterpolateCubicMonoEnd(
			OutLinearVelocity,
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame0], FrameAttributeData[1][RangeOffset + SampleFrame0], FrameAttributeData[2][RangeOffset + SampleFrame0]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame1], FrameAttributeData[1][RangeOffset + SampleFrame1], FrameAttributeData[2][RangeOffset + SampleFrame1]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame2], FrameAttributeData[1][RangeOffset + SampleFrame2], FrameAttributeData[2][RangeOffset + SampleFrame2]),
			SampleAlpha);
	}
	else
	{
		UE::AnimDatabase::Math::LocationInterpolateCubicMono(
			OutLinearVelocity,
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame0], FrameAttributeData[1][RangeOffset + SampleFrame0], FrameAttributeData[2][RangeOffset + SampleFrame0]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame1], FrameAttributeData[1][RangeOffset + SampleFrame1], FrameAttributeData[2][RangeOffset + SampleFrame1]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame2], FrameAttributeData[1][RangeOffset + SampleFrame2], FrameAttributeData[2][RangeOffset + SampleFrame2]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame3], FrameAttributeData[1][RangeOffset + SampleFrame3], FrameAttributeData[2][RangeOffset + SampleFrame3]),
			SampleAlpha);
	}

	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleAngularVelocity(FVector& OutAngularVelocity, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate)
{
	FVector3f OutAngularVelocityFloat = FVector3f::ZeroVector;
	if (!FrameAttributeSampleAngularVelocityFloat(OutAngularVelocityFloat, FrameAttribute, SequenceIdx, SequenceTime, FrameRate))
	{
		OutAngularVelocity = FVector::ZeroVector;
		return false;
	}

	OutAngularVelocity = (FVector)OutAngularVelocityFloat;
	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleAngularVelocityFloat(FVector3f& OutAngularVelocity, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate)
{
	if (!FrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeSampleAngularVelocityFloat: Invalid FrameAttribute.");
		OutAngularVelocity = FVector3f::ZeroVector;
		return false;
	}

	if (FrameAttribute.Type != EAnimDatabaseAttributeType::AngularVelocity)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeSampleAngularVelocityFloat: FrameAttribute Types don't match. Got {Type}. Expected AngularVelocity.", AttributeTypeNameInternal(FrameAttribute.Type));
		OutAngularVelocity = FVector3f::ZeroVector;
		return false;
	}

	int32 EntryIdx = INDEX_NONE;
	int32 RangeIdx = INDEX_NONE;
	float RangeTime = 1.0f;
	if (!FrameAttribute.FrameAttribute->FrameRangeSet.FindTime(EntryIdx, RangeIdx, RangeTime, SequenceIdx, SequenceTime, 1.0f / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER)))
	{
		OutAngularVelocity = FVector3f::ZeroVector;
		return false;
	}

	const int32 RangeOffset = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeOffset(EntryIdx, RangeIdx);
	const int32 RangeLength = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeLength(EntryIdx, RangeIdx);
	const float RangeFrameTime = 1.0f / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER);

	int64 SampleFrame0, SampleFrame1, SampleFrame2, SampleFrame3;
	float SampleAlpha;
	UE::AnimDatabase::Math::ComputeCubicSampleFramesAndAlpha(
		SampleFrame0,
		SampleFrame1,
		SampleFrame2,
		SampleFrame3,
		SampleAlpha,
		RangeTime /	RangeFrameTime,
		RangeLength);

	const TLearningArrayView<2, const float> FrameAttributeData = FrameAttribute.FrameAttribute->AttributeData;

	if (SampleFrame0 == SampleFrame1 && SampleFrame2 == SampleFrame3)
	{
		UE::AnimDatabase::Math::LocationInterpolateLinear(
			OutAngularVelocity,
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame1], FrameAttributeData[1][RangeOffset + SampleFrame1], FrameAttributeData[2][RangeOffset + SampleFrame1]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame2], FrameAttributeData[1][RangeOffset + SampleFrame2], FrameAttributeData[2][RangeOffset + SampleFrame2]),
			SampleAlpha);
	}
	else if (SampleFrame0 == SampleFrame1)
	{
		UE::AnimDatabase::Math::LocationInterpolateCubicMonoStart(
			OutAngularVelocity,
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame1], FrameAttributeData[1][RangeOffset + SampleFrame1], FrameAttributeData[2][RangeOffset + SampleFrame1]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame2], FrameAttributeData[1][RangeOffset + SampleFrame2], FrameAttributeData[2][RangeOffset + SampleFrame2]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame3], FrameAttributeData[1][RangeOffset + SampleFrame3], FrameAttributeData[2][RangeOffset + SampleFrame3]),
			SampleAlpha);
	}
	else if (SampleFrame2 == SampleFrame3)
	{
		UE::AnimDatabase::Math::LocationInterpolateCubicMonoEnd(
			OutAngularVelocity,
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame0], FrameAttributeData[1][RangeOffset + SampleFrame0], FrameAttributeData[2][RangeOffset + SampleFrame0]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame1], FrameAttributeData[1][RangeOffset + SampleFrame1], FrameAttributeData[2][RangeOffset + SampleFrame1]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame2], FrameAttributeData[1][RangeOffset + SampleFrame2], FrameAttributeData[2][RangeOffset + SampleFrame2]),
			SampleAlpha);
	}
	else
	{
		UE::AnimDatabase::Math::LocationInterpolateCubicMono(
			OutAngularVelocity,
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame0], FrameAttributeData[1][RangeOffset + SampleFrame0], FrameAttributeData[2][RangeOffset + SampleFrame0]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame1], FrameAttributeData[1][RangeOffset + SampleFrame1], FrameAttributeData[2][RangeOffset + SampleFrame1]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame2], FrameAttributeData[1][RangeOffset + SampleFrame2], FrameAttributeData[2][RangeOffset + SampleFrame2]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame3], FrameAttributeData[1][RangeOffset + SampleFrame3], FrameAttributeData[2][RangeOffset + SampleFrame3]),
			SampleAlpha);
	}

	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleScalarVelocity(FVector& OutScalarVelocity, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate)
{
	FVector3f OutScalarVelocityFloat = FVector3f::ZeroVector;
	if (!FrameAttributeSampleScalarVelocityFloat(OutScalarVelocityFloat, FrameAttribute, SequenceIdx, SequenceTime, FrameRate))
	{
		OutScalarVelocity = FVector::ZeroVector;
		return false;
	}

	OutScalarVelocity = (FVector)OutScalarVelocityFloat;
	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleScalarVelocityFloat(FVector3f& OutScalarVelocity, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate)
{
	if (!FrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeSampleScalarVelocityFloat: Invalid FrameAttribute.");
		OutScalarVelocity = FVector3f::ZeroVector;
		return false;
	}

	if (FrameAttribute.Type != EAnimDatabaseAttributeType::ScalarVelocity)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeSampleScalarVelocityFloat: FrameAttribute Types don't match. Got {Type}. Expected ScalarVelocity.", AttributeTypeNameInternal(FrameAttribute.Type));
		OutScalarVelocity = FVector3f::ZeroVector;
		return false;
	}

	int32 EntryIdx = INDEX_NONE;
	int32 RangeIdx = INDEX_NONE;
	float RangeTime = 1.0f;
	if (!FrameAttribute.FrameAttribute->FrameRangeSet.FindTime(EntryIdx, RangeIdx, RangeTime, SequenceIdx, SequenceTime, 1.0f / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER)))
	{
		OutScalarVelocity = FVector3f::ZeroVector;
		return false;
	}

	const int32 RangeOffset = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeOffset(EntryIdx, RangeIdx);
	const int32 RangeLength = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeLength(EntryIdx, RangeIdx);
	const float RangeFrameTime = 1.0f / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER);

	int64 SampleFrame0, SampleFrame1, SampleFrame2, SampleFrame3;
	float SampleAlpha;
	UE::AnimDatabase::Math::ComputeCubicSampleFramesAndAlpha(
		SampleFrame0,
		SampleFrame1,
		SampleFrame2,
		SampleFrame3,
		SampleAlpha,
		RangeTime /	RangeFrameTime,
		RangeLength);

	const TLearningArrayView<2, const float> FrameAttributeData = FrameAttribute.FrameAttribute->AttributeData;

	if (SampleFrame0 == SampleFrame1 && SampleFrame2 == SampleFrame3)
	{
		UE::AnimDatabase::Math::LocationInterpolateLinear(
			OutScalarVelocity,
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame1], FrameAttributeData[1][RangeOffset + SampleFrame1], FrameAttributeData[2][RangeOffset + SampleFrame1]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame2], FrameAttributeData[1][RangeOffset + SampleFrame2], FrameAttributeData[2][RangeOffset + SampleFrame2]),
			SampleAlpha);
	}
	else if (SampleFrame0 == SampleFrame1)
	{
		UE::AnimDatabase::Math::LocationInterpolateCubicMonoStart(
			OutScalarVelocity,
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame1], FrameAttributeData[1][RangeOffset + SampleFrame1], FrameAttributeData[2][RangeOffset + SampleFrame1]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame2], FrameAttributeData[1][RangeOffset + SampleFrame2], FrameAttributeData[2][RangeOffset + SampleFrame2]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame3], FrameAttributeData[1][RangeOffset + SampleFrame3], FrameAttributeData[2][RangeOffset + SampleFrame3]),
			SampleAlpha);
	}
	else if (SampleFrame2 == SampleFrame3)
	{
		UE::AnimDatabase::Math::LocationInterpolateCubicMonoEnd(
			OutScalarVelocity,
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame0], FrameAttributeData[1][RangeOffset + SampleFrame0], FrameAttributeData[2][RangeOffset + SampleFrame0]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame1], FrameAttributeData[1][RangeOffset + SampleFrame1], FrameAttributeData[2][RangeOffset + SampleFrame1]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame2], FrameAttributeData[1][RangeOffset + SampleFrame2], FrameAttributeData[2][RangeOffset + SampleFrame2]),
			SampleAlpha);
	}
	else
	{
		UE::AnimDatabase::Math::LocationInterpolateCubicMono(
			OutScalarVelocity,
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame0], FrameAttributeData[1][RangeOffset + SampleFrame0], FrameAttributeData[2][RangeOffset + SampleFrame0]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame1], FrameAttributeData[1][RangeOffset + SampleFrame1], FrameAttributeData[2][RangeOffset + SampleFrame1]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame2], FrameAttributeData[1][RangeOffset + SampleFrame2], FrameAttributeData[2][RangeOffset + SampleFrame2]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame3], FrameAttributeData[1][RangeOffset + SampleFrame3], FrameAttributeData[2][RangeOffset + SampleFrame3]),
			SampleAlpha);
	}

	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleDirection(FVector& OutDirection, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate)
{
	FVector3f OutDirectionFloat = FVector3f::ForwardVector;
	if (!FrameAttributeSampleDirectionFloat(OutDirectionFloat, FrameAttribute, SequenceIdx, SequenceTime, FrameRate))
	{
		OutDirection = FVector::ZeroVector;
		return false;
	}

	OutDirection = ((FVector)OutDirectionFloat).GetUnsafeNormal();
	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleDirectionFloat(FVector3f& OutDirection, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate)
{
	if (!FrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeSampleDirectionFloat: Invalid FrameAttribute.");
		OutDirection = FVector3f::ForwardVector;
		return false;
	}

	if (FrameAttribute.Type != EAnimDatabaseAttributeType::Direction)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeSampleDirectionFloat: FrameAttribute Types don't match. Got {Type}. Expected Direction.", AttributeTypeNameInternal(FrameAttribute.Type));
		OutDirection = FVector3f::ForwardVector;
		return false;
	}

	int32 EntryIdx = INDEX_NONE;
	int32 RangeIdx = INDEX_NONE;
	float RangeTime = 1.0f;
	if (!FrameAttribute.FrameAttribute->FrameRangeSet.FindTime(EntryIdx, RangeIdx, RangeTime, SequenceIdx, SequenceTime, 1.0f / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER)))
	{
		OutDirection = FVector3f::ForwardVector;
		return false;
	}

	const int32 RangeOffset = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeOffset(EntryIdx, RangeIdx);
	const int32 RangeLength = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeLength(EntryIdx, RangeIdx);
	const float RangeFrameTime = 1.0f / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER);

	int64 SampleFrame0, SampleFrame1, SampleFrame2, SampleFrame3;
	float SampleAlpha;
	UE::AnimDatabase::Math::ComputeCubicSampleFramesAndAlpha(
		SampleFrame0,
		SampleFrame1,
		SampleFrame2,
		SampleFrame3,
		SampleAlpha,
		RangeTime /	RangeFrameTime,
		RangeLength);

	const TLearningArrayView<2, const float> FrameAttributeData = FrameAttribute.FrameAttribute->AttributeData;

	if (SampleFrame0 == SampleFrame1 && SampleFrame2 == SampleFrame3)
	{
		UE::AnimDatabase::Math::DirectionInterpolateLinear(
			OutDirection,
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame1], FrameAttributeData[1][RangeOffset + SampleFrame1], FrameAttributeData[2][RangeOffset + SampleFrame1]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame2], FrameAttributeData[1][RangeOffset + SampleFrame2], FrameAttributeData[2][RangeOffset + SampleFrame2]),
			SampleAlpha);
	}
	else if (SampleFrame0 == SampleFrame1)
	{
		UE::AnimDatabase::Math::DirectionInterpolateCubicMonoStart(
			OutDirection,
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame1], FrameAttributeData[1][RangeOffset + SampleFrame1], FrameAttributeData[2][RangeOffset + SampleFrame1]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame2], FrameAttributeData[1][RangeOffset + SampleFrame2], FrameAttributeData[2][RangeOffset + SampleFrame2]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame3], FrameAttributeData[1][RangeOffset + SampleFrame3], FrameAttributeData[2][RangeOffset + SampleFrame3]),
			SampleAlpha);
	}
	else if (SampleFrame2 == SampleFrame3)
	{
		UE::AnimDatabase::Math::DirectionInterpolateCubicMonoEnd(
			OutDirection,
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame0], FrameAttributeData[1][RangeOffset + SampleFrame0], FrameAttributeData[2][RangeOffset + SampleFrame0]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame1], FrameAttributeData[1][RangeOffset + SampleFrame1], FrameAttributeData[2][RangeOffset + SampleFrame1]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame2], FrameAttributeData[1][RangeOffset + SampleFrame2], FrameAttributeData[2][RangeOffset + SampleFrame2]),
			SampleAlpha);
	}
	else
	{
		UE::AnimDatabase::Math::DirectionInterpolateCubicMono(
			OutDirection,
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame0], FrameAttributeData[1][RangeOffset + SampleFrame0], FrameAttributeData[2][RangeOffset + SampleFrame0]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame1], FrameAttributeData[1][RangeOffset + SampleFrame1], FrameAttributeData[2][RangeOffset + SampleFrame1]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame2], FrameAttributeData[1][RangeOffset + SampleFrame2], FrameAttributeData[2][RangeOffset + SampleFrame2]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame3], FrameAttributeData[1][RangeOffset + SampleFrame3], FrameAttributeData[2][RangeOffset + SampleFrame3]),
			SampleAlpha);
	}

	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleTransform(FTransform& OutTransform, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate)
{
	FTransform3f OutTransformFloat = FTransform3f::Identity;
	if (!FrameAttributeSampleTransformFloat(OutTransformFloat, FrameAttribute, SequenceIdx, SequenceTime, FrameRate))
	{
		OutTransform = FTransform::Identity;
		return false;
	}

	OutTransform = FTransform(
		((FQuat)OutTransformFloat.GetRotation()).GetNormalized(),
		(FVector)OutTransformFloat.GetLocation(),
		(FVector)OutTransformFloat.GetScale3D());
	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleTransformFloat(FTransform3f& OutTransform, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate)
{
	if (!FrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeSampleTransformFloat: Invalid FrameAttribute.");
		OutTransform = FTransform3f::Identity;
		return false;
	}

	if (FrameAttribute.Type != EAnimDatabaseAttributeType::Transform)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeSampleTransformFloat: FrameAttribute Types don't match. Got {Type}. Expected Transform.", AttributeTypeNameInternal(FrameAttribute.Type));
		OutTransform = FTransform3f::Identity;
		return false;
	}

	int32 EntryIdx = INDEX_NONE;
	int32 RangeIdx = INDEX_NONE;
	float RangeTime = 1.0f;
	if (!FrameAttribute.FrameAttribute->FrameRangeSet.FindTime(EntryIdx, RangeIdx, RangeTime, SequenceIdx, SequenceTime, 1.0f / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER)))
	{
		OutTransform = FTransform3f::Identity;
		return false;
	}

	const int32 RangeOffset = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeOffset(EntryIdx, RangeIdx);
	const int32 RangeLength = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeLength(EntryIdx, RangeIdx);
	const float RangeFrameTime = 1.0f / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER);

	int64 SampleFrame0, SampleFrame1, SampleFrame2, SampleFrame3;
	float SampleAlpha;
	UE::AnimDatabase::Math::ComputeCubicSampleFramesAndAlpha(
		SampleFrame0,
		SampleFrame1,
		SampleFrame2,
		SampleFrame3,
		SampleAlpha,
		RangeTime /	RangeFrameTime,
		RangeLength);

	const TLearningArrayView<2, const float> FrameAttributeData = FrameAttribute.FrameAttribute->AttributeData;

	if (SampleFrame0 == SampleFrame1 && SampleFrame2 == SampleFrame3)
	{
		const FTransform3f Sample1 = FTransform3f(
			FQuat4f(FrameAttributeData[3][RangeOffset + SampleFrame1], FrameAttributeData[4][RangeOffset + SampleFrame1], FrameAttributeData[5][RangeOffset + SampleFrame1], FrameAttributeData[6][RangeOffset + SampleFrame1]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame1], FrameAttributeData[1][RangeOffset + SampleFrame1], FrameAttributeData[2][RangeOffset + SampleFrame1]),
			FVector3f(FrameAttributeData[7][RangeOffset + SampleFrame1], FrameAttributeData[8][RangeOffset + SampleFrame1], FrameAttributeData[9][RangeOffset + SampleFrame1]));

		const FTransform3f Sample2 = FTransform3f(
			FQuat4f(FrameAttributeData[3][RangeOffset + SampleFrame2], FrameAttributeData[4][RangeOffset + SampleFrame2], FrameAttributeData[5][RangeOffset + SampleFrame2], FrameAttributeData[6][RangeOffset + SampleFrame2]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame2], FrameAttributeData[1][RangeOffset + SampleFrame2], FrameAttributeData[2][RangeOffset + SampleFrame2]),
			FVector3f(FrameAttributeData[7][RangeOffset + SampleFrame2], FrameAttributeData[8][RangeOffset + SampleFrame2], FrameAttributeData[9][RangeOffset + SampleFrame2]));

		UE::AnimDatabase::Math::TransformInterpolateLinear(
			OutTransform,
			Sample1,
			Sample2,
			SampleAlpha);
	}
	else if (SampleFrame0 == SampleFrame1)
	{
		const FTransform3f Sample1 = FTransform3f(
			FQuat4f(FrameAttributeData[3][RangeOffset + SampleFrame1], FrameAttributeData[4][RangeOffset + SampleFrame1], FrameAttributeData[5][RangeOffset + SampleFrame1], FrameAttributeData[6][RangeOffset + SampleFrame1]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame1], FrameAttributeData[1][RangeOffset + SampleFrame1], FrameAttributeData[2][RangeOffset + SampleFrame1]),
			FVector3f(FrameAttributeData[7][RangeOffset + SampleFrame1], FrameAttributeData[8][RangeOffset + SampleFrame1], FrameAttributeData[9][RangeOffset + SampleFrame1]));

		const FTransform3f Sample2 = FTransform3f(
			FQuat4f(FrameAttributeData[3][RangeOffset + SampleFrame2], FrameAttributeData[4][RangeOffset + SampleFrame2], FrameAttributeData[5][RangeOffset + SampleFrame2], FrameAttributeData[6][RangeOffset + SampleFrame2]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame2], FrameAttributeData[1][RangeOffset + SampleFrame2], FrameAttributeData[2][RangeOffset + SampleFrame2]),
			FVector3f(FrameAttributeData[7][RangeOffset + SampleFrame2], FrameAttributeData[8][RangeOffset + SampleFrame2], FrameAttributeData[9][RangeOffset + SampleFrame2]));

		const FTransform3f Sample3 = FTransform3f(
			FQuat4f(FrameAttributeData[3][RangeOffset + SampleFrame3], FrameAttributeData[4][RangeOffset + SampleFrame3], FrameAttributeData[5][RangeOffset + SampleFrame3], FrameAttributeData[6][RangeOffset + SampleFrame3]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame3], FrameAttributeData[1][RangeOffset + SampleFrame3], FrameAttributeData[2][RangeOffset + SampleFrame3]),
			FVector3f(FrameAttributeData[7][RangeOffset + SampleFrame3], FrameAttributeData[8][RangeOffset + SampleFrame3], FrameAttributeData[9][RangeOffset + SampleFrame3]));

		UE::AnimDatabase::Math::TransformInterpolateCubicMonoStart(
			OutTransform,
			Sample1,
			Sample2,
			Sample3,
			SampleAlpha);
	}
	else if (SampleFrame2 == SampleFrame3)
	{
		const FTransform3f Sample0 = FTransform3f(
			FQuat4f(FrameAttributeData[3][RangeOffset + SampleFrame0], FrameAttributeData[4][RangeOffset + SampleFrame0], FrameAttributeData[5][RangeOffset + SampleFrame0], FrameAttributeData[6][RangeOffset + SampleFrame0]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame0], FrameAttributeData[1][RangeOffset + SampleFrame0], FrameAttributeData[2][RangeOffset + SampleFrame0]),
			FVector3f(FrameAttributeData[7][RangeOffset + SampleFrame0], FrameAttributeData[8][RangeOffset + SampleFrame0], FrameAttributeData[9][RangeOffset + SampleFrame0]));

		const FTransform3f Sample1 = FTransform3f(
			FQuat4f(FrameAttributeData[3][RangeOffset + SampleFrame1], FrameAttributeData[4][RangeOffset + SampleFrame1], FrameAttributeData[5][RangeOffset + SampleFrame1], FrameAttributeData[6][RangeOffset + SampleFrame1]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame1], FrameAttributeData[1][RangeOffset + SampleFrame1], FrameAttributeData[2][RangeOffset + SampleFrame1]),
			FVector3f(FrameAttributeData[7][RangeOffset + SampleFrame1], FrameAttributeData[8][RangeOffset + SampleFrame1], FrameAttributeData[9][RangeOffset + SampleFrame1]));

		const FTransform3f Sample2 = FTransform3f(
			FQuat4f(FrameAttributeData[3][RangeOffset + SampleFrame2], FrameAttributeData[4][RangeOffset + SampleFrame2], FrameAttributeData[5][RangeOffset + SampleFrame2], FrameAttributeData[6][RangeOffset + SampleFrame2]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame2], FrameAttributeData[1][RangeOffset + SampleFrame2], FrameAttributeData[2][RangeOffset + SampleFrame2]),
			FVector3f(FrameAttributeData[7][RangeOffset + SampleFrame2], FrameAttributeData[8][RangeOffset + SampleFrame2], FrameAttributeData[9][RangeOffset + SampleFrame2]));

		UE::AnimDatabase::Math::TransformInterpolateCubicMonoEnd(
			OutTransform,
			Sample0,
			Sample1,
			Sample2,
			SampleAlpha);
	}
	else
	{
		const FTransform3f Sample0 = FTransform3f(
			FQuat4f(FrameAttributeData[3][RangeOffset + SampleFrame0], FrameAttributeData[4][RangeOffset + SampleFrame0], FrameAttributeData[5][RangeOffset + SampleFrame0], FrameAttributeData[6][RangeOffset + SampleFrame0]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame0], FrameAttributeData[1][RangeOffset + SampleFrame0], FrameAttributeData[2][RangeOffset + SampleFrame0]),
			FVector3f(FrameAttributeData[7][RangeOffset + SampleFrame0], FrameAttributeData[8][RangeOffset + SampleFrame0], FrameAttributeData[9][RangeOffset + SampleFrame0]));

		const FTransform3f Sample1 = FTransform3f(
			FQuat4f(FrameAttributeData[3][RangeOffset + SampleFrame1], FrameAttributeData[4][RangeOffset + SampleFrame1], FrameAttributeData[5][RangeOffset + SampleFrame1], FrameAttributeData[6][RangeOffset + SampleFrame1]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame1], FrameAttributeData[1][RangeOffset + SampleFrame1], FrameAttributeData[2][RangeOffset + SampleFrame1]),
			FVector3f(FrameAttributeData[7][RangeOffset + SampleFrame1], FrameAttributeData[8][RangeOffset + SampleFrame1], FrameAttributeData[9][RangeOffset + SampleFrame1]));

		const FTransform3f Sample2 = FTransform3f(
			FQuat4f(FrameAttributeData[3][RangeOffset + SampleFrame2], FrameAttributeData[4][RangeOffset + SampleFrame2], FrameAttributeData[5][RangeOffset + SampleFrame2], FrameAttributeData[6][RangeOffset + SampleFrame2]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame2], FrameAttributeData[1][RangeOffset + SampleFrame2], FrameAttributeData[2][RangeOffset + SampleFrame2]),
			FVector3f(FrameAttributeData[7][RangeOffset + SampleFrame2], FrameAttributeData[8][RangeOffset + SampleFrame2], FrameAttributeData[9][RangeOffset + SampleFrame2]));

		const FTransform3f Sample3 = FTransform3f(
			FQuat4f(FrameAttributeData[3][RangeOffset + SampleFrame3], FrameAttributeData[4][RangeOffset + SampleFrame3], FrameAttributeData[5][RangeOffset + SampleFrame3], FrameAttributeData[6][RangeOffset + SampleFrame3]),
			FVector3f(FrameAttributeData[0][RangeOffset + SampleFrame3], FrameAttributeData[1][RangeOffset + SampleFrame3], FrameAttributeData[2][RangeOffset + SampleFrame3]),
			FVector3f(FrameAttributeData[7][RangeOffset + SampleFrame3], FrameAttributeData[8][RangeOffset + SampleFrame3], FrameAttributeData[9][RangeOffset + SampleFrame3]));

		UE::AnimDatabase::Math::TransformInterpolateCubicMono(
			OutTransform,
			Sample0,
			Sample1,
			Sample2,
			Sample3,
			SampleAlpha);
	}

	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleEvent(bool& OutTimeUntilEventKnown, float& OutTimeUntilEvent, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate)
{
	if (!FrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeSampleEvent: Invalid FrameAttribute.");
		OutTimeUntilEventKnown = false;
		OutTimeUntilEvent = UE_MAX_FLT;
		return false;
	}

	if (FrameAttribute.Type != EAnimDatabaseAttributeType::Event)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeSampleEvent: FrameAttribute Types don't match. Got {Type}. Expected Event.", AttributeTypeNameInternal(FrameAttribute.Type));
		OutTimeUntilEventKnown = false;
		OutTimeUntilEvent = UE_MAX_FLT;
		return false;
	}

	int32 EntryIdx = INDEX_NONE;
	int32 RangeIdx = INDEX_NONE;
	float RangeTime = 1.0f;
	if (!FrameAttribute.FrameAttribute->FrameRangeSet.FindTime(EntryIdx, RangeIdx, RangeTime, SequenceIdx, SequenceTime, 1.0f / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER)))
	{
		OutTimeUntilEventKnown = false;
		OutTimeUntilEvent = UE_MAX_FLT;
		return false;
	}

	const int32 RangeOffset = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeOffset(EntryIdx, RangeIdx);
	const int32 RangeLength = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeLength(EntryIdx, RangeIdx);
	const float RangeFrameTime = 1.0f / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER);

	int64 SampleFrame1, SampleFrame2;
	float SampleAlpha;
	UE::AnimDatabase::Math::ComputeLinearSampleFramesAndAlpha(
		SampleFrame1,
		SampleFrame2,
		SampleAlpha,
		RangeTime /	RangeFrameTime,
		RangeLength);

	const TLearningArrayView<2, const float> FrameAttributeData = FrameAttribute.FrameAttribute->AttributeData;

	if (FrameAttributeData[0][RangeOffset + SampleFrame1] != 0.0f &&
		FrameAttributeData[0][RangeOffset + SampleFrame2] != 0.0f)
	{
		OutTimeUntilEventKnown = true;

		UE::AnimDatabase::Math::ValueInterpolateLinear(
			OutTimeUntilEvent,
			FrameAttributeData[1][RangeOffset + SampleFrame1],
			FrameAttributeData[1][RangeOffset + SampleFrame2],
			SampleAlpha);
	}
	else
	{
		OutTimeUntilEventKnown = false;
		OutTimeUntilEvent = UE_MAX_FLT;
	}

	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleAngleDegrees(float& OutValue, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate)
{
	if (!FrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeSampleAngleDegrees: Invalid FrameAttribute.");
		OutValue = 0.0f;
		return false;
	}

	if (FrameAttribute.Type != EAnimDatabaseAttributeType::Angle)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeSampleAngleDegrees: FrameAttribute Types don't match. Got {Type}. Expected Angle.", AttributeTypeNameInternal(FrameAttribute.Type));
		OutValue = 0.0f;
		return false;
	}

	int32 EntryIdx = INDEX_NONE;
	int32 RangeIdx = INDEX_NONE;
	float RangeTime = 1.0f;
	if (!FrameAttribute.FrameAttribute->FrameRangeSet.FindTime(EntryIdx, RangeIdx, RangeTime, SequenceIdx, SequenceTime, 1.0f / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER)))
	{
		OutValue = 0.0f;
		return false;
	}

	const int32 RangeOffset = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeOffset(EntryIdx, RangeIdx);
	const int32 RangeLength = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeLength(EntryIdx, RangeIdx);
	const float RangeFrameTime = 1.0f / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER);

	int64 SampleFrame0, SampleFrame1, SampleFrame2, SampleFrame3;
	float SampleAlpha;
	UE::AnimDatabase::Math::ComputeCubicSampleFramesAndAlpha(
		SampleFrame0,
		SampleFrame1,
		SampleFrame2,
		SampleFrame3,
		SampleAlpha,
		RangeTime / RangeFrameTime,
		RangeLength);

	if (SampleFrame0 == SampleFrame1 && SampleFrame2 == SampleFrame3)
	{
		UE::AnimDatabase::Math::AngleInterpolateLinear(
			OutValue,
			FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset + SampleFrame1],
			FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset + SampleFrame2],
			SampleAlpha);
		OutValue = FMath::RadiansToDegrees(OutValue);
	}
	else if (SampleFrame0 == SampleFrame1)
	{
		UE::AnimDatabase::Math::AngleInterpolateCubicMonoStart(
			OutValue,
			FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset + SampleFrame1],
			FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset + SampleFrame2],
			FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset + SampleFrame3],
			SampleAlpha);
		OutValue = FMath::RadiansToDegrees(OutValue);
	}
	else if (SampleFrame2 == SampleFrame3)
	{
		UE::AnimDatabase::Math::AngleInterpolateCubicMonoEnd(
			OutValue,
			FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset + SampleFrame0],
			FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset + SampleFrame1],
			FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset + SampleFrame2],
			SampleAlpha);
		OutValue = FMath::RadiansToDegrees(OutValue);
	}
	else
	{
		UE::AnimDatabase::Math::AngleInterpolateCubicMono(
			OutValue,
			FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset + SampleFrame0],
			FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset + SampleFrame1],
			FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset + SampleFrame2],
			FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset + SampleFrame3],
			SampleAlpha);
		OutValue = FMath::RadiansToDegrees(OutValue);
	}

	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleAngleRadians(float& OutValue, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate)
{
	if (!FrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeSampleAngleRadians: Invalid FrameAttribute.");
		OutValue = 0.0f;
		return false;
	}

	if (FrameAttribute.Type != EAnimDatabaseAttributeType::Angle)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeSampleAngleRadians: FrameAttribute Types don't match. Got {Type}. Expected Angle.", AttributeTypeNameInternal(FrameAttribute.Type));
		OutValue = 0.0f;
		return false;
	}

	int32 EntryIdx = INDEX_NONE;
	int32 RangeIdx = INDEX_NONE;
	float RangeTime = 1.0f;
	if (!FrameAttribute.FrameAttribute->FrameRangeSet.FindTime(EntryIdx, RangeIdx, RangeTime, SequenceIdx, SequenceTime, 1.0f / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER)))
	{
		OutValue = 0.0f;
		return false;
	}

	const int32 RangeOffset = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeOffset(EntryIdx, RangeIdx);
	const int32 RangeLength = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeLength(EntryIdx, RangeIdx);
	const float RangeFrameTime = 1.0f / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER);

	int64 SampleFrame0, SampleFrame1, SampleFrame2, SampleFrame3;
	float SampleAlpha;
	UE::AnimDatabase::Math::ComputeCubicSampleFramesAndAlpha(
		SampleFrame0,
		SampleFrame1,
		SampleFrame2,
		SampleFrame3,
		SampleAlpha,
		RangeTime / RangeFrameTime,
		RangeLength);

	if (SampleFrame0 == SampleFrame1 && SampleFrame2 == SampleFrame3)
	{
		UE::AnimDatabase::Math::AngleInterpolateLinear(
			OutValue,
			FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset + SampleFrame1],
			FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset + SampleFrame2],
			SampleAlpha);
	}
	else if (SampleFrame0 == SampleFrame1)
	{
		UE::AnimDatabase::Math::AngleInterpolateCubicMonoStart(
			OutValue,
			FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset + SampleFrame1],
			FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset + SampleFrame2],
			FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset + SampleFrame3],
			SampleAlpha);
	}
	else if (SampleFrame2 == SampleFrame3)
	{
		UE::AnimDatabase::Math::AngleInterpolateCubicMonoEnd(
			OutValue,
			FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset + SampleFrame0],
			FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset + SampleFrame1],
			FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset + SampleFrame2],
			SampleAlpha);
	}
	else
	{
		UE::AnimDatabase::Math::AngleInterpolateCubicMono(
			OutValue,
			FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset + SampleFrame0],
			FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset + SampleFrame1],
			FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset + SampleFrame2],
			FrameAttribute.FrameAttribute->AttributeData[0][RangeOffset + SampleFrame3],
			SampleAlpha);
	}

	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleLocations(TArray<FVector>& OutLocations, const TArray<FAnimDatabaseFrameAttribute>& FrameAttributes, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate)
{
	OutLocations.SetNum(FrameAttributes.Num());
	return FrameAttributeSampleLocationsToArrayView(OutLocations, FrameAttributes, SequenceIdx, SequenceTime, FrameRate);
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleLocationsToArrayView(const TArrayView<FVector> OutLocations, const TArrayView<const FAnimDatabaseFrameAttribute> FrameAttributes, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate)
{
	check(OutLocations.Num() == FrameAttributes.Num());
	for (int32 Idx = 0; Idx < FrameAttributes.Num(); Idx++)
	{
		if (!FrameAttributeSampleLocation(OutLocations[Idx], FrameAttributes[Idx], SequenceIdx, SequenceTime, FrameRate))
		{
			UE::Learning::Array::Set<1, FVector>(OutLocations, FVector::ZeroVector);
			return false;
		}
	}

	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleDirections(TArray<FVector>& OutDirections, const TArray<FAnimDatabaseFrameAttribute>& FrameAttributes, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate)
{
	OutDirections.SetNum(FrameAttributes.Num());
	return FrameAttributeSampleDirectionsToArrayView(OutDirections, FrameAttributes, SequenceIdx, SequenceTime, FrameRate);
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleDirectionsToArrayView(const TArrayView<FVector> OutDirections, const TArrayView<const FAnimDatabaseFrameAttribute> FrameAttributes, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate)
{
	check(OutDirections.Num() == FrameAttributes.Num());
	for (int32 Idx = 0; Idx < FrameAttributes.Num(); Idx++)
	{
		if (!FrameAttributeSampleDirection(OutDirections[Idx], FrameAttributes[Idx], SequenceIdx, SequenceTime, FrameRate))
		{
			UE::Learning::Array::Set<1, FVector>(OutDirections, FVector::ForwardVector);
			return false;
		}
	}

	return true;
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleLinearVelocities(TArray<FVector>& OutLinearVelocities, const TArray<FAnimDatabaseFrameAttribute>& FrameAttributes, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate)
{
	OutLinearVelocities.SetNum(FrameAttributes.Num());
	return FrameAttributeSampleLinearVelocitiesToArrayView(OutLinearVelocities, FrameAttributes, SequenceIdx, SequenceTime, FrameRate);
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeSampleLinearVelocitiesToArrayView(const TArrayView<FVector> OutLinearVelocities, const TArrayView<const FAnimDatabaseFrameAttribute> FrameAttributes, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate)
{
	check(OutLinearVelocities.Num() == FrameAttributes.Num());
	for (int32 Idx = 0; Idx < FrameAttributes.Num(); Idx++)
	{
		if (!FrameAttributeSampleLinearVelocity(OutLinearVelocities[Idx], FrameAttributes[Idx], SequenceIdx, SequenceTime, FrameRate))
		{
			UE::Learning::Array::Set<1, FVector>(OutLinearVelocities, FVector::ZeroVector);
			return false;
		}
	}

	return true;
}

int32 UAnimDatabaseFrameAttributeLibrary::FrameAttributeTotalRangeNum(const FAnimDatabaseFrameAttribute& FrameAttribute)
{
	return UAnimDatabaseFrameRangesLibrary::FrameRangesTotalRangeNum(FrameAttributeFrameRanges(FrameAttribute));
}

int32 UAnimDatabaseFrameAttributeLibrary::FrameAttributeTotalFrameNum(const FAnimDatabaseFrameAttribute& FrameAttribute)
{
	return UAnimDatabaseFrameRangesLibrary::FrameRangesTotalFrameNum(FrameAttributeFrameRanges(FrameAttribute));
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributeToBoolArray(TArray<bool>& OutArray, const FAnimDatabaseFrameAttribute& FrameAttribute)
{
	OutArray.SetNumZeroed(FrameAttributeTotalFrameNum(FrameAttribute));
	FrameAttributeToBoolArrayView(OutArray, FrameAttribute);
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributeToBoolArrayView(const TArrayView<bool> OutArray, const FAnimDatabaseFrameAttribute& FrameAttribute)
{
	if (!FrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeToBoolArrayView: Invalid FrameAttribute.");
		return;
	}

	if (FrameAttribute.Type != EAnimDatabaseAttributeType::Bool)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeToBoolArrayView: Incorrect FrameAttribute Type. Expected Bool, got {Type}.", AttributeTypeNameInternal(FrameAttribute.Type));
		return;
	}

	const int32 FrameNum = FrameAttributeTotalFrameNum(FrameAttribute);
	check(OutArray.Num() == FrameNum);

	for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
	{
		OutArray[FrameIdx] = FrameAttribute.GetAsBool(FrameIdx);
	}
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributeToFloatArray(TArray<float>& OutArray, const FAnimDatabaseFrameAttribute& FrameAttribute)
{
	OutArray.SetNumZeroed(FrameAttributeTotalFrameNum(FrameAttribute));
	FrameAttributeToFloatArrayView(OutArray, FrameAttribute);
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributeToFloatArrayView(const TArrayView<float> OutArray, const FAnimDatabaseFrameAttribute& FrameAttribute)
{
	if (!FrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeToFloatArrayView: Invalid FrameAttribute.");
		return;
	}

	if (FrameAttribute.Type != EAnimDatabaseAttributeType::Float)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeToFloatArrayView: Incorrect FrameAttribute Type. Expected Float, got {Type}.", AttributeTypeNameInternal(FrameAttribute.Type));
		return;
	}

	const int32 FrameNum = FrameAttributeTotalFrameNum(FrameAttribute);
	check(OutArray.Num() == FrameNum);

	for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
	{
		OutArray[FrameIdx] = FrameAttribute.GetAsFloat(FrameIdx);
	}
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributeToVectorArray(TArray<FVector>& OutArray, const FAnimDatabaseFrameAttribute& FrameAttribute)
{
	OutArray.SetNumZeroed(FrameAttributeTotalFrameNum(FrameAttribute));
	FrameAttributeToVectorArrayView(OutArray, FrameAttribute);
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributeToVectorArrayView(TArray<FVector>& OutArray, const FAnimDatabaseFrameAttribute& FrameAttribute)
{
	if (!FrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeToVectorArrayView: Invalid FrameAttribute.");
		return;
	}

	if (FrameAttribute.Type != EAnimDatabaseAttributeType::Location &&
		FrameAttribute.Type != EAnimDatabaseAttributeType::Direction &&
		FrameAttribute.Type != EAnimDatabaseAttributeType::Scale &&
		FrameAttribute.Type != EAnimDatabaseAttributeType::LinearVelocity &&
		FrameAttribute.Type != EAnimDatabaseAttributeType::AngularVelocity &&
		FrameAttribute.Type != EAnimDatabaseAttributeType::ScalarVelocity)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeToVectorArrayView: Incorrect FrameAttribute Type. Expected Location, Direction, Scale or Velocity, got {Type}.", AttributeTypeNameInternal(FrameAttribute.Type));
		return;
	}

	const int32 FrameNum = FrameAttributeTotalFrameNum(FrameAttribute);
	check(OutArray.Num() == FrameNum);

	switch (FrameAttribute.Type)
	{
	case EAnimDatabaseAttributeType::Location:
	{
		for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
		{
			OutArray[FrameIdx] = FrameAttribute.GetAsLocationDouble(FrameIdx);
		}
	}
	break;

	case EAnimDatabaseAttributeType::Direction:
	{
		for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
		{
			OutArray[FrameIdx] = FrameAttribute.GetAsDirectionDouble(FrameIdx);
		}
	}
	break;

	case EAnimDatabaseAttributeType::Scale:
	{
		for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
		{
			OutArray[FrameIdx] = FrameAttribute.GetAsScaleDouble(FrameIdx);
		}
	}
	break;

	case EAnimDatabaseAttributeType::LinearVelocity:
	case EAnimDatabaseAttributeType::AngularVelocity:
	case EAnimDatabaseAttributeType::ScalarVelocity:
	{
		for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
		{
			OutArray[FrameIdx] = FrameAttribute.GetAsVelocityDouble(FrameIdx);
		}
	}
	break;

	default:
	{
		checkNoEntry();
	}
	}

}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributeToQuatArray(TArray<FQuat>& OutArray, const FAnimDatabaseFrameAttribute& FrameAttribute)
{
	OutArray.SetNumZeroed(FrameAttributeTotalFrameNum(FrameAttribute));
	FrameAttributeToQuatArrayView(OutArray, FrameAttribute);
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributeToQuatArrayView(TArray<FQuat>& OutArray, const FAnimDatabaseFrameAttribute& FrameAttribute)
{
	if (!FrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeToQuatArrayView: Invalid FrameAttribute.");
		return;
	}

	if (FrameAttribute.Type != EAnimDatabaseAttributeType::Rotation)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeToQuatArray: Incorrect FrameAttribute Type. Expected Rotation, got {Type}.", AttributeTypeNameInternal(FrameAttribute.Type));
		return;
	}

	const int32 FrameNum = FrameAttributeTotalFrameNum(FrameAttribute);
	check(OutArray.Num() == FrameNum);

	for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
	{
		OutArray[FrameIdx] = FrameAttribute.GetAsRotationDouble(FrameIdx);
	}
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributeToRotatorArray(TArray<FRotator>& OutArray, const FAnimDatabaseFrameAttribute& FrameAttribute)
{
	OutArray.SetNumZeroed(FrameAttributeTotalFrameNum(FrameAttribute));
	FrameAttributeToRotatorArrayView(OutArray, FrameAttribute);
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributeToRotatorArrayView(TArray<FRotator>& OutArray, const FAnimDatabaseFrameAttribute& FrameAttribute)
{
	if (!FrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeToRotatorArrayView: Invalid FrameAttribute.");
		return;
	}

	if (FrameAttribute.Type != EAnimDatabaseAttributeType::Rotation)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeToRotatorArrayView: Incorrect FrameAttribute Type. Expected Rotation, got {Type}.", AttributeTypeNameInternal(FrameAttribute.Type));
		return;
	}

	const int32 FrameNum = FrameAttributeTotalFrameNum(FrameAttribute);
	check(OutArray.Num() == FrameNum);

	for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
	{
		OutArray[FrameIdx] = FrameAttribute.GetAsRotationDouble(FrameIdx).Rotator();
	}
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributeToTransformArray(TArray<FTransform>& OutArray, const FAnimDatabaseFrameAttribute& FrameAttribute)
{
	OutArray.SetNumZeroed(FrameAttributeTotalFrameNum(FrameAttribute));
	FrameAttributeToTransformArrayView(OutArray, FrameAttribute);
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributeToTransformArrayView(TArray<FTransform>& OutArray, const FAnimDatabaseFrameAttribute& FrameAttribute)
{
	if (!FrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeToTransformArrayView: Invalid FrameAttribute.");
		return;
	}

	if (FrameAttribute.Type != EAnimDatabaseAttributeType::Transform)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeToTransformArrayView: Incorrect FrameAttribute Type. Expected Transform, got {Type}.", AttributeTypeNameInternal(FrameAttribute.Type));
		return;
	}

	const int32 FrameNum = FrameAttributeTotalFrameNum(FrameAttribute);
	check(OutArray.Num() == FrameNum);

	for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
	{
		OutArray[FrameIdx] = FrameAttribute.GetAsTransformDouble(FrameIdx);
	}
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributeToAngleArrayDegrees(TArray<float>& OutArray, const FAnimDatabaseFrameAttribute& FrameAttribute)
{
	OutArray.SetNumZeroed(FrameAttributeTotalFrameNum(FrameAttribute));
	FrameAttributeToAngleArrayViewDegrees(OutArray, FrameAttribute);
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributeToAngleArrayViewDegrees(const TArrayView<float> OutArray, const FAnimDatabaseFrameAttribute& FrameAttribute)
{
	if (!FrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeToAngleArrayViewDegrees: Invalid FrameAttribute.");
		return;
	}

	if (FrameAttribute.Type != EAnimDatabaseAttributeType::Angle)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeToAngleArrayViewDegrees: Incorrect FrameAttribute Type. Expected Angle, got {Type}.", AttributeTypeNameInternal(FrameAttribute.Type));
		return;
	}

	const int32 FrameNum = FrameAttributeTotalFrameNum(FrameAttribute);
	check(OutArray.Num() == FrameNum);

	for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
	{
		OutArray[FrameIdx] = FrameAttribute.GetAsAngleDegrees(FrameIdx);
	}
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributeToAngleArrayRadians(TArray<float>& OutArray, const FAnimDatabaseFrameAttribute& FrameAttribute)
{
	OutArray.SetNumZeroed(FrameAttributeTotalFrameNum(FrameAttribute));
	FrameAttributeToAngleArrayViewRadians(OutArray, FrameAttribute);
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributeToAngleArrayViewRadians(const TArrayView<float> OutArray, const FAnimDatabaseFrameAttribute& FrameAttribute)
{
	if (!FrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeToAngleArrayViewRadians: Invalid FrameAttribute.");
		return;
	}

	if (FrameAttribute.Type != EAnimDatabaseAttributeType::Angle)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeToAngleArrayViewRadians: Incorrect FrameAttribute Type. Expected Angle, got {Type}.", AttributeTypeNameInternal(FrameAttribute.Type));
		return;
	}

	const int32 FrameNum = FrameAttributeTotalFrameNum(FrameAttribute);
	check(OutArray.Num() == FrameNum);

	for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
	{
		OutArray[FrameIdx] = FrameAttribute.GetAsAngleRadians(FrameIdx);
	}
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeLocationMatchingDistance(
	const FAnimDatabaseFrameAttribute& LocationFrameAttribute,
	const FVector Location,
	const float Scale,
	const float Weight)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeLocationMatchingDistance);

	if (!LocationFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeLocationMatchingDistance: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (LocationFrameAttribute.Type != EAnimDatabaseAttributeType::Location)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeLocationMatchingDistance: Incorrect FrameAttribute Type. Expected Location, got {Type}.", AttributeTypeNameInternal(LocationFrameAttribute.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Float;

	UE::Learning::FrameAttribute::ConstantDist(
		*Out.FrameAttribute,
		{ (float)Location.X, (float)Location.Y, (float)Location.Z },
		*LocationFrameAttribute.FrameAttribute);

	UE::Learning::FrameAttribute::MulConstantInplace(*Out.FrameAttribute, { Weight / FMath::Max(Scale, UE_SMALL_NUMBER) });

	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeDirectionMatchingDistance(
	const FAnimDatabaseFrameAttribute& DirectionFrameAttribute,
	const FVector Direction,
	const float Weight)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeDirectionMatchingDistance);

	if (!DirectionFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeDirectionMatchingDistance: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (DirectionFrameAttribute.Type != EAnimDatabaseAttributeType::Direction)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeDirectionMatchingDistance: Incorrect FrameAttribute Type. Expected Direction, got {Type}.", AttributeTypeNameInternal(DirectionFrameAttribute.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Float;

	UE::Learning::FrameAttribute::ConstantDot(
		*Out.FrameAttribute,
		{ (float)Direction.X, (float)Direction.Y, (float)Direction.Z },
		* DirectionFrameAttribute.FrameAttribute);

	UE::Learning::FrameAttribute::ApproxAcosInplace(*Out.FrameAttribute);
	UE::Learning::FrameAttribute::MulConstantInplace(*Out.FrameAttribute, { Weight / UE_HALF_PI });

	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeRotationMatchingDistance(
	const FAnimDatabaseFrameAttribute& RotationFrameAttribute,
	const FQuat Rotation,
	const float Weight)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeRotationMatchingDistance);

	if (!RotationFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeRotationMatchingDistance: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (RotationFrameAttribute.Type != EAnimDatabaseAttributeType::Rotation)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeRotationMatchingDistance: Incorrect FrameAttribute Type. Expected Rotation, got {Type}.", AttributeTypeNameInternal(RotationFrameAttribute.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Float;

	UE::Learning::FrameAttribute::QuatApproxShortestAngleBetweenConstant(
		*Out.FrameAttribute,
		*RotationFrameAttribute.FrameAttribute,
		(FQuat4f)Rotation);

	UE::Learning::FrameAttribute::MulConstantInplace(*Out.FrameAttribute, { Weight / UE_HALF_PI });

	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeVelocityMatchingDistance(
	const FAnimDatabaseFrameAttribute& VelocityFrameAttribute,
	const FVector Velocity,
	const float Scale,
	const float Weight)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeVelocityMatchingDistance);

	if (!VelocityFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeVelocityMatchingDistance: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (VelocityFrameAttribute.Type != EAnimDatabaseAttributeType::LinearVelocity &&
		VelocityFrameAttribute.Type != EAnimDatabaseAttributeType::AngularVelocity &&
		VelocityFrameAttribute.Type != EAnimDatabaseAttributeType::ScalarVelocity)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeVelocityMatchingDistance: Incorrect FrameAttribute Type. Expected LinearVelocity, AngularVelocity, or ScalarVelocity, got {Type}.", AttributeTypeNameInternal(VelocityFrameAttribute.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Float;

	UE::Learning::FrameAttribute::ConstantDist(
		*Out.FrameAttribute,
		{ (float)Velocity.X, (float)Velocity.Y, (float)Velocity.Z },
		* VelocityFrameAttribute.FrameAttribute);

	UE::Learning::FrameAttribute::MulConstantInplace(*Out.FrameAttribute, { Weight / FMath::Max(Scale, UE_SMALL_NUMBER) });

	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeEventMatchingDistance(
	const FAnimDatabaseFrameAttribute& EventFrameAttribute,
	const bool bTimeUntilEventKnown,
	const float TimeUntilEvent,
	const float Weight)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeEventMatchingDistance);

	if (!EventFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeEventMatchingDistance: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (EventFrameAttribute.Type != EAnimDatabaseAttributeType::Event)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeEventMatchingDistance: Incorrect FrameAttribute Type. Expected Event, got {Type}.", AttributeTypeNameInternal(EventFrameAttribute.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Float;

	UE::Learning::FrameAttribute::UnaryOp(*Out.FrameAttribute, AttributeTypeSize(Out.Type), *EventFrameAttribute.FrameAttribute, [bTimeUntilEventKnown, TimeUntilEvent, Weight](
		UE::Learning::FFrameAttribute& Out,
		const UE::Learning::FFrameAttribute& In,
		const TLearningArrayView<1, const int32> RangeOffsets,
		const TLearningArrayView<1, const int32> RangeLengths) {

			UE::AnimDatabase::FrameAttribute::Private::EventDifference(
				Out.GetChannelAttributeData(0).GetData(),
				In.GetChannelAttributeData(0).GetData(),
				In.GetChannelAttributeData(1).GetData(),
				bTimeUntilEventKnown,
				TimeUntilEvent,
				In.GetTotalFrameNum(),
				Weight);

		});

	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeInertializationMatchingDistance(
	const FAnimDatabaseFrameAttribute& LocationFrameAttribute,
	const FAnimDatabaseFrameAttribute& VelocityFrameAttribute,
	const FVector Location,
	const FVector Velocity,
	const float BlendTime,
	const float LocationScale,
	const float VelocityScale,
	const float LocationWeight,
	const float VelocityWeight)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeInertializationMatchingDistance);

	if (!LocationFrameAttribute.IsValid() || !VelocityFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeInertializationMatchingDistance: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (LocationFrameAttribute.Type != EAnimDatabaseAttributeType::Location ||
		VelocityFrameAttribute.Type != EAnimDatabaseAttributeType::LinearVelocity)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeInertializationMatchingDistance: Incorrect FrameAttribute Type. Expected Location and LinearVelocity, got {Type0} and {Type1}.", AttributeTypeNameInternal(LocationFrameAttribute.Type), AttributeTypeNameInternal(VelocityFrameAttribute.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Tmp = MakeEmptyFrameAttribute();
	Tmp.Type = EAnimDatabaseAttributeType::Location;

	UE::Learning::FrameAttribute::InertializationDistanceConstant(
		*Tmp.FrameAttribute,
		*LocationFrameAttribute.FrameAttribute,
		*VelocityFrameAttribute.FrameAttribute,
		{ (float)Location.X, (float)Location.Y,(float)Location.Z },
		{ (float)Velocity.X, (float)Velocity.Y, (float)Velocity.Z },
		BlendTime,
		LocationWeight / FMath::Max(LocationScale, UE_SMALL_NUMBER),
		VelocityWeight / FMath::Max(VelocityScale, UE_SMALL_NUMBER));

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Float;
	UE::Learning::FrameAttribute::Length(*Out.FrameAttribute, *Tmp.FrameAttribute);

	return Out;
}

float UAnimDatabaseFrameAttributeLibrary::FrameAttributeAverageStd(const FAnimDatabaseFrameAttribute& FrameAttribute)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeAverageStd);

	if (!FrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeAverageStd: Invalid FrameAttribute.");
		return 0.0f;
	}

	if (FrameAttribute.Type != EAnimDatabaseAttributeType::Float &&
		FrameAttribute.Type != EAnimDatabaseAttributeType::Location &&
		FrameAttribute.Type != EAnimDatabaseAttributeType::LinearVelocity &&
		FrameAttribute.Type != EAnimDatabaseAttributeType::AngularVelocity &&
		FrameAttribute.Type != EAnimDatabaseAttributeType::ScalarVelocity)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeAverageStd: Incorrect FrameAttribute Type. Expected Float, Location, LinearVelocity, AngularVelocity, or ScalarVelocity, got {Type}.", AttributeTypeNameInternal(FrameAttribute.Type));
		return 0.0f;
	}

	const int32 DimNum = AttributeTypeSize(FrameAttribute.Type);

	TLearningArray<1, float, TInlineAllocator<3>> Mean;
	TLearningArray<1, float, TInlineAllocator<3>> Std;
	TLearningArray<1, int32, TInlineAllocator<3>> Count;
	Mean.SetNumZeroed({ DimNum });
	Std.SetNumZeroed({ DimNum });
	Count.SetNumZeroed({ DimNum });

	UE::Learning::FrameAttribute::ReduceOp(*FrameAttribute.FrameAttribute, [&Mean, &Std, &Count](
		const UE::Learning::FFrameAttribute& In,
		const TLearningArrayView<1, const int32> RangeOffsets,
		const TLearningArrayView<1, const int32> RangeLengths)
		{
			const int32 FrameNum = In.GetTotalFrameNum();
			const int32 ChannelNum = In.GetChannelNum();

			for (int32 ChannelIdx = 0; ChannelIdx < ChannelNum; ChannelIdx++)
			{
				for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
				{
					const float Value = In.GetChannelAttributeDataAtFrame(ChannelIdx, FrameIdx);
					Std[ChannelIdx] += (((float)Count[ChannelIdx] / FrameNum) / (Count[ChannelIdx] + 1)) * FMath::Square(Value - Mean[ChannelIdx]);
					Mean[ChannelIdx] += (Value - Mean[ChannelIdx]) / (Count[ChannelIdx] + 1);
					Count[ChannelIdx]++;
				}
			}
		});

	float AvgStd = 0.0f;
	for (int32 DimIdx = 0; DimIdx < DimNum; DimIdx++)
	{
		AvgStd += FMath::Sqrt(Std[DimIdx]) / DimNum;
	}

	return AvgStd;
}

float UAnimDatabaseFrameAttributeLibrary::FrameAttributesAverageStd(const TArray<FAnimDatabaseFrameAttribute>& FrameAttributes)
{
	return FrameAttributesAverageStdFromArrayView(FrameAttributes);
}

float UAnimDatabaseFrameAttributeLibrary::FrameAttributesAverageStdFromArrayView(const TArrayView<const FAnimDatabaseFrameAttribute> FrameAttributes)
{
	if (FrameAttributes.Num() == 0)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributesAverageStdFromArrayView: No FrameAttributes Given.");
		return 0.0f;
	}

	float AvgStd = 0.0f;
	for (const FAnimDatabaseFrameAttribute& FrameAttribute : FrameAttributes)
	{
		AvgStd += FrameAttributeAverageStd(FrameAttribute) / FrameAttributes.Num();
	}
	return AvgStd;
}

void UAnimDatabaseFrameAttributeLibrary::FindFrameAttributeMinimum(
	int32& OutSequenceIdx,
	int32& OutSequenceFrame,
	float& OutMinimumValue,
	const FAnimDatabaseFrameAttribute& FrameAttribute)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FindFrameAttributeMinimum);

	if (!FrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FindFrameAttributeMinimum: Invalid FrameAttribute.");
		OutSequenceIdx = INDEX_NONE;
		OutSequenceFrame = INDEX_NONE;
		OutMinimumValue = 0.0f;
		return;
	}

	if (FrameAttribute.Type != EAnimDatabaseAttributeType::Float)
	{
		OutSequenceIdx = INDEX_NONE;
		OutSequenceFrame = INDEX_NONE;
		OutMinimumValue = 0.0f;
		UE_LOGFMT(LogAnimDatabase, Error, "FindFrameAttributeMinimum: Incorrect FrameAttribute Type. Expected Float, got {Type}.", AttributeTypeNameInternal(FrameAttribute.Type));
		return;
	}

	int32 FrameAttributeIdx = INDEX_NONE;
	int32 FrameIdx = INDEX_NONE;
	if (UE::Learning::FrameAttribute::FindMinimum(
		FrameAttributeIdx,
		FrameIdx,
		OutMinimumValue,
		*FrameAttribute.FrameAttribute))
	{
		int32 EntryIdx = INDEX_NONE;
		int32 RangeIdx = INDEX_NONE;
		int32 RangeFrame = INDEX_NONE;
		const bool bFound = FrameAttribute.FrameAttribute->FrameRangeSet.FindOffset(EntryIdx, RangeIdx, RangeFrame, FrameIdx);
		check(bFound);
		OutSequenceIdx = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntrySequence(EntryIdx);
		OutSequenceFrame = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeStart(EntryIdx, RangeIdx) + RangeFrame;
	}
	else
	{
		OutSequenceIdx = INDEX_NONE;
		OutSequenceFrame = INDEX_NONE;
		OutMinimumValue = 0.0f;
		UE_LOGFMT(LogAnimDatabase, Error, "FindFrameAttributeMinimum: Empty FrameAttribute.");
		return;
	}
}

void UAnimDatabaseFrameAttributeLibrary::FindFrameAttributeMinimumSequence(
	UAnimSequence*& OutAnimSequence,
	float& OutAnimSequenceTime,
	bool& bOutAnimSequenceMirrored,
	float& OutMinimumValue,
	const FAnimDatabaseFrameAttribute& FrameAttribute,
	const UAnimDatabase* Database)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FindFrameAttributeMinimumSequence: Database is null.");
		OutAnimSequence = nullptr;
		OutAnimSequenceTime = -1.0f;
		bOutAnimSequenceMirrored = false;
		OutMinimumValue = 0.0f;
		return;
	}

	int32 MinSequenceIdx = INDEX_NONE;
	int32 MinSequenceFrame = INDEX_NONE;
	float MinSequenceValue = 0.0;
	FindFrameAttributeMinimum(MinSequenceIdx, MinSequenceFrame, MinSequenceValue, FrameAttribute);

	if (MinSequenceIdx != INDEX_NONE)
	{
		OutAnimSequence = Database->GetAnimSequence(MinSequenceIdx);
		OutAnimSequenceTime = MinSequenceFrame / Database->GetFrameRate().AsDecimal();
		bOutAnimSequenceMirrored = Database->GetIsMirrored(MinSequenceIdx);
		OutMinimumValue = MinSequenceValue;
	}
	else
	{
		OutAnimSequence = nullptr;
		OutAnimSequenceTime = -1.0f;
		bOutAnimSequenceMirrored = false;
		OutMinimumValue = 0.0f;
	}
}

void UAnimDatabaseFrameAttributeLibrary::FindFrameAttributeMaximum(
	int32& OutSequenceIdx,
	int32& OutSequenceFrame,
	float& OutMaximumValue,
	const FAnimDatabaseFrameAttribute& FrameAttribute)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FindFrameAttributeMaximum);

	if (!FrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FindFrameAttributeMaximum: Invalid FrameAttribute.");
		OutSequenceIdx = INDEX_NONE;
		OutSequenceFrame = INDEX_NONE;
		OutMaximumValue = 0.0f;
		return;
	}

	if (FrameAttribute.Type != EAnimDatabaseAttributeType::Float)
	{
		OutSequenceIdx = INDEX_NONE;
		OutSequenceFrame = INDEX_NONE;
		OutMaximumValue = 0.0f;
		UE_LOGFMT(LogAnimDatabase, Error, "FindFrameAttributeMaximum: Incorrect FrameAttribute Type. Expected Float, got {FrameAttribute}.", AttributeTypeNameInternal(FrameAttribute.Type));
		return;
	}

	int32 FrameAttributeIdx = INDEX_NONE;
	int32 FrameIdx = INDEX_NONE;
	if (UE::Learning::FrameAttribute::FindMaximum(
		FrameAttributeIdx,
		FrameIdx,
		OutMaximumValue,
		*FrameAttribute.FrameAttribute))
	{
		int32 EntryIdx = INDEX_NONE;
		int32 RangeIdx = INDEX_NONE;
		int32 RangeFrame = INDEX_NONE;
		const bool bFound = FrameAttribute.FrameAttribute->FrameRangeSet.FindOffset(EntryIdx, RangeIdx, RangeFrame, FrameIdx);
		check(bFound);
		OutSequenceIdx = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntrySequence(EntryIdx);
		OutSequenceFrame = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeStart(EntryIdx, RangeIdx) + RangeFrame;
	}
	else
	{
		OutSequenceIdx = INDEX_NONE;
		OutSequenceFrame = INDEX_NONE;
		OutMaximumValue = 0.0f;
		UE_LOGFMT(LogAnimDatabase, Error, "FindFrameAttributeMaximum: Empty FrameAttribute.");
		return;
	}
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeIntersection(const FAnimDatabaseFrameAttribute& InFrameAttribute, const FAnimDatabaseFrameRanges& InFrameRanges)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeIntersection);

	if (!InFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeIntersection: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (!InFrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeIntersection: Invalid FrameRanges.");
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = InFrameAttribute.Type;
	UE::Learning::FrameAttribute::Intersection(*Out.FrameAttribute, *InFrameAttribute.FrameAttribute, *InFrameRanges.FrameRangeSet);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeAdd(FAnimDatabaseFrameAttribute A, FAnimDatabaseFrameAttribute B)
{
	if (!A.IsValid() || !B.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeAdd: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (!(
		(A.Type == EAnimDatabaseAttributeType::Float && B.Type == EAnimDatabaseAttributeType::Float) ||
		(A.Type == EAnimDatabaseAttributeType::Bool && B.Type == EAnimDatabaseAttributeType::Float) ||
		(A.Type == EAnimDatabaseAttributeType::Float && B.Type == EAnimDatabaseAttributeType::Bool) ||
		(A.Type == EAnimDatabaseAttributeType::Location && B.Type == EAnimDatabaseAttributeType::Location) ||
		(A.Type == EAnimDatabaseAttributeType::Angle && B.Type == EAnimDatabaseAttributeType::Angle)))
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeAdd: Invalid FrameAttribute Types. Got {Type0} and {Type1}.", AttributeTypeNameInternal(A.Type), AttributeTypeNameInternal(B.Type));
		return FAnimDatabaseFrameAttribute();
	}

	EAnimDatabaseAttributeType OutputType = EAnimDatabaseAttributeType::Float;
	if (A.Type == EAnimDatabaseAttributeType::Location && B.Type == EAnimDatabaseAttributeType::Location) { OutputType = EAnimDatabaseAttributeType::Location; }
	if (A.Type == EAnimDatabaseAttributeType::Angle && B.Type == EAnimDatabaseAttributeType::Angle) { OutputType = EAnimDatabaseAttributeType::Angle; }

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = OutputType;
	UE::Learning::FrameAttribute::Add(*Out.FrameAttribute, *A.FrameAttribute, *B.FrameAttribute);
	if (Out.Type == EAnimDatabaseAttributeType::Angle) { UE::Learning::FrameAttribute::WrapAngleInplace(*Out.FrameAttribute); }
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeSubtract(FAnimDatabaseFrameAttribute A, FAnimDatabaseFrameAttribute B)
{
	if (!A.IsValid() || !B.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeSubtract: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (!(
		(A.Type == EAnimDatabaseAttributeType::Float && B.Type == EAnimDatabaseAttributeType::Float) ||
		(A.Type == EAnimDatabaseAttributeType::Bool && B.Type == EAnimDatabaseAttributeType::Float) ||
		(A.Type == EAnimDatabaseAttributeType::Float && B.Type == EAnimDatabaseAttributeType::Bool) ||
		(A.Type == EAnimDatabaseAttributeType::Location && B.Type == EAnimDatabaseAttributeType::Location) ||
		(A.Type == EAnimDatabaseAttributeType::Angle && B.Type == EAnimDatabaseAttributeType::Angle)))
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeSubtract: Invalid FrameAttribute Types. Got {Type0} and {Type1}.", AttributeTypeNameInternal(A.Type), AttributeTypeNameInternal(B.Type));
		return FAnimDatabaseFrameAttribute();
	}

	EAnimDatabaseAttributeType OutputType = EAnimDatabaseAttributeType::Float;
	if (A.Type == EAnimDatabaseAttributeType::Location && B.Type == EAnimDatabaseAttributeType::Location) { OutputType = EAnimDatabaseAttributeType::Location; }
	if (A.Type == EAnimDatabaseAttributeType::Angle && B.Type == EAnimDatabaseAttributeType::Angle) { OutputType = EAnimDatabaseAttributeType::Angle; }

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = OutputType;
	UE::Learning::FrameAttribute::Sub(*Out.FrameAttribute, *A.FrameAttribute, *B.FrameAttribute);
	if (Out.Type == EAnimDatabaseAttributeType::Angle) { UE::Learning::FrameAttribute::WrapAngleInplace(*Out.FrameAttribute); }
	return Out;
}


FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeMultiply(FAnimDatabaseFrameAttribute A, FAnimDatabaseFrameAttribute B)
{
	if (!A.IsValid() || !B.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeMultiply: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (!(
		(A.Type == EAnimDatabaseAttributeType::Float && B.Type == EAnimDatabaseAttributeType::Float) ||
		(A.Type == EAnimDatabaseAttributeType::Location && B.Type == EAnimDatabaseAttributeType::Location) ||
		(A.Type == EAnimDatabaseAttributeType::Location && B.Type == EAnimDatabaseAttributeType::Float) ||
		(A.Type == EAnimDatabaseAttributeType::Float && B.Type == EAnimDatabaseAttributeType::Location) ||
		(A.Type == EAnimDatabaseAttributeType::Rotation && B.Type == EAnimDatabaseAttributeType::Rotation) ||
		(A.Type == EAnimDatabaseAttributeType::Transform && B.Type == EAnimDatabaseAttributeType::Transform) ||
		(A.Type == EAnimDatabaseAttributeType::Angle && B.Type == EAnimDatabaseAttributeType::Float) ||
		(A.Type == EAnimDatabaseAttributeType::Float && B.Type == EAnimDatabaseAttributeType::Angle)))
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeMultiply: Invalid FrameAttribute Types. Got {Type0} and {Type1}.", AttributeTypeNameInternal(A.Type), AttributeTypeNameInternal(B.Type));
		return FAnimDatabaseFrameAttribute();
	}

	if ((A.Type == EAnimDatabaseAttributeType::Float && B.Type == EAnimDatabaseAttributeType::Float) ||
		(A.Type == EAnimDatabaseAttributeType::Location && B.Type == EAnimDatabaseAttributeType::Location))
	{
		FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
		Out.Type = A.Type;
		UE::Learning::FrameAttribute::Mul(*Out.FrameAttribute, *A.FrameAttribute, *B.FrameAttribute);
		return Out;
	}
	else if (A.Type == EAnimDatabaseAttributeType::Float && B.Type == EAnimDatabaseAttributeType::Location)
	{
		FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
		Out.Type = B.Type;
		UE::Learning::FFrameAttribute LocationA;
		UE::Learning::FrameAttribute::Repeat(LocationA, 3, *A.FrameAttribute);
		UE::Learning::FrameAttribute::Mul(*Out.FrameAttribute, LocationA, *B.FrameAttribute);
		return Out;
	}
	else if (A.Type == EAnimDatabaseAttributeType::Location && B.Type == EAnimDatabaseAttributeType::Float)
	{
		FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
		Out.Type = A.Type;
		UE::Learning::FFrameAttribute LocationB;
		UE::Learning::FrameAttribute::Repeat(LocationB, 3, *B.FrameAttribute);
		UE::Learning::FrameAttribute::Mul(*Out.FrameAttribute, *A.FrameAttribute, LocationB);
		return Out;
	}
	else if (A.Type == EAnimDatabaseAttributeType::Rotation && B.Type == EAnimDatabaseAttributeType::Rotation)
	{
		FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
		Out.Type = EAnimDatabaseAttributeType::Rotation;
		UE::Learning::FrameAttribute::QuatMul(*Out.FrameAttribute, *A.FrameAttribute, *B.FrameAttribute);
		return Out;
	}
	else if (A.Type == EAnimDatabaseAttributeType::Transform && B.Type == EAnimDatabaseAttributeType::Transform)
	{
		FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
		Out.Type = EAnimDatabaseAttributeType::Transform;
		UE::Learning::FrameAttribute::TransformMul(*Out.FrameAttribute, *A.FrameAttribute, *B.FrameAttribute);
		return Out;
	}
	else if (A.Type == EAnimDatabaseAttributeType::Angle || B.Type == EAnimDatabaseAttributeType::Angle)
	{
		FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
		Out.Type = EAnimDatabaseAttributeType::Angle;
		UE::Learning::FrameAttribute::Mul(*Out.FrameAttribute, *A.FrameAttribute, *B.FrameAttribute);
		UE::Learning::FrameAttribute::WrapAngleInplace(*Out.FrameAttribute);
		return Out;
	}
	else
	{
		checkNoEntry();
		return FAnimDatabaseFrameAttribute();
	}
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeDivide(FAnimDatabaseFrameAttribute A, FAnimDatabaseFrameAttribute B)
{
	if (!A.IsValid() || !B.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeDivide: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (!(
		(A.Type == EAnimDatabaseAttributeType::Float && B.Type == EAnimDatabaseAttributeType::Float) ||
		(A.Type == EAnimDatabaseAttributeType::Location && B.Type == EAnimDatabaseAttributeType::Location) ||
		(A.Type == EAnimDatabaseAttributeType::Rotation && B.Type == EAnimDatabaseAttributeType::Rotation) ||
		(A.Type == EAnimDatabaseAttributeType::Transform && B.Type == EAnimDatabaseAttributeType::Transform) ||
		(A.Type == EAnimDatabaseAttributeType::Angle && B.Type == EAnimDatabaseAttributeType::Float) ||
		(A.Type == EAnimDatabaseAttributeType::Float && B.Type == EAnimDatabaseAttributeType::Angle)))
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeDivide: Invalid FrameAttribute Types. Got {Type0} and {Type1}.", AttributeTypeNameInternal(A.Type), AttributeTypeNameInternal(B.Type));
		return FAnimDatabaseFrameAttribute();
	}

	if ((A.Type == EAnimDatabaseAttributeType::Float && B.Type == EAnimDatabaseAttributeType::Float) ||
		(A.Type == EAnimDatabaseAttributeType::Location && B.Type == EAnimDatabaseAttributeType::Location))
	{
		FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
		Out.Type = A.Type;
		UE::Learning::FrameAttribute::Div(*Out.FrameAttribute, *A.FrameAttribute, *B.FrameAttribute);
		return Out;
	}
	else if (A.Type == EAnimDatabaseAttributeType::Rotation && B.Type == EAnimDatabaseAttributeType::Rotation)
	{
		FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
		Out.Type = EAnimDatabaseAttributeType::Rotation;
		UE::Learning::FrameAttribute::QuatDiv(*Out.FrameAttribute, *A.FrameAttribute, *B.FrameAttribute);
		return Out;
	}
	else if (A.Type == EAnimDatabaseAttributeType::Transform && B.Type == EAnimDatabaseAttributeType::Transform)
	{
		FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
		Out.Type = EAnimDatabaseAttributeType::Transform;
		UE::Learning::FrameAttribute::TransformDiv(*Out.FrameAttribute, *A.FrameAttribute, *B.FrameAttribute);
		return Out;
	}
	else if (A.Type == EAnimDatabaseAttributeType::Angle || B.Type == EAnimDatabaseAttributeType::Angle)
	{
		FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
		Out.Type = EAnimDatabaseAttributeType::Angle;
		UE::Learning::FrameAttribute::Div(*Out.FrameAttribute, *A.FrameAttribute, *B.FrameAttribute);
		UE::Learning::FrameAttribute::WrapAngleInplace(*Out.FrameAttribute);
		return Out;
	}
	else
	{
		checkNoEntry();
		return FAnimDatabaseFrameAttribute();
	}
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeMax(FAnimDatabaseFrameAttribute A, FAnimDatabaseFrameAttribute B)
{
	if (!A.IsValid() || !B.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeMax: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (!(
		(A.Type == EAnimDatabaseAttributeType::Float && B.Type == EAnimDatabaseAttributeType::Float) ||
		(A.Type == EAnimDatabaseAttributeType::Location && B.Type == EAnimDatabaseAttributeType::Location)))
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeMax: Invalid FrameAttribute Types. Got {Type0} and {Type1}.", AttributeTypeNameInternal(A.Type), AttributeTypeNameInternal(B.Type));
		return FAnimDatabaseFrameAttribute();
	}

	EAnimDatabaseAttributeType OutputType = EAnimDatabaseAttributeType::Float;
	if (A.Type == EAnimDatabaseAttributeType::Location && B.Type == EAnimDatabaseAttributeType::Location) { OutputType = EAnimDatabaseAttributeType::Location; }

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = OutputType;
	UE::Learning::FrameAttribute::Max(*Out.FrameAttribute, *A.FrameAttribute, *B.FrameAttribute);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeMin(FAnimDatabaseFrameAttribute A, FAnimDatabaseFrameAttribute B)
{
	if (!A.IsValid() || !B.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeMin: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (!(
		(A.Type == EAnimDatabaseAttributeType::Float && B.Type == EAnimDatabaseAttributeType::Float) ||
		(A.Type == EAnimDatabaseAttributeType::Location && B.Type == EAnimDatabaseAttributeType::Location)))
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeMin: Invalid FrameAttribute Types. Got {Type0} and {Type1}.", AttributeTypeNameInternal(A.Type), AttributeTypeNameInternal(B.Type));
		return FAnimDatabaseFrameAttribute();
	}

	EAnimDatabaseAttributeType OutputType = EAnimDatabaseAttributeType::Float;
	if (A.Type == EAnimDatabaseAttributeType::Location && B.Type == EAnimDatabaseAttributeType::Location) { OutputType = EAnimDatabaseAttributeType::Location; }

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = OutputType;
	UE::Learning::FrameAttribute::Min(*Out.FrameAttribute, *A.FrameAttribute, *B.FrameAttribute);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeDot(FAnimDatabaseFrameAttribute A, FAnimDatabaseFrameAttribute B)
{
	if (!A.IsValid() || !B.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeDot: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (!(
		(A.Type == EAnimDatabaseAttributeType::Location && B.Type == EAnimDatabaseAttributeType::Location) ||
		(A.Type == EAnimDatabaseAttributeType::Direction && B.Type == EAnimDatabaseAttributeType::Direction)))
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeDot: Invalid FrameAttribute Types. Got {Type0} and {Type1}.", AttributeTypeNameInternal(A.Type), AttributeTypeNameInternal(B.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Float;
	UE::Learning::FrameAttribute::Dot(*Out.FrameAttribute, *A.FrameAttribute, *B.FrameAttribute);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeCross(FAnimDatabaseFrameAttribute A, FAnimDatabaseFrameAttribute B)
{
	if (!A.IsValid() || !B.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeDot: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (!(
		(A.Type == EAnimDatabaseAttributeType::Location && B.Type == EAnimDatabaseAttributeType::Location) ||
		(A.Type == EAnimDatabaseAttributeType::Direction && B.Type == EAnimDatabaseAttributeType::Direction)))
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeDot: Invalid FrameAttribute Types. Got {Type0} and {Type1}.", AttributeTypeNameInternal(A.Type), AttributeTypeNameInternal(B.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = A.Type;
	UE::Learning::FrameAttribute::Cross(*Out.FrameAttribute, *A.FrameAttribute, *B.FrameAttribute);
	if (Out.Type == EAnimDatabaseAttributeType::Direction)
	{
		UE::Learning::FrameAttribute::NormalizeInplace(*Out.FrameAttribute);
	}
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeAddFloat(const FAnimDatabaseFrameAttribute& A, const float B)
{
	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeAddFloat: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type != EAnimDatabaseAttributeType::Float &&
		A.Type != EAnimDatabaseAttributeType::Location &&
		A.Type != EAnimDatabaseAttributeType::Scale &&
		A.Type != EAnimDatabaseAttributeType::LinearVelocity &&
		A.Type != EAnimDatabaseAttributeType::AngularVelocity &&
		A.Type != EAnimDatabaseAttributeType::ScalarVelocity &&
		A.Type != EAnimDatabaseAttributeType::Angle)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeAddFloat: FrameAttribute Type not valid. Got {Type}.", AttributeTypeNameInternal(A.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = A.Type;
	if (A.Type == EAnimDatabaseAttributeType::Float)
	{
		UE::Learning::FrameAttribute::AddConstant(*Out.FrameAttribute, *A.FrameAttribute, { B });
	}
	else if (A.Type == EAnimDatabaseAttributeType::Angle)
	{
		UE::Learning::FrameAttribute::AddConstant(*Out.FrameAttribute, *A.FrameAttribute, { B });
		UE::Learning::FrameAttribute::WrapAngleInplace(*Out.FrameAttribute);
	}
	else
	{
		UE::Learning::FrameAttribute::AddConstant(*Out.FrameAttribute, *A.FrameAttribute, { B, B, B });
	}
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeSubtractFloat(const FAnimDatabaseFrameAttribute& A, const float B)
{
	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeSubtractFloat: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type != EAnimDatabaseAttributeType::Float &&
		A.Type != EAnimDatabaseAttributeType::Location &&
		A.Type != EAnimDatabaseAttributeType::Scale &&
		A.Type != EAnimDatabaseAttributeType::LinearVelocity &&
		A.Type != EAnimDatabaseAttributeType::AngularVelocity &&
		A.Type != EAnimDatabaseAttributeType::ScalarVelocity &&
		A.Type != EAnimDatabaseAttributeType::Angle)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeSubtractFloat: FrameAttribute Type not valid. Got {Type}.", AttributeTypeNameInternal(A.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = A.Type;
	if (A.Type == EAnimDatabaseAttributeType::Float)
	{
		UE::Learning::FrameAttribute::SubConstant(*Out.FrameAttribute, *A.FrameAttribute, { B });
	}
	else if (A.Type == EAnimDatabaseAttributeType::Angle)
	{
		UE::Learning::FrameAttribute::SubConstant(*Out.FrameAttribute, *A.FrameAttribute, { B });
		UE::Learning::FrameAttribute::WrapAngleInplace(*Out.FrameAttribute);
	}
	else
	{
		UE::Learning::FrameAttribute::SubConstant(*Out.FrameAttribute, *A.FrameAttribute, { B, B, B });
	}
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeMultiplyFloat(const FAnimDatabaseFrameAttribute& A, const float B)
{
	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeMultiplyFloat: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type != EAnimDatabaseAttributeType::Float &&
		A.Type != EAnimDatabaseAttributeType::Location &&
		A.Type != EAnimDatabaseAttributeType::Scale &&
		A.Type != EAnimDatabaseAttributeType::LinearVelocity &&
		A.Type != EAnimDatabaseAttributeType::AngularVelocity &&
		A.Type != EAnimDatabaseAttributeType::ScalarVelocity &&
		A.Type != EAnimDatabaseAttributeType::Angle)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeMultiplyFloat: FrameAttribute Type not valid. Got {Type}.", AttributeTypeNameInternal(A.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = A.Type;
	if (A.Type == EAnimDatabaseAttributeType::Float)
	{
		UE::Learning::FrameAttribute::MulConstant(*Out.FrameAttribute, *A.FrameAttribute, { B });
	}
	else if (A.Type == EAnimDatabaseAttributeType::Angle)
	{
		UE::Learning::FrameAttribute::MulConstant(*Out.FrameAttribute, *A.FrameAttribute, { B });
		UE::Learning::FrameAttribute::WrapAngleInplace(*Out.FrameAttribute);
	}
	else
	{
		UE::Learning::FrameAttribute::MulConstant(*Out.FrameAttribute, *A.FrameAttribute, { B, B, B });
	}
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeDivideFloat(const FAnimDatabaseFrameAttribute& A, const float B)
{
	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeDivideFloat: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type != EAnimDatabaseAttributeType::Float &&
		A.Type != EAnimDatabaseAttributeType::Location &&
		A.Type != EAnimDatabaseAttributeType::Scale &&
		A.Type != EAnimDatabaseAttributeType::LinearVelocity &&
		A.Type != EAnimDatabaseAttributeType::AngularVelocity &&
		A.Type != EAnimDatabaseAttributeType::ScalarVelocity &&
		A.Type != EAnimDatabaseAttributeType::Angle)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeDivideFloat: FrameAttribute Type not valid. Got {Type}.", AttributeTypeNameInternal(A.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = A.Type;
	if (A.Type == EAnimDatabaseAttributeType::Float)
	{
		UE::Learning::FrameAttribute::DivConstant(*Out.FrameAttribute, *A.FrameAttribute, { B });
	}
	else if (A.Type == EAnimDatabaseAttributeType::Angle)
	{
		UE::Learning::FrameAttribute::DivConstant(*Out.FrameAttribute, *A.FrameAttribute, { B });
		UE::Learning::FrameAttribute::WrapAngleInplace(*Out.FrameAttribute);
	}
	else
	{
		UE::Learning::FrameAttribute::DivConstant(*Out.FrameAttribute, *A.FrameAttribute, { B, B, B });
	}
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeMaxFloat(const FAnimDatabaseFrameAttribute& A, const float B)
{
	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeMaxFloat: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type != EAnimDatabaseAttributeType::Float &&
		A.Type != EAnimDatabaseAttributeType::Location)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeMaxFloat: FrameAttribute Type not valid. Got {Type}.", AttributeTypeNameInternal(A.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = A.Type;
	if (A.Type == EAnimDatabaseAttributeType::Float)
	{
		UE::Learning::FrameAttribute::MaxConstant(*Out.FrameAttribute, *A.FrameAttribute, { B });
	}
	else
	{
		UE::Learning::FrameAttribute::MaxConstant(*Out.FrameAttribute, *A.FrameAttribute, { B, B, B });
	}
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeMinFloat(const FAnimDatabaseFrameAttribute& A, const float B)
{
	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeMinFloat: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type != EAnimDatabaseAttributeType::Float &&
		A.Type != EAnimDatabaseAttributeType::Location)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeMinFloat: FrameAttribute Type not valid. Got {Type}.", AttributeTypeNameInternal(A.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = A.Type;
	if (A.Type == EAnimDatabaseAttributeType::Float)
	{
		UE::Learning::FrameAttribute::MinConstant(*Out.FrameAttribute, *A.FrameAttribute, { B });
	}
	else
	{
		UE::Learning::FrameAttribute::MinConstant(*Out.FrameAttribute, *A.FrameAttribute, { B, B, B });
	}
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeAddVector(const FAnimDatabaseFrameAttribute& A, const FVector B)
{
	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeAddVector: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type != EAnimDatabaseAttributeType::Location &&
		A.Type != EAnimDatabaseAttributeType::Scale &&
		A.Type != EAnimDatabaseAttributeType::LinearVelocity &&
		A.Type != EAnimDatabaseAttributeType::AngularVelocity &&
		A.Type != EAnimDatabaseAttributeType::ScalarVelocity)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeAddVector: FrameAttribute Type not valid. Got {Type}.", AttributeTypeNameInternal(A.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = A.Type;
	UE::Learning::FrameAttribute::AddConstant(*Out.FrameAttribute, *A.FrameAttribute, { (float)B.X, (float)B.Y, (float)B.Z });
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeSubtractVector(const FAnimDatabaseFrameAttribute& A, const FVector B)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeSubtractVector);

	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeSubtractVector: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type != EAnimDatabaseAttributeType::Location &&
		A.Type != EAnimDatabaseAttributeType::Scale &&
		A.Type != EAnimDatabaseAttributeType::LinearVelocity &&
		A.Type != EAnimDatabaseAttributeType::AngularVelocity &&
		A.Type != EAnimDatabaseAttributeType::ScalarVelocity)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeSubtractVector: FrameAttribute Type not valid. Got {Type}.", AttributeTypeNameInternal(A.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = A.Type;
	UE::Learning::FrameAttribute::SubConstant(*Out.FrameAttribute, *A.FrameAttribute, { (float)B.X, (float)B.Y, (float)B.Z });
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeMultiplyVector(const FAnimDatabaseFrameAttribute& A, const FVector B)
{
	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeMultiplyVector: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type != EAnimDatabaseAttributeType::Location &&
		A.Type != EAnimDatabaseAttributeType::Scale &&
		A.Type != EAnimDatabaseAttributeType::LinearVelocity &&
		A.Type != EAnimDatabaseAttributeType::AngularVelocity &&
		A.Type != EAnimDatabaseAttributeType::ScalarVelocity)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeMultiplyVector: FrameAttribute Type not valid. Got {Type}.", AttributeTypeNameInternal(A.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = A.Type;
	UE::Learning::FrameAttribute::MulConstant(*Out.FrameAttribute, *A.FrameAttribute, { (float)B.X, (float)B.Y, (float)B.Z });
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeDivideVector(const FAnimDatabaseFrameAttribute& A, const FVector B)
{
	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeDivideVector: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type != EAnimDatabaseAttributeType::Location &&
		A.Type != EAnimDatabaseAttributeType::Scale &&
		A.Type != EAnimDatabaseAttributeType::LinearVelocity &&
		A.Type != EAnimDatabaseAttributeType::AngularVelocity &&
		A.Type != EAnimDatabaseAttributeType::ScalarVelocity)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeDivideVector: FrameAttribute Type not valid. Got {Type}.", AttributeTypeNameInternal(A.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = A.Type;
	UE::Learning::FrameAttribute::DivConstant(*Out.FrameAttribute, *A.FrameAttribute, { (float)B.X, (float)B.Y, (float)B.Z });
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeQuatMultiply(const FQuat A, const FAnimDatabaseFrameAttribute& B)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeQuatMultiply);

	if (!B.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeQuatMultiply: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (B.Type != EAnimDatabaseAttributeType::Rotation)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeQuatMultiply: FrameAttribute Type not valid. Got {Type}. Expected Rotation.", AttributeTypeNameInternal(B.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = B.Type;
	UE::Learning::FrameAttribute::QuatConstantMul(*Out.FrameAttribute, (FQuat4f)A, *B.FrameAttribute);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeQuatDivide(const FQuat A, const FAnimDatabaseFrameAttribute& B)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeQuatDivide);

	if (!B.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeQuatDivide: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (B.Type != EAnimDatabaseAttributeType::Rotation)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeQuatDivide: FrameAttribute Type not valid. Got {Type}. Expected Rotation.", AttributeTypeNameInternal(B.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = B.Type;
	UE::Learning::FrameAttribute::QuatConstantDiv(*Out.FrameAttribute, (FQuat4f)A, *B.FrameAttribute);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeMultiplyQuat(const FAnimDatabaseFrameAttribute& A, const FQuat B)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeMultiplyQuat);

	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeMultiplyQuat: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type != EAnimDatabaseAttributeType::Rotation)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeMultiplyQuat: FrameAttribute Type not valid. Got {Type}. Expected Rotation.", AttributeTypeNameInternal(A.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = A.Type;
	UE::Learning::FrameAttribute::QuatMulConstant(*Out.FrameAttribute, *A.FrameAttribute, (FQuat4f)B);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeDivideQuat(const FAnimDatabaseFrameAttribute& A, const FQuat B)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeDivideQuat);

	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeDivideQuat: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type != EAnimDatabaseAttributeType::Rotation)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeDivideQuat: FrameAttribute Type not valid. Got {Type}. Expected Rotation.", AttributeTypeNameInternal(A.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = A.Type;
	UE::Learning::FrameAttribute::QuatDivConstant(*Out.FrameAttribute, *A.FrameAttribute, (FQuat4f)B);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeTransformMultiply(const FTransform& A, const FAnimDatabaseFrameAttribute& B)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeTransformMultiply);

	if (!B.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeTransformMultiply: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (B.Type != EAnimDatabaseAttributeType::Transform)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeTransformMultiply: FrameAttribute Type not valid. Got {Type}. Expected Transform.", AttributeTypeNameInternal(B.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = B.Type;
	UE::Learning::FrameAttribute::TransformConstantMul(*Out.FrameAttribute, (FTransform3f)A, *B.FrameAttribute);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeTransformDivide(const FTransform& A, const FAnimDatabaseFrameAttribute& B)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeTransformDivide);

	if (!B.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeTransformDivide: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (B.Type != EAnimDatabaseAttributeType::Transform)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeTransformDivide: FrameAttribute Type not valid. Got {Type}. Expected Transform.", AttributeTypeNameInternal(B.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = B.Type;
	UE::Learning::FrameAttribute::TransformConstantDiv(*Out.FrameAttribute, (FTransform3f)A, *B.FrameAttribute);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeMultiplyTransform(const FAnimDatabaseFrameAttribute& A, const FTransform& B)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeMultiplyTransform);

	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeMultiplyTransform: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type != EAnimDatabaseAttributeType::Transform)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeMultiplyTransform: FrameAttribute Type not valid. Got {Type}. Expected Transform.", AttributeTypeNameInternal(A.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = A.Type;
	UE::Learning::FrameAttribute::TransformMulConstant(*Out.FrameAttribute, *A.FrameAttribute, (FTransform3f)B);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeDivideTransform(const FAnimDatabaseFrameAttribute& A, const FTransform& B)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeDivideTransform);

	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeDivideTransform: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type != EAnimDatabaseAttributeType::Transform)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeDivideTransform: FrameAttribute Type not valid. Got {Type}. Expected Transform.", AttributeTypeNameInternal(A.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = A.Type;
	UE::Learning::FrameAttribute::TransformDivConstant(*Out.FrameAttribute, *A.FrameAttribute, (FTransform3f)B);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeTransformApply(const FTransform& A, const FAnimDatabaseFrameAttribute& B)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeTransformApply);

	if (!B.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeTransformApply: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (B.Type != EAnimDatabaseAttributeType::Location &&
		B.Type != EAnimDatabaseAttributeType::Direction &&
		B.Type != EAnimDatabaseAttributeType::LinearVelocity &&
		B.Type != EAnimDatabaseAttributeType::AngularVelocity)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeTransformApply: FrameAttribute Type not valid. Got {Type}. Expected Location, Direction, LinearVelocity or AngularVelocity.", AttributeTypeNameInternal(B.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = B.Type;
	if (B.Type == EAnimDatabaseAttributeType::Location)
	{
		UE::Learning::FrameAttribute::TransformConstantApply(*Out.FrameAttribute, (FTransform3f)A, *B.FrameAttribute);
	}
	else if (B.Type == EAnimDatabaseAttributeType::LinearVelocity || B.Type == EAnimDatabaseAttributeType::AngularVelocity)
	{
		UE::Learning::FrameAttribute::TransformConstantApplyNoTranslation(*Out.FrameAttribute, (FTransform3f)A, *B.FrameAttribute);
	}
	else if (B.Type == EAnimDatabaseAttributeType::Direction)
	{
		UE::Learning::FrameAttribute::TransformConstantApplyNoTranslationScale(*Out.FrameAttribute, (FTransform3f)A, *B.FrameAttribute);
	}
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeApplyTransformLocation(const FAnimDatabaseFrameAttribute& A, const FVector B)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeApplyTransformLocation);

	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeApplyTransformLocation: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type != EAnimDatabaseAttributeType::Transform)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeApplyTransformLocation: FrameAttribute Type not valid. Got {Type}. Expected Transform.", AttributeTypeNameInternal(A.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Location;
	UE::Learning::FrameAttribute::TransformApplyConstantLocation(*Out.FrameAttribute, *A.FrameAttribute, (FVector3f)B);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeApplyTransformDirection(const FAnimDatabaseFrameAttribute& A, const FVector B)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeApplyTransformDirection);

	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeApplyTransformDirection: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type != EAnimDatabaseAttributeType::Transform)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeApplyTransformDirection: FrameAttribute Type not valid. Got {Type}. Expected Transform.", AttributeTypeNameInternal(A.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Direction;
	UE::Learning::FrameAttribute::TransformApplyConstantDirection(*Out.FrameAttribute, *A.FrameAttribute, (FVector3f)B);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeAddAngleDegrees(const FAnimDatabaseFrameAttribute& A, const float B)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeAddAngleDegrees);

	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeAddAngleDegrees: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type != EAnimDatabaseAttributeType::Angle)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeAddAngleDegrees: FrameAttribute Type not valid. Got {Type}. Expected Angle.", AttributeTypeNameInternal(A.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = A.Type;
	UE::Learning::FrameAttribute::AddConstant(*Out.FrameAttribute, *A.FrameAttribute, { FMath::DegreesToRadians(B) });
	UE::Learning::FrameAttribute::WrapAngleInplace(*Out.FrameAttribute);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeSubtractAngleDegrees(const FAnimDatabaseFrameAttribute& A, const float B)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeSubtractAngleDegrees);

	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeSubtractAngleDegrees: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type != EAnimDatabaseAttributeType::Angle)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeSubtractAngleDegrees: FrameAttribute Type not valid. Got {Type}. Expected Angle.", AttributeTypeNameInternal(A.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = A.Type;
	UE::Learning::FrameAttribute::SubConstant(*Out.FrameAttribute, *A.FrameAttribute, { FMath::DegreesToRadians(B) });
	UE::Learning::FrameAttribute::WrapAngleInplace(*Out.FrameAttribute);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeAddAngleRadians(const FAnimDatabaseFrameAttribute& A, const float B)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeAddAngleRadians);

	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeAddAngleRadians: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type != EAnimDatabaseAttributeType::Angle)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeAddAngleRadians: FrameAttribute Type not valid. Got {Type}. Expected Angle.", AttributeTypeNameInternal(A.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = A.Type;
	UE::Learning::FrameAttribute::AddConstant(*Out.FrameAttribute, *A.FrameAttribute, { B });
	UE::Learning::FrameAttribute::WrapAngleInplace(*Out.FrameAttribute);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeSubtractAngleRadians(const FAnimDatabaseFrameAttribute& A, const float B)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeSubtractAngleRadians);

	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeSubtractAngleRadians: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type != EAnimDatabaseAttributeType::Angle)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeSubtractAngleRadians: FrameAttribute Type not valid. Got {Type}. Expected Angle.", AttributeTypeNameInternal(A.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = A.Type;
	UE::Learning::FrameAttribute::SubConstant(*Out.FrameAttribute, *A.FrameAttribute, { B });
	UE::Learning::FrameAttribute::WrapAngleInplace(*Out.FrameAttribute);
	return Out;
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributesAdd(TArray<FAnimDatabaseFrameAttribute>& Out, const TArray<FAnimDatabaseFrameAttribute>& A, const FAnimDatabaseFrameAttribute& B)
{
	Out.SetNum(A.Num());
	FrameAttributesAddArrayView(Out, A, B);
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributesAddArrayView(const TArrayView<FAnimDatabaseFrameAttribute> Out, const TArrayView<const FAnimDatabaseFrameAttribute> A, const FAnimDatabaseFrameAttribute& B)
{
	check(Out.Num() == A.Num());
	ParallelFor(Out.Num(), [&](int32 Idx) {
		Out[Idx] = FrameAttributeAdd(A[Idx], B);
		});
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributesSubtract(TArray<FAnimDatabaseFrameAttribute>& Out, const TArray<FAnimDatabaseFrameAttribute>& A, const FAnimDatabaseFrameAttribute& B)
{
	Out.SetNum(A.Num());
	FrameAttributesSubtractArrayView(Out, A, B);
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributesSubtractArrayView(const TArrayView<FAnimDatabaseFrameAttribute> Out, const TArrayView<const FAnimDatabaseFrameAttribute> A, const FAnimDatabaseFrameAttribute& B)
{
	check(Out.Num() == A.Num());
	ParallelFor(Out.Num(), [&](int32 Idx) {
		Out[Idx] = FrameAttributeSubtract(A[Idx], B);
		});
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributesMultiply(TArray<FAnimDatabaseFrameAttribute>& Out, const TArray<FAnimDatabaseFrameAttribute>& A, const FAnimDatabaseFrameAttribute& B)
{
	Out.SetNum(A.Num());
	FrameAttributesMultiplyArrayView(Out, A, B);
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributesMultiplyArrayView(const TArrayView<FAnimDatabaseFrameAttribute> Out, const TArrayView<const FAnimDatabaseFrameAttribute> A, const FAnimDatabaseFrameAttribute& B)
{
	check(Out.Num() == A.Num());
	ParallelFor(Out.Num(), [&](int32 Idx) {
		Out[Idx] = FrameAttributeMultiply(A[Idx], B);
		});
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributesDivide(TArray<FAnimDatabaseFrameAttribute>& Out, const TArray<FAnimDatabaseFrameAttribute>& A, const FAnimDatabaseFrameAttribute& B)
{
	Out.SetNum(A.Num());
	FrameAttributesDivideArrayView(Out, A, B);
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributesDivideArrayView(const TArrayView<FAnimDatabaseFrameAttribute> Out, const TArrayView<const FAnimDatabaseFrameAttribute> A, const FAnimDatabaseFrameAttribute& B)
{
	check(Out.Num() == A.Num());
	ParallelFor(Out.Num(), [&](int32 Idx) {
		Out[Idx] = FrameAttributeDivide(A[Idx], B);
		});
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeInverseMultiply(const FAnimDatabaseFrameAttribute& A, const FAnimDatabaseFrameAttribute& B)
{
	return FrameAttributeMultiply(FrameAttributeInvert(A), B);
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeMultiplyInverse(const FAnimDatabaseFrameAttribute& A, const FAnimDatabaseFrameAttribute& B)
{
	return FrameAttributeMultiply(A, FrameAttributeInvert(B));
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeRotate(const FAnimDatabaseFrameAttribute& Rotation, const FAnimDatabaseFrameAttribute& Vector)
{
	if (!Rotation.IsValid() || !Vector.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeRotate: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (Rotation.Type != EAnimDatabaseAttributeType::Rotation || (
		(Vector.Type != EAnimDatabaseAttributeType::Location) &&
		(Vector.Type != EAnimDatabaseAttributeType::Direction) &&
		(Vector.Type != EAnimDatabaseAttributeType::LinearVelocity) &&
		(Vector.Type != EAnimDatabaseAttributeType::AngularVelocity)))
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeRotate: FrameAttribute Types don't match. Got {Type0} and {Type1}. Expected Rotation and Location, Direction, or Velocity.", AttributeTypeNameInternal(Rotation.Type), AttributeTypeNameInternal(Vector.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = Vector.Type;
	UE::Learning::FrameAttribute::QuatRotate(*Out.FrameAttribute, *Rotation.FrameAttribute, *Vector.FrameAttribute);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeUnrotate(const FAnimDatabaseFrameAttribute& Rotation, const FAnimDatabaseFrameAttribute& Vector)
{
	if (!Rotation.IsValid() || !Vector.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeUnrotate: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (Rotation.Type != EAnimDatabaseAttributeType::Rotation || (
		(Vector.Type != EAnimDatabaseAttributeType::Location) &&
		(Vector.Type != EAnimDatabaseAttributeType::Direction) &&
		(Vector.Type != EAnimDatabaseAttributeType::LinearVelocity) &&
		(Vector.Type != EAnimDatabaseAttributeType::AngularVelocity)))
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeUnrotate: FrameAttribute Types don't match. Got {Type0} and {Type1}. Expected Rotation and Location, Direction, or Velocity.", AttributeTypeNameInternal(Rotation.Type), AttributeTypeNameInternal(Vector.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = Vector.Type;
	UE::Learning::FrameAttribute::QuatUnrotate(*Out.FrameAttribute, *Rotation.FrameAttribute, *Vector.FrameAttribute);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeQuatRotate(const FQuat& Rotation, const FAnimDatabaseFrameAttribute& Vector)
{
	if (!Vector.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeQuatRotate: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (Vector.Type != EAnimDatabaseAttributeType::Location &&
		Vector.Type != EAnimDatabaseAttributeType::Direction &&
		Vector.Type != EAnimDatabaseAttributeType::LinearVelocity &&
		Vector.Type != EAnimDatabaseAttributeType::AngularVelocity)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeQuatRotate: FrameAttribute Types don't match. Got {Type1}. Expected Location, Direction, or Velocity.", AttributeTypeNameInternal(Vector.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = Vector.Type;
	UE::Learning::FrameAttribute::QuatConstantRotate(*Out.FrameAttribute, (FQuat4f)Rotation, *Vector.FrameAttribute);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeQuatUnrotate(const FQuat& Rotation, const FAnimDatabaseFrameAttribute& Vector)
{
	if (!Vector.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeQuatUnrotate: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (Vector.Type != EAnimDatabaseAttributeType::Location &&
		Vector.Type != EAnimDatabaseAttributeType::Direction &&
		Vector.Type != EAnimDatabaseAttributeType::LinearVelocity &&
		Vector.Type != EAnimDatabaseAttributeType::AngularVelocity)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeQuatUnrotate: FrameAttribute Types don't match. Got {Type1}. Expected Location, Direction, or Velocity.", AttributeTypeNameInternal(Vector.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = Vector.Type;
	UE::Learning::FrameAttribute::QuatConstantUnrotate(*Out.FrameAttribute, (FQuat4f)Rotation, *Vector.FrameAttribute);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeRotateAndAdd(const FAnimDatabaseFrameAttribute& Vector, const FAnimDatabaseFrameAttribute& Rotation, const FAnimDatabaseFrameAttribute& Addition)
{
	return FrameAttributeAdd(FrameAttributeRotate(Rotation, Vector), Addition);
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeSubtractAndUnrotate(const FAnimDatabaseFrameAttribute& Vector, const FAnimDatabaseFrameAttribute& Subtraction, const FAnimDatabaseFrameAttribute& Rotation)
{
	return FrameAttributeUnrotate(Rotation, FrameAttributeSubtract(Vector, Subtraction));
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributesRotate(TArray<FAnimDatabaseFrameAttribute>& OutVectors, const FAnimDatabaseFrameAttribute& Rotation, const TArray<FAnimDatabaseFrameAttribute>& InVectors)
{
	OutVectors.SetNum(InVectors.Num());
	FrameAttributesRotateArrayView(OutVectors, Rotation, InVectors);
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributesRotateArrayView(const TArrayView<FAnimDatabaseFrameAttribute> OutVectors, const FAnimDatabaseFrameAttribute& Rotation, const TArrayView<const FAnimDatabaseFrameAttribute> InVectors)
{
	check(OutVectors.Num() == InVectors.Num());
	ParallelFor(OutVectors.Num(), [&](int32 Idx) { OutVectors[Idx] = FrameAttributeRotate(Rotation, InVectors[Idx]); });
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributesUnrotate(TArray<FAnimDatabaseFrameAttribute>& OutVectors, const FAnimDatabaseFrameAttribute& Rotation, const TArray<FAnimDatabaseFrameAttribute>& InVectors)
{
	OutVectors.SetNum(InVectors.Num());
	FrameAttributesUnrotateArrayView(OutVectors, Rotation, InVectors);
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributesUnrotateArrayView(const TArrayView<FAnimDatabaseFrameAttribute> OutVectors, const FAnimDatabaseFrameAttribute& Rotation, const TArrayView<const FAnimDatabaseFrameAttribute> InVectors)
{
	check(OutVectors.Num() == InVectors.Num());
	ParallelFor(OutVectors.Num(), [&](int32 Idx) { OutVectors[Idx] = FrameAttributeUnrotate(Rotation, InVectors[Idx]); });
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributesInverseMultiply(TArray<FAnimDatabaseFrameAttribute>& Out, const TArray<FAnimDatabaseFrameAttribute>& A, const FAnimDatabaseFrameAttribute& B)
{
	Out.SetNum(A.Num());
	FrameAttributesInverseMultiplyArrayView(Out, A, B);
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributesInverseMultiplyArrayView(const TArrayView<FAnimDatabaseFrameAttribute> Out, const TArrayView<const FAnimDatabaseFrameAttribute> A, const FAnimDatabaseFrameAttribute& B)
{
	check(Out.Num() == A.Num());
	ParallelFor(Out.Num(), [&](int32 Idx) { Out[Idx] = FrameAttributeInverseMultiply(A[Idx], B); });
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributesMultiplyInverse(TArray<FAnimDatabaseFrameAttribute>& Out, const TArray<FAnimDatabaseFrameAttribute>& A, const FAnimDatabaseFrameAttribute& B)
{
	Out.SetNum(A.Num());
	FrameAttributesMultiplyInverseArrayView(Out, A, B);
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributesMultiplyInverseArrayView(const TArrayView<FAnimDatabaseFrameAttribute> Out, const TArrayView<const FAnimDatabaseFrameAttribute> A, const FAnimDatabaseFrameAttribute& B)
{
	check(Out.Num() == A.Num());
	ParallelFor(Out.Num(), [&](int32 Idx) { Out[Idx] = FrameAttributeMultiplyInverse(A[Idx], B); });
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributesRotateAndAdd(TArray<FAnimDatabaseFrameAttribute>& Out, const TArray<FAnimDatabaseFrameAttribute>& FrameAttributes, const FAnimDatabaseFrameAttribute& Rotation, const FAnimDatabaseFrameAttribute& Addition)
{
	Out.SetNum(FrameAttributes.Num());
	FrameAttributesRotateAndAddArrayView(Out, FrameAttributes, Rotation, Addition);
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributesRotateAndAddArrayView(const TArrayView<FAnimDatabaseFrameAttribute> Out, const TArrayView<const FAnimDatabaseFrameAttribute> FrameAttributes, const FAnimDatabaseFrameAttribute& Rotation, const FAnimDatabaseFrameAttribute& Addition)
{
	check(Out.Num() == FrameAttributes.Num());
	ParallelFor(Out.Num(), [&](int32 Idx) { Out[Idx] = FrameAttributeAdd(FrameAttributeRotate(Rotation, FrameAttributes[Idx]), Addition); });
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributesSubtractAndUnrotate(TArray<FAnimDatabaseFrameAttribute>& Out, const TArray<FAnimDatabaseFrameAttribute>& FrameAttributes, const FAnimDatabaseFrameAttribute& Subtraction, const FAnimDatabaseFrameAttribute& Rotation)
{
	Out.SetNum(FrameAttributes.Num());
	FrameAttributesSubtractAndUnrotateArrayView(Out, FrameAttributes, Subtraction, Rotation);
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributesSubtractAndUnrotateArrayView(const TArrayView<FAnimDatabaseFrameAttribute> Out, const TArrayView<const FAnimDatabaseFrameAttribute> FrameAttributes, const FAnimDatabaseFrameAttribute& Subtraction, const FAnimDatabaseFrameAttribute& Rotation)
{
	check(Out.Num() == FrameAttributes.Num());
	ParallelFor(Out.Num(), [&](int32 Idx) { Out[Idx] = FrameAttributeUnrotate(Rotation, FrameAttributeSubtract(FrameAttributes[Idx], Subtraction)); });
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeRotateDirection(const FAnimDatabaseFrameAttribute& Rotation, const FVector Direction)
{
	if (!Rotation.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeRotateDirection: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (Rotation.Type != EAnimDatabaseAttributeType::Rotation)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeRotateDirection: FrameAttribute Types don't match. Got {Type}. Expected Rotation.", AttributeTypeNameInternal(Rotation.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Direction;
	UE::Learning::FrameAttribute::QuatRotateConstant(*Out.FrameAttribute, *Rotation.FrameAttribute, ((FVector3f)Direction).GetSafeNormal());
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeUnrotateDirection(const FAnimDatabaseFrameAttribute& Rotation, const FVector Direction)
{
	if (!Rotation.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeUnrotateDirection: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (Rotation.Type != EAnimDatabaseAttributeType::Rotation)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeUnrotateDirection: FrameAttribute Types don't match. Got {Type}. Expected Rotation.", AttributeTypeNameInternal(Rotation.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Direction;
	UE::Learning::FrameAttribute::QuatUnrotateConstant(*Out.FrameAttribute, *Rotation.FrameAttribute, ((FVector3f)Direction).GetSafeNormal());
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeRotationBetweenDirection(const FAnimDatabaseFrameAttribute& Directions, const FVector Direction)
{
	if (!Directions.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeRotationBetweenDirection: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (Directions.Type != EAnimDatabaseAttributeType::Direction)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeRotationBetweenDirection: FrameAttribute Types don't match. Got {Type}. Expected Direction.", AttributeTypeNameInternal(Directions.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Rotation;
	UE::Learning::FrameAttribute::QuatBetweenConstant(*Out.FrameAttribute, *Directions.FrameAttribute, ((FVector3f)Direction).GetSafeNormal());
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeProject(const FAnimDatabaseFrameAttribute& A, const FVector Projection)
{
	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeProject: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type != EAnimDatabaseAttributeType::Direction &&
		A.Type != EAnimDatabaseAttributeType::Location &&
		A.Type != EAnimDatabaseAttributeType::LinearVelocity &&
		A.Type != EAnimDatabaseAttributeType::AngularVelocity &&
		A.Type != EAnimDatabaseAttributeType::ScalarVelocity)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeProject: FrameAttribute Types don't match. Got {Type}. Expected Location, Direction or Velocity.", AttributeTypeNameInternal(A.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = A.Type;

	if (A.Type == EAnimDatabaseAttributeType::Direction)
	{
		UE::Learning::FrameAttribute::MulConstant(*Out.FrameAttribute, *A.FrameAttribute, { (float)Projection.X, (float)Projection.Y, (float)Projection.Z });
		UE::Learning::FrameAttribute::NormalizeInplace(*Out.FrameAttribute);
	}
	else
	{
		UE::Learning::FrameAttribute::MulConstant(*Out.FrameAttribute, *A.FrameAttribute, { (float)Projection.X, (float)Projection.Y, (float)Projection.Z });

	}
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeCopy(const FAnimDatabaseFrameAttribute& A)
{
	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeCopy: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = A.Type;
	UE::Learning::FrameAttribute::Copy(*Out.FrameAttribute, *A.FrameAttribute);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeNegate(const FAnimDatabaseFrameAttribute& A)
{
	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeNegate: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type != EAnimDatabaseAttributeType::Float && 
		A.Type != EAnimDatabaseAttributeType::Location &&
		A.Type != EAnimDatabaseAttributeType::Scale &&
		A.Type != EAnimDatabaseAttributeType::LinearVelocity &&
		A.Type != EAnimDatabaseAttributeType::AngularVelocity &&
		A.Type != EAnimDatabaseAttributeType::ScalarVelocity &&
		A.Type != EAnimDatabaseAttributeType::Direction &&
		A.Type != EAnimDatabaseAttributeType::Angle)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeNegate: FrameAttribute Types can't be negated. Got {Type}.", AttributeTypeNameInternal(A.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = A.Type;
	UE::Learning::FrameAttribute::Neg(*Out.FrameAttribute, *A.FrameAttribute);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeInvert(const FAnimDatabaseFrameAttribute& A)
{
	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeInvert: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type != EAnimDatabaseAttributeType::Rotation && A.Type != EAnimDatabaseAttributeType::Transform)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeInvert: FrameAttribute Types don't match. Got {Type}. Expected Rotation or Transform.", AttributeTypeNameInternal(A.Type));
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type == EAnimDatabaseAttributeType::Rotation)
	{
		FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
		Out.Type = EAnimDatabaseAttributeType::Rotation;
		UE::Learning::FrameAttribute::QuatInv(*Out.FrameAttribute, *A.FrameAttribute);
		return Out;
	}
	else if (A.Type == EAnimDatabaseAttributeType::Transform)
	{
		FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
		Out.Type = EAnimDatabaseAttributeType::Transform;
		UE::Learning::FrameAttribute::TransformInv(*Out.FrameAttribute, *A.FrameAttribute);
		return Out;
	}
	else
	{
		checkNoEntry();
		return FAnimDatabaseFrameAttribute();
	}
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeAbs(const FAnimDatabaseFrameAttribute& A)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeAbs);

	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeAbs: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type != EAnimDatabaseAttributeType::Float && 
		A.Type != EAnimDatabaseAttributeType::Location &&
		A.Type != EAnimDatabaseAttributeType::Rotation &&
		A.Type != EAnimDatabaseAttributeType::Scale &&
		A.Type != EAnimDatabaseAttributeType::LinearVelocity &&
		A.Type != EAnimDatabaseAttributeType::AngularVelocity &&
		A.Type != EAnimDatabaseAttributeType::ScalarVelocity &&
		A.Type != EAnimDatabaseAttributeType::Angle)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeAbs: FrameAttribute Types invalid. Got {Type}.", AttributeTypeNameInternal(A.Type));
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type == EAnimDatabaseAttributeType::Rotation)
	{
		FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
		Out.Type = EAnimDatabaseAttributeType::Rotation;
		UE::Learning::FrameAttribute::QuatAbs(*Out.FrameAttribute, *A.FrameAttribute);
		return Out;
	}
	else
	{
		FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
		Out.Type = A.Type;
		UE::Learning::FrameAttribute::Abs(*Out.FrameAttribute, *A.FrameAttribute);
		return Out;
	}
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeLog(const FAnimDatabaseFrameAttribute& A)
{
	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeLog: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type != EAnimDatabaseAttributeType::Float &&
		A.Type != EAnimDatabaseAttributeType::Location &&
		A.Type != EAnimDatabaseAttributeType::Scale &&
		A.Type != EAnimDatabaseAttributeType::LinearVelocity &&
		A.Type != EAnimDatabaseAttributeType::AngularVelocity &&
		A.Type != EAnimDatabaseAttributeType::ScalarVelocity)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeLog: FrameAttribute Types invalid. Got {Type}.", AttributeTypeNameInternal(A.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = A.Type;
	UE::Learning::FrameAttribute::Log(*Out.FrameAttribute, *A.FrameAttribute);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeExp(const FAnimDatabaseFrameAttribute& A)
{
	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeExp: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type != EAnimDatabaseAttributeType::Float &&
		A.Type != EAnimDatabaseAttributeType::Location &&
		A.Type != EAnimDatabaseAttributeType::Scale &&
		A.Type != EAnimDatabaseAttributeType::LinearVelocity &&
		A.Type != EAnimDatabaseAttributeType::AngularVelocity &&
		A.Type != EAnimDatabaseAttributeType::ScalarVelocity)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeExp: FrameAttribute Types invalid. Got {Type}.", AttributeTypeNameInternal(A.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = A.Type;
	UE::Learning::FrameAttribute::Exp(*Out.FrameAttribute, *A.FrameAttribute);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeSin(const FAnimDatabaseFrameAttribute& A)
{
	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeSin: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type != EAnimDatabaseAttributeType::Angle)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeSin: FrameAttribute Types invalid. Got {Type}.", AttributeTypeNameInternal(A.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Float;
	UE::Learning::FrameAttribute::Sin(*Out.FrameAttribute, *A.FrameAttribute);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeCos(const FAnimDatabaseFrameAttribute& A)
{
	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeCos: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type != EAnimDatabaseAttributeType::Angle)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeCos: FrameAttribute Types invalid. Got {Type}.", AttributeTypeNameInternal(A.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Float;
	UE::Learning::FrameAttribute::Cos(*Out.FrameAttribute, *A.FrameAttribute);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeAtan2(const FAnimDatabaseFrameAttribute& Y, const FAnimDatabaseFrameAttribute& X)
{
	if (!Y.IsValid() || !X.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeAtan2: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (Y.Type != EAnimDatabaseAttributeType::Float || X.Type != EAnimDatabaseAttributeType::Float)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeAtan2: FrameAttribute Types invalid. Got {Type} and {Type}.", AttributeTypeNameInternal(Y.Type), AttributeTypeNameInternal(X.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Angle;
	UE::Learning::FrameAttribute::Atan2(*Out.FrameAttribute, *Y.FrameAttribute, *X.FrameAttribute);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeLength(const FAnimDatabaseFrameAttribute& Vector)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeLength);

	if (!Vector.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeLength: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (Vector.Type != EAnimDatabaseAttributeType::Location &&
		Vector.Type != EAnimDatabaseAttributeType::Scale &&
		Vector.Type != EAnimDatabaseAttributeType::LinearVelocity &&
		Vector.Type != EAnimDatabaseAttributeType::AngularVelocity &&
		Vector.Type != EAnimDatabaseAttributeType::ScalarVelocity)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeLength: FrameAttribute Types don't match. Got {Type}. Expected Location, Scale, LinearVelocity, AngularVelocity, or ScalarVelocity.", AttributeTypeNameInternal(Vector.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Float;
	UE::Learning::FrameAttribute::Length(*Out.FrameAttribute, *Vector.FrameAttribute);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeToRotationVector(const FAnimDatabaseFrameAttribute& A)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeToRotationVector);

	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeToRotationVector: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type != EAnimDatabaseAttributeType::Rotation)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeToRotationVector: FrameAttribute Types don't match. Got {Type}. Expected Rotation.", AttributeTypeNameInternal(A.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::AngularVelocity;
	UE::Learning::FrameAttribute::QuatToRotationVector(*Out.FrameAttribute, *A.FrameAttribute);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeFromRotationVector(const FAnimDatabaseFrameAttribute& A)
{
	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeToRotationVector: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type != EAnimDatabaseAttributeType::AngularVelocity)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeToRotationVector: FrameAttribute Types don't match. Got {Type}. Expected Angular Velocity.", AttributeTypeNameInternal(A.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Rotation;
	UE::Learning::FrameAttribute::QuatFromRotationVector(*Out.FrameAttribute, *A.FrameAttribute);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeX(const FAnimDatabaseFrameAttribute& Vector)
{
	if (!Vector.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeX: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (Vector.Type != EAnimDatabaseAttributeType::Location &&
		Vector.Type != EAnimDatabaseAttributeType::Rotation &&
		Vector.Type != EAnimDatabaseAttributeType::Scale &&
		Vector.Type != EAnimDatabaseAttributeType::LinearVelocity &&
		Vector.Type != EAnimDatabaseAttributeType::AngularVelocity &&
		Vector.Type != EAnimDatabaseAttributeType::ScalarVelocity)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeX: FrameAttribute Types don't match. Got {Type}. Expected Location, Rotation, Scale, LinearVelocity, AngularVelocity, or ScalarVelocity.", AttributeTypeNameInternal(Vector.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Float;
	UE::Learning::FrameAttribute::Index(*Out.FrameAttribute, *Vector.FrameAttribute, 0);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeY(const FAnimDatabaseFrameAttribute& Vector)
{
	if (!Vector.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeY: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (Vector.Type != EAnimDatabaseAttributeType::Location &&
		Vector.Type != EAnimDatabaseAttributeType::Rotation &&
		Vector.Type != EAnimDatabaseAttributeType::Scale &&
		Vector.Type != EAnimDatabaseAttributeType::LinearVelocity &&
		Vector.Type != EAnimDatabaseAttributeType::AngularVelocity &&
		Vector.Type != EAnimDatabaseAttributeType::ScalarVelocity)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeY: FrameAttribute Types don't match. Got {Type}. Expected Location, Rotation, Scale, LinearVelocity, AngularVelocity, or ScalarVelocity.", AttributeTypeNameInternal(Vector.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Float;
	UE::Learning::FrameAttribute::Index(*Out.FrameAttribute, *Vector.FrameAttribute, 1);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeZ(const FAnimDatabaseFrameAttribute& Vector)
{
	if (!Vector.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeZ: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (Vector.Type != EAnimDatabaseAttributeType::Location &&
		Vector.Type != EAnimDatabaseAttributeType::Rotation &&
		Vector.Type != EAnimDatabaseAttributeType::Scale &&
		Vector.Type != EAnimDatabaseAttributeType::LinearVelocity &&
		Vector.Type != EAnimDatabaseAttributeType::AngularVelocity &&
		Vector.Type != EAnimDatabaseAttributeType::ScalarVelocity)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeZ: FrameAttribute Types don't match. Got {Type}. Expected Location, Rotation, Scale, LinearVelocity, AngularVelocity, or ScalarVelocity.", AttributeTypeNameInternal(Vector.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Float;
	UE::Learning::FrameAttribute::Index(*Out.FrameAttribute, *Vector.FrameAttribute, 2);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeW(const FAnimDatabaseFrameAttribute& Vector)
{
	if (!Vector.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeW: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (Vector.Type != EAnimDatabaseAttributeType::Rotation)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeW: FrameAttribute Types don't match. Got {Type}. Expected Rotation.", AttributeTypeNameInternal(Vector.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Float;
	UE::Learning::FrameAttribute::Index(*Out.FrameAttribute, *Vector.FrameAttribute, 3);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeYaw(const FAnimDatabaseFrameAttribute& A)
{
	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeYaw: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type != EAnimDatabaseAttributeType::Rotation)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeYaw: FrameAttribute Types don't match. Got {Type}. Expected Rotation.", AttributeTypeNameInternal(A.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Angle;
	UE::Learning::FrameAttribute::QuatYaw(*Out.FrameAttribute, *A.FrameAttribute);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributePitch(const FAnimDatabaseFrameAttribute& A)
{
	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributePitch: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type != EAnimDatabaseAttributeType::Rotation)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributePitch: FrameAttribute Types don't match. Got {Type}. Expected Rotation.", AttributeTypeNameInternal(A.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Angle;
	UE::Learning::FrameAttribute::QuatPitch(*Out.FrameAttribute, *A.FrameAttribute);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeRoll(const FAnimDatabaseFrameAttribute& A)
{
	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeRoll: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type != EAnimDatabaseAttributeType::Rotation)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeRoll: FrameAttribute Types don't match. Got {Type}. Expected Rotation.", AttributeTypeNameInternal(A.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Angle;
	UE::Learning::FrameAttribute::QuatRoll(*Out.FrameAttribute, *A.FrameAttribute);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeTransformLocation(const FAnimDatabaseFrameAttribute& Transform)
{
	if (!Transform.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeTransformLocation: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (Transform.Type != EAnimDatabaseAttributeType::Transform)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeTransformLocation: FrameAttribute Types don't match. Got {Type}. Expected Transform.", AttributeTypeNameInternal(Transform.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Location;
	UE::Learning::FrameAttribute::TransformLocation(*Out.FrameAttribute, *Transform.FrameAttribute);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeTransformRotation(const FAnimDatabaseFrameAttribute& Transform)
{
	if (!Transform.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeTransformRotation: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (Transform.Type != EAnimDatabaseAttributeType::Transform)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeTransformRotation: FrameAttribute Types don't match. Got {Type}. Expected Transform.", AttributeTypeNameInternal(Transform.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Rotation;
	UE::Learning::FrameAttribute::TransformRotation(*Out.FrameAttribute, *Transform.FrameAttribute);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeTransformScale(const FAnimDatabaseFrameAttribute& Transform)
{
	if (!Transform.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeTransformScale: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (Transform.Type != EAnimDatabaseAttributeType::Transform)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeTransformScale: FrameAttribute Types don't match. Got {Type}. Expected Transform.", AttributeTypeNameInternal(Transform.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Scale;
	UE::Learning::FrameAttribute::TransformScale(*Out.FrameAttribute, *Transform.FrameAttribute);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeEqual(FAnimDatabaseFrameAttribute A, FAnimDatabaseFrameAttribute B)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeEqual);

	if (!A.IsValid() || !B.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeEqual: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type != B.Type)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeEqual: FrameAttribute Types don't match. Got {Type} and {Type}.", AttributeTypeNameInternal(A.Type), AttributeTypeNameInternal(B.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Bool;
	UE::Learning::FrameAttribute::Eq(*Out.FrameAttribute, *A.FrameAttribute, *B.FrameAttribute);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeEqualTransform(const FAnimDatabaseFrameAttribute& TransformFrameAttribute, const FTransform& Transform)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeEqualTransform);

	if (!TransformFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeEqualTransform: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (TransformFrameAttribute.Type != EAnimDatabaseAttributeType::Transform)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeEqualTransform: FrameAttribute Types don't match. Got {Type}. Expected Transform.", AttributeTypeNameInternal(TransformFrameAttribute.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Bool;
	UE::Learning::FrameAttribute::EqConstant(*Out.FrameAttribute, *TransformFrameAttribute.FrameAttribute, 
		{
			(float)Transform.GetLocation().X, (float)Transform.GetLocation().Y, (float)Transform.GetLocation().Z,
			(float)Transform.GetRotation().X, (float)Transform.GetRotation().Y, (float)Transform.GetRotation().Z, (float)Transform.GetRotation().W,
			(float)Transform.GetScale3D().X, (float)Transform.GetScale3D().Y, (float)Transform.GetScale3D().Z,
		});
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeGreaterThanFloat(const FAnimDatabaseFrameAttribute& FloatFrameAttribute, const float Threshold)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeGreaterThanFloat);

	if (!FloatFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeGreaterThanFloat: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (FloatFrameAttribute.Type != EAnimDatabaseAttributeType::Float)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeGreaterThanFloat: FrameAttribute Types don't match. Got {Type}. Expected Float.", AttributeTypeNameInternal(FloatFrameAttribute.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Bool;
	UE::Learning::FrameAttribute::GtConstant(*Out.FrameAttribute, *FloatFrameAttribute.FrameAttribute, { Threshold });
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeLessThanFloat(const FAnimDatabaseFrameAttribute& FloatFrameAttribute, const float Threshold)
{
	if (!FloatFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeLessThanFloat: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (FloatFrameAttribute.Type != EAnimDatabaseAttributeType::Float)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeLessThanFloat: FrameAttribute Types don't match. Got {Type}. Expected Float.", AttributeTypeNameInternal(FloatFrameAttribute.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Bool;
	UE::Learning::FrameAttribute::LtConstant(*Out.FrameAttribute, *FloatFrameAttribute.FrameAttribute, { Threshold });
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeGreaterThanAngleDegrees(const FAnimDatabaseFrameAttribute& AngleFrameAttribute, const float Threshold)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeGreaterThanAngleDegrees);

	if (!AngleFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeGreaterThanAngleDegrees: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (AngleFrameAttribute.Type != EAnimDatabaseAttributeType::Angle)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeGreaterThanAngleDegrees: FrameAttribute Types don't match. Got {Type}. Expected Angle.", AttributeTypeNameInternal(AngleFrameAttribute.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Bool;
	UE::Learning::FrameAttribute::GtConstant(*Out.FrameAttribute, *AngleFrameAttribute.FrameAttribute, { FMath::DegreesToRadians(Threshold) });
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeLessThanAngleDegrees(const FAnimDatabaseFrameAttribute& AngleFrameAttribute, const float Threshold)
{
	if (!AngleFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeLessThanAngleDegrees: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (AngleFrameAttribute.Type != EAnimDatabaseAttributeType::Angle)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeLessThanAngleDegrees: FrameAttribute Types don't match. Got {Type}. Expected Angle.", AttributeTypeNameInternal(AngleFrameAttribute.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Bool;
	UE::Learning::FrameAttribute::LtConstant(*Out.FrameAttribute, *AngleFrameAttribute.FrameAttribute, { FMath::DegreesToRadians(Threshold) });
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeGreaterThanAngleRadians(const FAnimDatabaseFrameAttribute& AngleFrameAttribute, const float Threshold)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeGreaterThanAngleRadians);

	if (!AngleFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeGreaterThanAngleRadians: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (AngleFrameAttribute.Type != EAnimDatabaseAttributeType::Angle)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeGreaterThanAngleRadians: FrameAttribute Types don't match. Got {Type}. Expected Angle.", AttributeTypeNameInternal(AngleFrameAttribute.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Bool;
	UE::Learning::FrameAttribute::GtConstant(*Out.FrameAttribute, *AngleFrameAttribute.FrameAttribute, { Threshold });
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeLessThanAngleRadians(const FAnimDatabaseFrameAttribute& AngleFrameAttribute, const float Threshold)
{
	if (!AngleFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeLessThanAngleRadians: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (AngleFrameAttribute.Type != EAnimDatabaseAttributeType::Angle)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeLessThanAngleRadians: FrameAttribute Types don't match. Got {Type}. Expected Angle.", AttributeTypeNameInternal(AngleFrameAttribute.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Bool;
	UE::Learning::FrameAttribute::LtConstant(*Out.FrameAttribute, *AngleFrameAttribute.FrameAttribute, { Threshold });
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeBoolToFloat(const FAnimDatabaseFrameAttribute& BoolFrameAttribute)
{
	if (!BoolFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeBoolToFloat: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (BoolFrameAttribute.Type != EAnimDatabaseAttributeType::Bool)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeBoolToFloat: FrameAttribute Types don't match. Got {Type}. Expected Bool.", AttributeTypeNameInternal(BoolFrameAttribute.Type));
		return FAnimDatabaseFrameAttribute();
	}

	// We can just return it directly with a different type the underlying data doesn't need changing
	FAnimDatabaseFrameAttribute Out = BoolFrameAttribute;
	Out.Type = EAnimDatabaseAttributeType::Float;
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeAngleToFloatDegrees(const FAnimDatabaseFrameAttribute& AngleFrameAttribute)
{
	if (!AngleFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeAngleToFloatDegrees: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (AngleFrameAttribute.Type != EAnimDatabaseAttributeType::Angle)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeAngleToFloatDegrees: FrameAttribute Types don't match. Got {Type}. Expected Angle.", AttributeTypeNameInternal(AngleFrameAttribute.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = FrameAttributeCopy(AngleFrameAttribute);
	Out.Type = EAnimDatabaseAttributeType::Float;
	UE::Learning::FrameAttribute::MulConstantInplace(*Out.FrameAttribute, { 180.0f / UE_PI });
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeAngleToFloatRadians(const FAnimDatabaseFrameAttribute& AngleFrameAttribute)
{
	if (!AngleFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeAngleToFloatRadians: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (AngleFrameAttribute.Type != EAnimDatabaseAttributeType::Angle)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeAngleToFloatRadians: FrameAttribute Types don't match. Got {Type}. Expected Angle.", AttributeTypeNameInternal(AngleFrameAttribute.Type));
		return FAnimDatabaseFrameAttribute();
	}

	// We can just return it directly with a different type the underlying data doesn't need changing
	FAnimDatabaseFrameAttribute Out = AngleFrameAttribute;
	Out.Type = EAnimDatabaseAttributeType::Float;
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeFloatToBool(const FAnimDatabaseFrameAttribute& A)
{
	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeFloatToBool: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type != EAnimDatabaseAttributeType::Float)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeFloatToBool: FrameAttribute Types don't match. Got {Type}. Expected Float.", AttributeTypeNameInternal(A.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Bool;
	UE::Learning::FrameAttribute::NeqConstant(*Out.FrameAttribute, *A.FrameAttribute, { 0.0f });
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeFloatToAngleDegrees(const FAnimDatabaseFrameAttribute& FloatFrameAttribute)
{
	if (!FloatFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeFloatToAngleDegrees: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (FloatFrameAttribute.Type != EAnimDatabaseAttributeType::Float)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeFloatToAngleDegrees: FrameAttribute Types don't match. Got {Type}. Expected Float.", AttributeTypeNameInternal(FloatFrameAttribute.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = FrameAttributeCopy(FloatFrameAttribute);
	Out.Type = EAnimDatabaseAttributeType::Angle;
	UE::Learning::FrameAttribute::MulConstantInplace(*Out.FrameAttribute, { UE_PI / 180.0f });
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeFloatToAngleRadians(const FAnimDatabaseFrameAttribute& FloatFrameAttribute)
{
	if (!FloatFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeFloatToAngleRadians: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (FloatFrameAttribute.Type != EAnimDatabaseAttributeType::Float)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeFloatToAngleRadians: FrameAttribute Types don't match. Got {Type}. Expected Float.", AttributeTypeNameInternal(FloatFrameAttribute.Type));
		return FAnimDatabaseFrameAttribute();
	}

	// We can just return it directly with a different type the underlying data doesn't need changing
	FAnimDatabaseFrameAttribute Out = FloatFrameAttribute;
	Out.Type = EAnimDatabaseAttributeType::Angle;
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeDirectionToLocation(const FAnimDatabaseFrameAttribute& DirectionFrameAttribute)
{
	if (!DirectionFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeDirectionToLocation: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (DirectionFrameAttribute.Type != EAnimDatabaseAttributeType::Direction)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeDirectionToLocation: FrameAttribute Types don't match. Got {Type}. Expected Direction.", AttributeTypeNameInternal(DirectionFrameAttribute.Type));
		return FAnimDatabaseFrameAttribute();
	}

	// We can just return it directly with a different type the underlying data doesn't need changing
	FAnimDatabaseFrameAttribute Out = DirectionFrameAttribute;
	Out.Type = EAnimDatabaseAttributeType::Location;
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeLocationToDirection(const FAnimDatabaseFrameAttribute& LocationFrameAttribute)
{
	if (!LocationFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeLocationToDirection: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (LocationFrameAttribute.Type != EAnimDatabaseAttributeType::Location)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeLocationToDirection: FrameAttribute Types don't match. Got {Type}. Expected Location.", AttributeTypeNameInternal(LocationFrameAttribute.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = FrameAttributeCopy(LocationFrameAttribute);
	Out.Type = EAnimDatabaseAttributeType::Direction;
	UE::Learning::FrameAttribute::NormalizeInplace(*Out.FrameAttribute);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeVelocityToDirection(const FAnimDatabaseFrameAttribute& VelocityFrameAttribute)
{
	if (!VelocityFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeVelocityToDirection: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (VelocityFrameAttribute.Type != EAnimDatabaseAttributeType::LinearVelocity &&
		VelocityFrameAttribute.Type != EAnimDatabaseAttributeType::AngularVelocity)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeVelocityToDirection: FrameAttribute Types don't match. Got {Type}. Expected LinearVelocity or AngularVelocity.", AttributeTypeNameInternal(VelocityFrameAttribute.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = FrameAttributeCopy(VelocityFrameAttribute);
	Out.Type = EAnimDatabaseAttributeType::Direction;
	UE::Learning::FrameAttribute::NormalizeInplace(*Out.FrameAttribute);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeExtractPhase(const FAnimDatabaseFrameRanges& FrameRanges, const FAnimDatabaseFrames& ZeroPhaseFrames, const FAnimDatabaseFrames& HalfPhaseFrames, const EAnimDatabasePhaseExtrapolationMode ExtrapolationMode)
{
	if (!FrameRanges.IsValid() || !ZeroPhaseFrames.IsValid() || !HalfPhaseFrames.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeExtractPhase: Invalid FrameRanges or Frames.");
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Angle;
	UE::Learning::FrameAttribute::ExtractPhase(*Out.FrameAttribute, *FrameRanges.FrameRangeSet, *ZeroPhaseFrames.FrameSet, *HalfPhaseFrames.FrameSet,
		ExtrapolationMode == EAnimDatabasePhaseExtrapolationMode::Extrapolate ? UE::Learning::FrameAttribute::EPhaseExtrapolationMode::Extrapolate :
		ExtrapolationMode == EAnimDatabasePhaseExtrapolationMode::Repeat ? UE::Learning::FrameAttribute::EPhaseExtrapolationMode::Repeat : UE::Learning::FrameAttribute::EPhaseExtrapolationMode::Repeat);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeFilterGaussian(const FAnimDatabaseFrameAttribute& A, const float StandardDeviationInFrames)
{
	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeFilterGaussian: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type != EAnimDatabaseAttributeType::Float && 
		A.Type != EAnimDatabaseAttributeType::Bool &&
		A.Type != EAnimDatabaseAttributeType::Location && 
		A.Type != EAnimDatabaseAttributeType::Direction)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeFilterGaussian: FrameAttribute Types don't match. Got {Type}. Expected Float, Bool, Location, or Direction.", AttributeTypeNameInternal(A.Type));
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type == EAnimDatabaseAttributeType::Float || A.Type == EAnimDatabaseAttributeType::Bool)
	{
		FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
		Out.Type = EAnimDatabaseAttributeType::Float;
		UE::Learning::FrameAttribute::FilterGaussian(*Out.FrameAttribute, *A.FrameAttribute, StandardDeviationInFrames);
		return Out;
	}
	else if (A.Type == EAnimDatabaseAttributeType::Location)
	{
		FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
		Out.Type = EAnimDatabaseAttributeType::Location;
		UE::Learning::FrameAttribute::FilterGaussian(*Out.FrameAttribute, *A.FrameAttribute, StandardDeviationInFrames);
		return Out;
	}
	else if (A.Type == EAnimDatabaseAttributeType::Direction)
	{
		FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
		Out.Type = EAnimDatabaseAttributeType::Direction;
		UE::Learning::FFrameAttribute Tmp;
		UE::Learning::FrameAttribute::FilterGaussian(Tmp, *A.FrameAttribute, StandardDeviationInFrames);
		UE::Learning::FrameAttribute::Normalize(*Out.FrameAttribute, Tmp);
		return Out;
	}
	else
	{
		checkNoEntry();
		return FAnimDatabaseFrameAttribute();
	}
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeFilterSavGol(const FAnimDatabaseFrameAttribute& A, const int32 FilterWidthInFrames, const int32 PolynomialDegree, const bool bGaussianWindowed)
{
	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeFilterSavGol: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type != EAnimDatabaseAttributeType::Float && A.Type != EAnimDatabaseAttributeType::Location && A.Type != EAnimDatabaseAttributeType::Direction)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeFilterSavGol: FrameAttribute Types don't match. Got {Type}. Expected Float, Location, or Direction.", AttributeTypeNameInternal(A.Type));
		return FAnimDatabaseFrameAttribute();
	}

	if (FilterWidthInFrames < 0 || PolynomialDegree < 1)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeFilterSavGol: Invalid Settings.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type == EAnimDatabaseAttributeType::Float)
	{
		FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
		Out.Type = EAnimDatabaseAttributeType::Float;
		UE::Learning::FrameAttribute::FilterSavGol(*Out.FrameAttribute, *A.FrameAttribute, FilterWidthInFrames, PolynomialDegree, bGaussianWindowed);
		return Out;
	}
	else if (A.Type == EAnimDatabaseAttributeType::Location)
	{
		FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
		Out.Type = EAnimDatabaseAttributeType::Location;
		UE::Learning::FrameAttribute::FilterSavGol(*Out.FrameAttribute, *A.FrameAttribute, FilterWidthInFrames, PolynomialDegree, bGaussianWindowed);
		return Out;
	}
	else if (A.Type == EAnimDatabaseAttributeType::Direction)
	{
		FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
		Out.Type = EAnimDatabaseAttributeType::Direction;
		UE::Learning::FFrameAttribute Tmp;
		UE::Learning::FrameAttribute::FilterSavGol(Tmp, *A.FrameAttribute, FilterWidthInFrames, PolynomialDegree, bGaussianWindowed);
		UE::Learning::FrameAttribute::Normalize(*Out.FrameAttribute, Tmp);
		return Out;
	}
	else
	{
		checkNoEntry();
		return FAnimDatabaseFrameAttribute();
	}
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeFilterMajorityVote(const FAnimDatabaseFrameAttribute& A, const int32 FilterWidthInFrames)
{
	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeFilterMajorityVote: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type != EAnimDatabaseAttributeType::Bool)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeFilterMajorityVote: FrameAttribute Types don't match. Got {Type}. Expected Bool.", AttributeTypeNameInternal(A.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Bool;
	UE::Learning::FrameAttribute::FilterMajorityVote(*Out.FrameAttribute, *A.FrameAttribute, FilterWidthInFrames);
	return Out;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameAttributeLibrary::FrameAttributeBoolToFrameRanges(const FAnimDatabaseFrameAttribute& BoolFrameAttribute)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeBoolToFrameRanges);

	if (!BoolFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeBoolToFrameRanges: Invalid FrameAttribute.");
		return FAnimDatabaseFrameRanges();
	}

	if (BoolFrameAttribute.Type != EAnimDatabaseAttributeType::Bool)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeBoolToFrameRanges: FrameAttribute Types don't match. Got {Type}. Expected Bool.", AttributeTypeNameInternal(BoolFrameAttribute.Type));
		return FAnimDatabaseFrameRanges();
	}

	FAnimDatabaseFrameRanges Out = UAnimDatabaseFrameRangesLibrary::MakeEmptyFrameRanges();
	UE::Learning::FrameAttribute::NonZeroFrameRangeSet(*Out.FrameRangeSet, *BoolFrameAttribute.FrameAttribute, 0);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeFrameRangesToBool(const FAnimDatabaseFrameRanges& ActiveFrameRanges, const FAnimDatabaseFrameRanges& InFrameRanges)
{
	return MakeBoolFrameAttributeFromActiveRanges(ActiveFrameRanges, InFrameRanges);
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeSelect(const FAnimDatabaseFrameAttribute& Cond, const FAnimDatabaseFrameAttribute& True, const FAnimDatabaseFrameAttribute& False)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeSelect);

	if (!Cond.IsValid() || !True.IsValid() || !False.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeSelect: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (Cond.Type != EAnimDatabaseAttributeType::Bool)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeSelect: FrameAttribute Types don't match. Got {Type}. Expected Bool.", AttributeTypeNameInternal(Cond.Type));
		return FAnimDatabaseFrameAttribute();
	}

	if (True.Type != False.Type)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeSelect: FrameAttribute Types don't match. Got {Type0} and {Type1}.", AttributeTypeNameInternal(True.Type), AttributeTypeNameInternal(False.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = UAnimDatabaseFrameAttributeLibrary::MakeEmptyFrameAttribute();
	Out.Type = True.Type;
	UE::Learning::FrameAttribute::Select(*Out.FrameAttribute, *Cond.FrameAttribute, *True.FrameAttribute, *False.FrameAttribute);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeRepeatFirstInRange(const FAnimDatabaseFrameAttribute& FrameAttribute)
{
	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = FrameAttribute.Type;
	UE::Learning::FrameAttribute::Copy(*Out.FrameAttribute, *FrameAttribute.FrameAttribute);
	UE::Learning::FrameAttribute::RepeatFirstRangeEntryInplace(*Out.FrameAttribute);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeNot(const FAnimDatabaseFrameAttribute& A)
{
	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeNot: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type != EAnimDatabaseAttributeType::Bool)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeNot: FrameAttribute Types don't match. Got {Type}. Expected Bool.", AttributeTypeNameInternal(A.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Bool;
	UE::Learning::FrameAttribute::LogicalNot(*Out.FrameAttribute, *A.FrameAttribute);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeAnd(FAnimDatabaseFrameAttribute A, FAnimDatabaseFrameAttribute B)
{
	if (!A.IsValid() || !B.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeAnd: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type != EAnimDatabaseAttributeType::Bool || B.Type != EAnimDatabaseAttributeType::Bool)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeAnd: FrameAttribute Types don't match. Got {Type0} and {Type1}. Expected Bool.", AttributeTypeNameInternal(A.Type), AttributeTypeNameInternal(B.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Bool;
	UE::Learning::FrameAttribute::LogicalAnd(*Out.FrameAttribute, *A.FrameAttribute, *B.FrameAttribute);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeOr(FAnimDatabaseFrameAttribute A, FAnimDatabaseFrameAttribute B)
{
	if (!A.IsValid() || !B.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeOr: Invalid FrameAttribute.");
		return FAnimDatabaseFrameAttribute();
	}

	if (A.Type != EAnimDatabaseAttributeType::Bool || B.Type != EAnimDatabaseAttributeType::Bool)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeOr: FrameAttribute Types don't match. Got {Type0} and {Type1}. Expected Bool.", AttributeTypeNameInternal(A.Type), AttributeTypeNameInternal(B.Type));
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Out = MakeEmptyFrameAttribute();
	Out.Type = EAnimDatabaseAttributeType::Bool;
	UE::Learning::FrameAttribute::LogicalOr(*Out.FrameAttribute, *A.FrameAttribute, *B.FrameAttribute);
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeSum(const TArray<FAnimDatabaseFrameAttribute>& FrameAttributes)
{
	return FrameAttributeSumFromArrayView(FrameAttributes);
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeSumFromArrayView(const TArrayView<const FAnimDatabaseFrameAttribute> FrameAttributes)
{
	const int32 FrameAttributeNum = FrameAttributes.Num();

	if (FrameAttributeNum == 0)
	{
		UE_LOGFMT(LogAnimDatabase, Warning, "FrameAttributeSumFromArrayView: No FrameAttributes Provided.");
		return FAnimDatabaseFrameAttribute();
	}

	if (FrameAttributeNum == 1)
	{
		return FrameAttributes[0];
	}

	// TODO: Optimize this properly
	FAnimDatabaseFrameAttribute Out = FrameAttributes[0];
	for (int32 FrameAttributeIdx = 1; FrameAttributeIdx < FrameAttributeNum; FrameAttributeIdx++)
	{
		Out = FrameAttributeAdd(Out, FrameAttributes[FrameAttributeIdx]);
	}
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeWeightedSum(const TArray<FAnimDatabaseFrameAttribute>& FrameAttributes, const TArray<float>& Weights)
{
	return FrameAttributeWeightedSumFromArrayView(FrameAttributes, Weights);
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeWeightedSumFromArrayView(const TArrayView<const FAnimDatabaseFrameAttribute> FrameAttributes, const TArrayView<const float> Weights)
{
	const int32 FrameAttributeNum = FrameAttributes.Num();
	const int32 WeightNum = Weights.Num();

	if (FrameAttributeNum == 0)
	{
		UE_LOGFMT(LogAnimDatabase, Warning, "FrameAttributeWeightedSumFromArrayView: No FrameAttributes Provided.");
		return FAnimDatabaseFrameAttribute();
	}

	if (WeightNum != FrameAttributeNum)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeWeightedSumFromArrayView: Number of Frame Attributes {NumAtr} does not match number of weights {NumWeight}.",
			FrameAttributeNum, WeightNum);
		return FAnimDatabaseFrameAttribute();
	}

	if (FrameAttributeNum == 1)
	{
		return FrameAttributeMultiplyFloat(FrameAttributes[0], Weights[0]);
	}

	// TODO: Optimize this properly
	FAnimDatabaseFrameAttribute Out = FrameAttributeMultiplyFloat(FrameAttributes[0], Weights[0]);
	for (int32 FrameAttributeIdx = 1; FrameAttributeIdx < FrameAttributeNum; FrameAttributeIdx++)
	{
		Out = FrameAttributeAdd(Out, FrameAttributeMultiplyFloat(FrameAttributes[FrameAttributeIdx], Weights[FrameAttributeIdx]));
	}
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeDynamicWeightedSum(const TArray<FAnimDatabaseFrameAttribute>& FrameAttributes, const TArray<FAnimDatabaseFrameAttribute>& Weights)
{
	return FrameAttributeDynamicWeightedSumFromArrayView(FrameAttributes, Weights);
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeDynamicWeightedSumFromArrayView(const TArrayView<const FAnimDatabaseFrameAttribute> FrameAttributes, const TArrayView<const FAnimDatabaseFrameAttribute> Weights)
{
	const int32 FrameAttributeNum = FrameAttributes.Num();
	const int32 WeightNum = Weights.Num();

	if (FrameAttributeNum == 0)
	{
		UE_LOGFMT(LogAnimDatabase, Warning, "FrameAttributeDynamicWeightedSumFromArrayView: No FrameAttributes Provided.");
		return FAnimDatabaseFrameAttribute();
	}

	if (WeightNum != FrameAttributeNum)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeDynamicWeightedSumFromArrayView: Number of Frame Attributes {NumAtr} does not match number of weights {NumWeight}.",
			FrameAttributeNum, WeightNum);
		return FAnimDatabaseFrameAttribute();
	}

	if (FrameAttributeNum == 1)
	{
		return FrameAttributeMultiply(FrameAttributes[0], Weights[0]);
	}

	// TODO: Optimize this properly
	FAnimDatabaseFrameAttribute Out = FrameAttributeMultiply(FrameAttributes[0], Weights[0]);
	for (int32 FrameAttributeIdx = 1; FrameAttributeIdx < FrameAttributeNum; FrameAttributeIdx++)
	{
		Out = FrameAttributeAdd(Out, FrameAttributeMultiply(FrameAttributes[FrameAttributeIdx], Weights[FrameAttributeIdx]));
	}
	return Out;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeProduct(const TArray<FAnimDatabaseFrameAttribute>& FrameAttributes)
{
	return FrameAttributeProductFromArrayView(FrameAttributes);
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeProductFromArrayView(const TArrayView<const FAnimDatabaseFrameAttribute> FrameAttributes)
{
	const int32 FrameAttributeNum = FrameAttributes.Num();

	if (FrameAttributeNum == 0)
	{
		UE_LOGFMT(LogAnimDatabase, Warning, "FrameAttributeProductFromArrayView: No FrameAttributes Provided.");
		return FAnimDatabaseFrameAttribute();
	}

	if (FrameAttributeNum == 1)
	{
		return FrameAttributes[0];
	}

	// TODO: Optimize this properly
	FAnimDatabaseFrameAttribute Out = FrameAttributes[0];
	for (int32 FrameAttributeIdx = 1; FrameAttributeIdx < FrameAttributeNum; FrameAttributeIdx++)
	{
		Out = FrameAttributeMultiply(Out, FrameAttributes[FrameAttributeIdx]);
	}
	return Out;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameAttributeLibrary::FrameAttributeFrameRanges(const FAnimDatabaseFrameAttribute& FrameAttribute)
{
	if (!FrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeFrameRanges: Invalid FrameAttribute.");
		return FAnimDatabaseFrameRanges();
	}

	FAnimDatabaseFrameRanges FrameRanges;
	FrameRanges.FrameRangeSet = MakeShared<UE::Learning::FFrameRangeSet>(FrameAttribute.FrameAttribute->FrameRangeSet);
	return FrameRanges;
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributeAnimSequences(TArray<UAnimSequence*>& OutAnimSequences, const UAnimDatabase* Database, const FAnimDatabaseFrameAttribute& FrameAttribute)
{
	if (!FrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeAnimSequences: Invalid FrameRanges.");
		OutAnimSequences.Empty();
		return;
	}

	const int32 EntryNum = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryNum();

	OutAnimSequences.Empty(EntryNum);
	for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
	{
		OutAnimSequences.AddUnique(Database->GetAnimSequence(FrameAttribute.FrameAttribute->FrameRangeSet.GetEntrySequence(EntryIdx)));
	}
}

bool UAnimDatabaseFrameAttributeLibrary::FrameAttributeFrameRangesEqual(const FAnimDatabaseFrameAttribute& A, const FAnimDatabaseFrameAttribute& B)
{
	if (!A.IsValid() || !B.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeFrameRangesEqual: Invalid FrameAttribute.");
		return false;
	}

	return UE::Learning::FrameRangeSet::Equal(A.FrameAttribute->FrameRangeSet, B.FrameAttribute->FrameRangeSet);
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributeLocationMeanAndStd(FVector& OutMean, FVector& OutStd, const FAnimDatabaseFrameAttribute& LocationFrameAttribute)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeLocationMeanAndStd);

	if (!LocationFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeLocationMeanAndStd: Invalid FrameAttribute.");
		OutMean = FVector::ZeroVector;
		OutStd = FVector::ZeroVector;
		return;
	}

	if (LocationFrameAttribute.Type != EAnimDatabaseAttributeType::Location)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeLocationMeanAndStd: FrameAttribute Types don't match. Got {Type}. Expected Location.", AttributeTypeNameInternal(LocationFrameAttribute.Type));
		OutMean = FVector::ZeroVector;
		OutStd = FVector::ZeroVector;
		return;
	}

	TStaticArray<float, 3> Mean, Std;
	UE::Learning::FrameAttribute::MeanStd(Mean, Std, *LocationFrameAttribute.FrameAttribute);
	OutMean = FVector(Mean[0], Mean[1], Mean[2]);
	OutStd = FVector(Std[0], Std[1], Std[2]);
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributeFloatAverage(float& OutAverageFloat, const FAnimDatabaseFrameAttribute& FloatFrameAttribute)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeFloatAverage);

	if (!FloatFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeFloatAverage: Invalid FrameAttribute.");
		OutAverageFloat = 0.0f;
		return;
	}

	if (FloatFrameAttribute.Type != EAnimDatabaseAttributeType::Float)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeFloatAverage: FrameAttribute Types don't match. Got {Type}. Expected Float.", AttributeTypeNameInternal(FloatFrameAttribute.Type));
		OutAverageFloat = 0.0f;
		return;
	}

	UE::Learning::FrameAttribute::Mean(MakeArrayView(&OutAverageFloat, 1), *FloatFrameAttribute.FrameAttribute);
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributeAngleAverageDegrees(float& OutAverageAngle, const FAnimDatabaseFrameAttribute& AngleFrameAttribute)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeAngleAverageDegrees);

	if (!AngleFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeAngleAverageDegrees: Invalid FrameAttribute.");
		OutAverageAngle = 0.0f;
		return;
	}

	if (AngleFrameAttribute.Type != EAnimDatabaseAttributeType::Angle)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeAngleAverageDegrees: FrameAttribute Types don't match. Got {Type}. Expected Angle.", AttributeTypeNameInternal(AngleFrameAttribute.Type));
		OutAverageAngle = 0.0f;
		return;
	}

	UE::Learning::FrameAttribute::AngularMean(MakeArrayView(&OutAverageAngle, 1), *AngleFrameAttribute.FrameAttribute);
	OutAverageAngle = FMath::RadiansToDegrees(OutAverageAngle);
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributeAngleAverageRadians(float& OutAverageAngle, const FAnimDatabaseFrameAttribute& AngleFrameAttribute)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeAngleAverageRadians);

	if (!AngleFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeAngleAverageRadians: Invalid FrameAttribute.");
		OutAverageAngle = 0.0f;
		return;
	}

	if (AngleFrameAttribute.Type != EAnimDatabaseAttributeType::Angle)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeAngleAverageRadians: FrameAttribute Types don't match. Got {Type}. Expected Angle.", AttributeTypeNameInternal(AngleFrameAttribute.Type));
		OutAverageAngle = 0.0f;
		return;
	}

	UE::Learning::FrameAttribute::AngularMean(MakeArrayView(&OutAverageAngle, 1), *AngleFrameAttribute.FrameAttribute);
}


void UAnimDatabaseFrameAttributeLibrary::FrameAttributeFloatMinAndMax(float& OutMin, float& OutMax, const FAnimDatabaseFrameAttribute& FloatFrameAttribute)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeFloatMinAndMax);

	if (!FloatFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeFloatMinAndMax: Invalid FrameAttribute.");
		OutMin = 0.0f;
		OutMax = 0.0f;
		return;
	}

	if (FloatFrameAttribute.Type != EAnimDatabaseAttributeType::Float)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeFloatMinAndMax: FrameAttribute Types don't match. Got {Type}. Expected Float.", AttributeTypeNameInternal(FloatFrameAttribute.Type));
		OutMin = 0.0f;
		OutMax = 0.0f;
		return;
	}

	UE::Learning::FrameAttribute::MinMax(MakeArrayView(&OutMin, 1), MakeArrayView(&OutMax, 1), *FloatFrameAttribute.FrameAttribute);
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributeAngleMinAndMaxDegrees(float& OutMin, float& OutMax, const FAnimDatabaseFrameAttribute& AngleFrameAttribute)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeAngleMinAndMaxDegrees);

	if (!AngleFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeAngleMinAndMaxDegrees: Invalid FrameAttribute.");
		OutMin = 0.0f;
		OutMax = 0.0f;
		return;
	}

	if (AngleFrameAttribute.Type != EAnimDatabaseAttributeType::Angle)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeAngleMinAndMaxDegrees: FrameAttribute Types don't match. Got {Type}. Expected Angle.", AttributeTypeNameInternal(AngleFrameAttribute.Type));
		OutMin = 0.0f;
		OutMax = 0.0f;
		return;
	}

	UE::Learning::FrameAttribute::MinMax(MakeArrayView(&OutMin, 1), MakeArrayView(&OutMax, 1), *AngleFrameAttribute.FrameAttribute);
	OutMin = FMath::RadiansToDegrees(OutMin);
	OutMax = FMath::RadiansToDegrees(OutMax);
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributeAngleMinAndMaxRadians(float& OutMin, float& OutMax, const FAnimDatabaseFrameAttribute& AngleFrameAttribute)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeAngleMinAndMaxRadians);

	if (!AngleFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeAngleMinAndMaxRadians: Invalid FrameAttribute.");
		OutMin = 0.0f;
		OutMax = 0.0f;
		return;
	}

	if (AngleFrameAttribute.Type != EAnimDatabaseAttributeType::Angle)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeAngleMinAndMaxRadians: FrameAttribute Types don't match. Got {Type}. Expected Angle.", AttributeTypeNameInternal(AngleFrameAttribute.Type));
		OutMin = 0.0f;
		OutMax = 0.0f;
		return;
	}

	UE::Learning::FrameAttribute::MinMax(MakeArrayView(&OutMin, 1), MakeArrayView(&OutMax, 1), *AngleFrameAttribute.FrameAttribute);
}


void UAnimDatabaseFrameAttributeLibrary::FrameAttributeLocationAverage(FVector& OutAverageLocation, const FAnimDatabaseFrameAttribute& LocationFrameAttribute)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeLocationAverage);

	if (!LocationFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeLocationAverage: Invalid FrameAttribute.");
		OutAverageLocation = FVector::ZeroVector;
		return;
	}

	if (LocationFrameAttribute.Type != EAnimDatabaseAttributeType::Location)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeLocationAverage: FrameAttribute Types don't match. Got {Type}. Expected Location.", AttributeTypeNameInternal(LocationFrameAttribute.Type));
		OutAverageLocation = FVector::ZeroVector;
		return;
	}

	TStaticArray<float, 3> Mean;
	UE::Learning::FrameAttribute::Mean(Mean, *LocationFrameAttribute.FrameAttribute);
	OutAverageLocation = FVector(Mean[0], Mean[1], Mean[2]);
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributeDirectionAverage(FVector& OutAverageDirection, const FAnimDatabaseFrameAttribute& DirectionFrameAttribute)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeDirectionAverage);

	if (!DirectionFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeDirectionAverage: Invalid FrameAttribute.");
		OutAverageDirection = FVector::ForwardVector;
		return;
	}

	if (DirectionFrameAttribute.Type != EAnimDatabaseAttributeType::Direction)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeDirectionAverage: FrameAttribute Types don't match. Got {Type}. Expected Direction.", AttributeTypeNameInternal(DirectionFrameAttribute.Type));
		OutAverageDirection = FVector::ForwardVector;
		return;
	}

	TStaticArray<float, 3> Mean, Std;
	UE::Learning::FrameAttribute::MeanStd(Mean, Std, *DirectionFrameAttribute.FrameAttribute);
	OutAverageDirection = FVector(Mean[0], Mean[1], Mean[2]).GetSafeNormal(UE_SMALL_NUMBER, FVector::ForwardVector);
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributeRotationMeanAndStd(FRotator& OutMean, FVector& OutStd, const FAnimDatabaseFrameAttribute& RotationFrameAttribute)
{
	FQuat OutMeanQuat = FQuat::Identity;
	FrameAttributeRotationMeanAndStdAsQuat(OutMeanQuat, OutStd, RotationFrameAttribute);
	OutMean = OutMeanQuat.Rotator();
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributeRotationMeanAndStdAsQuat(FQuat& OutMean, FVector& OutStd, const FAnimDatabaseFrameAttribute& RotationFrameAttribute)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeRotationMeanAndStdAsQuat);

	if (!RotationFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeRotationMeanAndStdAsQuat: Invalid FrameAttribute.");
		OutMean = FQuat::Identity;
		OutStd = FVector::ZeroVector;
		return;
	}

	if (RotationFrameAttribute.Type != EAnimDatabaseAttributeType::Rotation)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeRotationMeanAndStdAsQuat: FrameAttribute Types don't match. Got {Type}. Expected Rotation.", AttributeTypeNameInternal(RotationFrameAttribute.Type));
		OutMean = FQuat::Identity;
		OutStd = FVector::ZeroVector;
		return;
	}

	FQuat4f Mean; FVector3f Std;
	UE::Learning::FrameAttribute::QuatMeanStd(Mean, Std, *RotationFrameAttribute.FrameAttribute);
	OutMean = ((FQuat)Mean).GetNormalized();
	OutStd = (FVector)Std;
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributeScaleMeanAndStd(FVector& OutMean, FVector& OutLogStd, const FAnimDatabaseFrameAttribute& ScaleFrameAttribute)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeScaleMeanAndStd);

	if (!ScaleFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeScaleMeanAndStd: Invalid FrameAttribute.");
		OutMean = FVector::OneVector;
		OutLogStd = FVector::ZeroVector;
		return;
	}

	if (ScaleFrameAttribute.Type != EAnimDatabaseAttributeType::Scale)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeScaleMeanAndStd: FrameAttribute Types don't match. Got {Type}. Expected Scale.", AttributeTypeNameInternal(ScaleFrameAttribute.Type));
		OutMean = FVector::OneVector;
		OutLogStd = FVector::ZeroVector;
		return;
	}

	TStaticArray<float, 3> Mean, LogStd;
	UE::Learning::FrameAttribute::LogMeanStd(Mean, LogStd, *ScaleFrameAttribute.FrameAttribute);
	OutMean = FVector(Mean[0], Mean[1], Mean[2]);
	OutLogStd = FVector(LogStd[0], LogStd[1], LogStd[2]);
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributeVelocityMeanAndStd(FVector& OutMean, FVector& OutStd, const FAnimDatabaseFrameAttribute& VelocityFrameAttribute)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeVelocityMeanAndStd);

	if (!VelocityFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeVelocityMeanAndStd: Invalid FrameAttribute.");
		OutMean = FVector::ZeroVector;
		OutStd = FVector::ZeroVector;
		return;
	}

	if (VelocityFrameAttribute.Type != EAnimDatabaseAttributeType::LinearVelocity &&
		VelocityFrameAttribute.Type != EAnimDatabaseAttributeType::AngularVelocity &&
		VelocityFrameAttribute.Type != EAnimDatabaseAttributeType::ScalarVelocity)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeVelocityMeanAndStd: FrameAttribute Types don't match. Got {Type}. Expected Velocity.", AttributeTypeNameInternal(VelocityFrameAttribute.Type));
		OutMean = FVector::ZeroVector;
		OutStd = FVector::ZeroVector;
		return;
	}

	TStaticArray<float, 3> Mean, Std;
	UE::Learning::FrameAttribute::MeanStd(Mean, Std, *VelocityFrameAttribute.FrameAttribute);
	OutMean = FVector(Mean[0], Mean[1], Mean[2]);
	OutStd = FVector(Std[0], Std[1], Std[2]);
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributeLocationDistanceTraveled(float& OutMean, float& OutStd, float& OutMin, float& OutMax, const FAnimDatabaseFrameAttribute& LocationFrameAttribute)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeLocationDistanceTraveled);

	if (!LocationFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeLocationDistanceTraveled: Invalid FrameAttribute.");
		OutMean = 0.0f;
		OutStd = 0.0f;
		OutMin = 0.0f;
		OutMax = 0.0f;
		return;
	}

	if (LocationFrameAttribute.Type != EAnimDatabaseAttributeType::Location)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeLocationDistanceTraveled: FrameAttribute Types don't match. Got {Type}. Expected Location.", AttributeTypeNameInternal(LocationFrameAttribute.Type));
		OutMean = 0.0f;
		OutStd = 0.0f;
		OutMin = 0.0f;
		OutMax = 0.0f;
		return;
	}

	const int32 RangeNum = LocationFrameAttribute.FrameAttribute->FrameRangeSet.GetTotalRangeNum();

	if (RangeNum == 0)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeLocationDistanceTraveled: No ranges in frame attribute.");
		OutMean = 0.0f;
		OutStd = 0.0f;
		OutMin = 0.0f;
		OutMax = 0.0f;
		return;
	}

	TArray<float> RangeDistances;
	RangeDistances.SetNumUninitialized(LocationFrameAttribute.FrameAttribute->FrameRangeSet.GetTotalRangeNum());
	UE::Learning::FrameAttribute::LocationDistanceTraveled(RangeDistances, *LocationFrameAttribute.FrameAttribute);
	UE::AnimDatabase::Math::ComputeMinMax(OutMin, OutMax, RangeDistances);
	UE::AnimDatabase::Math::ComputeMeanStd(OutMean, OutStd, RangeDistances);
}

void UAnimDatabaseFrameAttributeLibrary::FrameAttributeRotationAngleTraveled(float& OutMean, float& OutStd, float& OutMin, float& OutMax, const FAnimDatabaseFrameAttribute& RotationFrameAttribute)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameAttributeLibrary::FrameAttributeRotationAngleTraveled);

	if (!RotationFrameAttribute.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeRotationAngleTraveled: Invalid FrameAttribute.");
		OutMean = 0.0f;
		OutStd = 0.0f;
		OutMin = 0.0f;
		OutMax = 0.0f;
		return;
	}

	if (RotationFrameAttribute.Type != EAnimDatabaseAttributeType::Rotation)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeRotationAngleTraveled: FrameAttribute Types don't match. Got {Type}. Expected Rotation.", AttributeTypeNameInternal(RotationFrameAttribute.Type));
		OutMean = 0.0f;
		OutStd = 0.0f;
		OutMin = 0.0f;
		OutMax = 0.0f;
		return;
	}

	const int32 RangeNum = RotationFrameAttribute.FrameAttribute->FrameRangeSet.GetTotalRangeNum();

	if (RangeNum == 0)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeRotationAngleTraveled: No ranges in frame attribute.");
		OutMean = 0.0f;
		OutStd = 0.0f;
		OutMin = 0.0f;
		OutMax = 0.0f;
		return;
	}

	TArray<float> RangelAngles;
	RangelAngles.SetNumUninitialized(RangeNum);
	UE::Learning::FrameAttribute::QuatAngleTraveled(RangelAngles, *RotationFrameAttribute.FrameAttribute);
	UE::AnimDatabase::Math::ComputeMinMax(OutMin, OutMax, RangelAngles);
	UE::AnimDatabase::Math::ComputeMeanStd(OutMean, OutStd, RangelAngles);

	OutMin = FMath::RadiansToDegrees(OutMin);
	OutMax = FMath::RadiansToDegrees(OutMax);
	OutMean = FMath::RadiansToDegrees(OutMean);
	OutStd = FMath::RadiansToDegrees(OutStd);
}

FString UAnimDatabaseFrameAttributeLibrary::FrameAttributeToString(const FAnimDatabaseFrameAttribute& FrameAttribute)
{
	if (!FrameAttribute.IsValid()) { return TEXT("Invalid"); }

	return FString::Printf(TEXT("Type=%s EntryNum=%i TotalRangeNum=%i TotalFrameNum=%i"), 
		AttributeTypeNameInternal(FrameAttribute.Type),
		FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryNum(), 
		FrameAttribute.FrameAttribute->FrameRangeSet.GetTotalRangeNum(),
		FrameAttribute.FrameAttribute->FrameRangeSet.GetTotalFrameNum());
}


FString UAnimDatabaseFrameAttributeLibrary::FrameAttributeToStringFormat(const UAnimDatabase* Database, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 Cutoff)
{
	if (!FrameAttribute.IsValid()) { return TEXT("Invalid"); }

	const int32 RoundedCuttoff = (Cutoff / 2) * 2 + 1;

	FString Output = FString::Printf(TEXT("Type=%s EntryNum=%i TotalRangeNum=%i TotalFrameNum=%i Data=[\n"),
		AttributeTypeNameInternal(FrameAttribute.Type),
		FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryNum(),
		FrameAttribute.FrameAttribute->FrameRangeSet.GetTotalRangeNum(),
		FrameAttribute.FrameAttribute->FrameRangeSet.GetTotalFrameNum());

	const int32 EntryNum = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryNum();
	for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
	{
		const int32 SequenceIdx = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntrySequence(EntryIdx);
		const int32 RangeNum = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeNum(EntryIdx);

		if (Database)
		{
			Output += FString::Printf(TEXT("   %s%s [\n"), 
				*Database->GetSequenceAssetName(SequenceIdx), 
				Database->GetIsMirrored(SequenceIdx) ? TEXT(" (mirrored)") : TEXT(""));
		}
		else
		{
			Output += FString::Printf(TEXT("   Sequence %i [\n"), SequenceIdx);

		}

		if (Cutoff < 0 || RangeNum < RoundedCuttoff)
		{
			for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
			{
				Output += FString::Printf(TEXT("        %i:%i "),
					FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeStart(EntryIdx, RangeIdx),
					FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeStop(EntryIdx, RangeIdx));

				const int32 FrameNum = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeLength(EntryIdx, RangeIdx);
				const int32 FrameOffset = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeOffset(EntryIdx, RangeIdx);

				if (Cutoff < 0 || FrameNum < RoundedCuttoff)
				{
					Output += TEXT("[");
					for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
					{
						Output += UE::AnimDatabase::FrameAttribute::Private::FrameAttributeFrameToString(FrameAttribute, FrameOffset + FrameIdx);
						if (FrameIdx != FrameNum - 1) { Output += TEXT(" "); }
					}
					Output += TEXT("]\n");
				}
				else
				{
					Output += TEXT("[");
					for (int32 FrameIdx = 0; FrameIdx < RoundedCuttoff / 2; FrameIdx++)
					{
						Output += UE::AnimDatabase::FrameAttribute::Private::FrameAttributeFrameToString(FrameAttribute, FrameOffset + FrameIdx);
						Output += TEXT(" ");
					}

					Output += TEXT("... ");

					for (int32 FrameIdx = FrameNum - RoundedCuttoff / 2 - 1; FrameIdx < FrameNum; FrameIdx++)
					{
						Output += UE::AnimDatabase::FrameAttribute::Private::FrameAttributeFrameToString(FrameAttribute, FrameOffset + FrameIdx);
						if (FrameIdx != FrameNum - 1) { Output += TEXT(" "); }
					}

					Output += TEXT("]\n");
				}
			}
		}
		else
		{
			for (int32 RangeIdx = 0; RangeIdx < RoundedCuttoff / 2; RangeIdx++)
			{
				Output += FString::Printf(TEXT("        %i:%i "),
					FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeStart(EntryIdx, RangeIdx),
					FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeStop(EntryIdx, RangeIdx));

				const int32 FrameNum = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeLength(EntryIdx, RangeIdx);
				const int32 FrameOffset = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeOffset(EntryIdx, RangeIdx);

				if (FrameNum < RoundedCuttoff)
				{
					Output += TEXT("[");
					for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
					{
						Output += UE::AnimDatabase::FrameAttribute::Private::FrameAttributeFrameToString(FrameAttribute, FrameOffset + FrameIdx);
						if (FrameIdx != FrameNum - 1) { Output += TEXT(" "); }
					}
					Output += TEXT("]\n");
				}
				else
				{
					Output += TEXT("[");
					for (int32 FrameIdx = 0; FrameIdx < RoundedCuttoff / 2; FrameIdx++)
					{
						Output += UE::AnimDatabase::FrameAttribute::Private::FrameAttributeFrameToString(FrameAttribute, FrameOffset + FrameIdx);
						Output += TEXT(" ");
					}

					Output += TEXT("... ");

					for (int32 FrameIdx = FrameNum - RoundedCuttoff / 2 - 1; FrameIdx < FrameNum; FrameIdx++)
					{
						Output += UE::AnimDatabase::FrameAttribute::Private::FrameAttributeFrameToString(FrameAttribute, FrameOffset + FrameIdx);
						if (FrameIdx != FrameNum - 1) { Output += TEXT(" "); }
					}

					Output += TEXT("]\n");
				}
			}

			Output += TEXT("    ...\n");

			for (int32 RangeIdx = RangeNum - RoundedCuttoff / 2 - 1; RangeIdx < RangeNum; RangeIdx++)
			{
				Output += FString::Printf(TEXT("        %i:%i "),
					FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeStart(EntryIdx, RangeIdx),
					FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeStop(EntryIdx, RangeIdx));

				const int32 FrameNum = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeLength(EntryIdx, RangeIdx);
				const int32 FrameOffset = FrameAttribute.FrameAttribute->FrameRangeSet.GetEntryRangeOffset(EntryIdx, RangeIdx);

				if (FrameNum < RoundedCuttoff)
				{
					Output += TEXT("[");
					for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
					{
						Output += UE::AnimDatabase::FrameAttribute::Private::FrameAttributeFrameToString(FrameAttribute, FrameOffset + FrameIdx);
						if (FrameIdx != FrameNum - 1) { Output += TEXT(" "); }
					}
					Output += TEXT("]\n");
				}
				else
				{
					Output += TEXT("[");
					for (int32 FrameIdx = 0; FrameIdx < RoundedCuttoff / 2; FrameIdx++)
					{
						Output += UE::AnimDatabase::FrameAttribute::Private::FrameAttributeFrameToString(FrameAttribute, FrameOffset + FrameIdx);
						Output += TEXT(" ");
					}

					Output += TEXT("... ");

					for (int32 FrameIdx = FrameNum - RoundedCuttoff / 2 - 1; FrameIdx < FrameNum; FrameIdx++)
					{
						Output += UE::AnimDatabase::FrameAttribute::Private::FrameAttributeFrameToString(FrameAttribute, FrameOffset + FrameIdx);
						if (FrameIdx != FrameNum - 1) { Output += TEXT(" "); }
					}

					Output += TEXT("]\n");
				}
			}
		}
		Output += TEXT("    ]\n");
	}
	Output += TEXT("]");

	return Output;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameAttributeLibrary::FrameAttributeOutlierFrameRanges(
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const int32 Padding,
	const float Threshold,
	const float LocationMinimumDifference,
	const float RotationMinimumDifference,
	const float ScaleMinimumDifference,
	const float LinearVelocityMinimumDifference,
	const float AngularVelocityMinimumDifference,
	const float ScalarVelocityMinimumDifference)
{
	const int32 BoneNum = Database->GetBoneNum();

	TArray<int32> AllBoneIndices;
	for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
	{
		AllBoneIndices.Add(BoneIdx);
	}

	TArray<FAnimDatabaseFrameAttribute> BoneLocations;
	TArray<FAnimDatabaseFrameAttribute> BoneRotations;
	TArray<FAnimDatabaseFrameAttribute> BoneScales;
	TArray<FAnimDatabaseFrameAttribute> BoneLinearVelocities;
	TArray<FAnimDatabaseFrameAttribute> BoneAngularVelocities;
	TArray<FAnimDatabaseFrameAttribute> BoneScalarVelocities;

	MakeBoneLocalFrameAttributes(
		BoneLocations, BoneRotations, BoneScales,
		BoneLinearVelocities, BoneAngularVelocities, BoneScalarVelocities,
		Database,
		FrameRanges,
		AllBoneIndices, AllBoneIndices, AllBoneIndices,
		AllBoneIndices, AllBoneIndices, AllBoneIndices);

	TArray<FAnimDatabaseFrameRanges> BoneFrameRanges;
	BoneFrameRanges.SetNum(BoneNum);

	ParallelFor(BoneNum, [&](int32 BoneIdx) {

		FVector LocationMean, LocationStd;
		FQuat RotationMean; FVector RotationStd;
		FVector ScaleMean, ScaleLogStd;
		FVector LinearVelocityMean, LinearVelocityStd;
		FVector AngularVelocityMean, AngularVelocityStd;
		FVector ScalarVelocityMean, ScalarVelocityStd;

		FrameAttributeLocationMeanAndStd(LocationMean, LocationStd, BoneLocations[BoneIdx]);
		FrameAttributeRotationMeanAndStdAsQuat(RotationMean, RotationStd, BoneRotations[BoneIdx]);
		FrameAttributeScaleMeanAndStd(ScaleMean, ScaleLogStd, BoneScales[BoneIdx]);
		FrameAttributeVelocityMeanAndStd(LinearVelocityMean, LinearVelocityStd, BoneLinearVelocities[BoneIdx]);
		FrameAttributeVelocityMeanAndStd(AngularVelocityMean, AngularVelocityStd, BoneAngularVelocities[BoneIdx]);
		FrameAttributeVelocityMeanAndStd(ScalarVelocityMean, ScalarVelocityStd, BoneScalarVelocities[BoneIdx]);

		const FAnimDatabaseFrameAttribute LocationDifference = FrameAttributeLength(FrameAttributeSubtractVector(BoneLocations[BoneIdx], LocationMean));
		const FAnimDatabaseFrameAttribute RotationDifference = FrameAttributeLength(FrameAttributeToRotationVector(FrameAttributeAbs(FrameAttributeDivideQuat(BoneRotations[BoneIdx], RotationMean))));
		const FAnimDatabaseFrameAttribute ScaleDifference = FrameAttributeLength(FrameAttributeLog(FrameAttributeDivideVector(BoneScales[BoneIdx], ScaleMean)));
		const FAnimDatabaseFrameAttribute LinearVelocityDifference = FrameAttributeLength(FrameAttributeSubtractVector(BoneLinearVelocities[BoneIdx], LinearVelocityMean));
		const FAnimDatabaseFrameAttribute AngularVelocityDifference = FrameAttributeLength(FrameAttributeSubtractVector(BoneAngularVelocities[BoneIdx], AngularVelocityMean));
		const FAnimDatabaseFrameAttribute ScaleVelocityDifference = FrameAttributeLength(FrameAttributeSubtractVector(BoneScalarVelocities[BoneIdx], ScalarVelocityMean));

		const FAnimDatabaseFrameAttribute LocationOutliers = FrameAttributeGreaterThanFloat(LocationDifference, Threshold * LocationStd.Length() + LocationMinimumDifference);
		const FAnimDatabaseFrameAttribute RotationOutliers = FrameAttributeGreaterThanFloat(RotationDifference, Threshold * RotationStd.Length() + FMath::DegreesToRadians(RotationMinimumDifference));
		const FAnimDatabaseFrameAttribute ScaleOutliers = FrameAttributeGreaterThanFloat(ScaleDifference, Threshold * ScaleLogStd.Length() + ScaleMinimumDifference);
		const FAnimDatabaseFrameAttribute LinearVelocityOutliers = FrameAttributeGreaterThanFloat(LinearVelocityDifference, Threshold * LinearVelocityStd.Length() + LinearVelocityMinimumDifference);
		const FAnimDatabaseFrameAttribute AngularVelocityOutliers = FrameAttributeGreaterThanFloat(AngularVelocityDifference, Threshold * AngularVelocityStd.Length() + FMath::DegreesToRadians(AngularVelocityMinimumDifference));
		const FAnimDatabaseFrameAttribute ScalarVelocityOutliers = FrameAttributeGreaterThanFloat(ScaleVelocityDifference, Threshold * ScalarVelocityStd.Length() + ScalarVelocityMinimumDifference);

		BoneFrameRanges[BoneIdx] = UAnimDatabaseFrameRangesLibrary::FrameRangesUnionFromArrayView({
			FrameAttributeBoolToFrameRanges(LocationOutliers),
			FrameAttributeBoolToFrameRanges(RotationOutliers),
			FrameAttributeBoolToFrameRanges(ScaleOutliers),
			FrameAttributeBoolToFrameRanges(LinearVelocityOutliers),
			FrameAttributeBoolToFrameRanges(AngularVelocityOutliers),
			FrameAttributeBoolToFrameRanges(ScalarVelocityOutliers),
		});
	});

	return UAnimDatabaseFrameRangesLibrary::FrameRangesPad(UAnimDatabaseFrameRangesLibrary::FrameRangesUnionFromArrayView(BoneFrameRanges), Padding / 2, Padding / 2);
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeSavGolSmoothedRootTransform(
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const FName RootLocationProjectionBoneName,
	const FName RootRotationProjectionBoneName,
	const FVector RootRotationBoneForwardDirection,
	const FVector RootForwardDirection,
	const int32 LocationSavGolFrameNum,
	const int32 LocationSavGolPolynomialDegree,
	const int32 DirectionSavGolFrameNum,
	const int32 DirectionSavGolPolynomialDegree,
	const bool bGaussianWindowed)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeSavGolSmoothedRootTransform: Database is nullptr.");
		return FAnimDatabaseFrameAttribute();
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeSavGolSmoothedRootTransform: Invalid FrameRanges.");
		return FAnimDatabaseFrameAttribute();
	}

	Database->WaitForCompressionOnAnimSequencesFromArrayView(FrameRanges.FrameRangeSet->GetEntrySequences());

	UE::Tasks::TTask<FAnimDatabaseFrameAttribute> RootLocationTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [Database, &FrameRanges, RootLocationProjectionBoneName, LocationSavGolFrameNum, LocationSavGolPolynomialDegree, bGaussianWindowed]()
		{
			const FAnimDatabaseFrameAttribute RootLocation = FrameAttributeProject(MakeBoneGlobalLocationFrameAttributeFromName(Database, FrameRanges, RootLocationProjectionBoneName));
			return LocationSavGolFrameNum > 0 ? FrameAttributeFilterSavGol(RootLocation, LocationSavGolFrameNum, LocationSavGolPolynomialDegree, bGaussianWindowed) : RootLocation;
		});

	UE::Tasks::TTask<FAnimDatabaseFrameAttribute> RootRotationTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [Database, &FrameRanges, RootRotationProjectionBoneName, RootRotationBoneForwardDirection, RootForwardDirection, DirectionSavGolFrameNum, DirectionSavGolPolynomialDegree, bGaussianWindowed]()
		{
			const FAnimDatabaseFrameAttribute RootDirection = FrameAttributeProject(MakeBoneGlobalDirectionFrameAttributeFromName(Database, FrameRanges, RootRotationProjectionBoneName, RootRotationBoneForwardDirection));
			const FAnimDatabaseFrameAttribute SmoothedRootDirection = DirectionSavGolFrameNum > 0 ? FrameAttributeFilterSavGol(RootDirection, DirectionSavGolFrameNum, DirectionSavGolPolynomialDegree, bGaussianWindowed) : RootDirection;
			return FrameAttributeInvert(FrameAttributeRotationBetweenDirection(SmoothedRootDirection, RootForwardDirection));
		});

	return MakeTransformFrameAttributeNoScale(RootLocationTask.GetResult(), RootRotationTask.GetResult());
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeGaussianSmoothedRootTransform(
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const FName RootLocationProjectionBoneName,
	const FName RootRotationProjectionBoneName,
	const FVector RootRotationBoneForwardDirection,
	const FVector RootForwardDirection,
	const float LocationSmoothingAmount,
	const float RotationSmoothingAmount)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeGaussianSmoothedRootTransform: Database is nullptr.");
		return FAnimDatabaseFrameAttribute();
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeGaussianSmoothedRootTransform: Invalid FrameRanges.");
		return FAnimDatabaseFrameAttribute();
	}

	Database->WaitForCompressionOnAnimSequencesFromArrayView(FrameRanges.FrameRangeSet->GetEntrySequences());

	UE::Tasks::TTask<FAnimDatabaseFrameAttribute> RootLocationTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [Database, &FrameRanges, RootLocationProjectionBoneName, LocationSmoothingAmount]()
		{
			const FAnimDatabaseFrameAttribute RootLocation = FrameAttributeProject(MakeBoneGlobalLocationFrameAttributeFromName(Database, FrameRanges, RootLocationProjectionBoneName));
			return LocationSmoothingAmount > 0.0f ? FrameAttributeFilterGaussian(RootLocation, LocationSmoothingAmount) : RootLocation;
		});

	UE::Tasks::TTask<FAnimDatabaseFrameAttribute> RootRotationTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [Database, &FrameRanges, RootRotationProjectionBoneName, RootRotationBoneForwardDirection, RootForwardDirection, RotationSmoothingAmount]()
		{
			const FAnimDatabaseFrameAttribute RootDirection = FrameAttributeProject(MakeBoneGlobalDirectionFrameAttributeFromName(Database, FrameRanges, RootRotationProjectionBoneName, RootRotationBoneForwardDirection));
			const FAnimDatabaseFrameAttribute SmoothedRootDirection = RotationSmoothingAmount > 0.0f ? FrameAttributeFilterGaussian(RootDirection, RotationSmoothingAmount) : RootDirection;
			return FrameAttributeInvert(FrameAttributeRotationBetweenDirection(SmoothedRootDirection, RootForwardDirection));
		});

	return MakeTransformFrameAttributeNoScale(RootLocationTask.GetResult(), RootRotationTask.GetResult());
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeBoneSpeed(
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const FName BoneName)
{
	return FrameAttributeLength(MakeBoneGlobalLinearVelocityFrameAttributeFromName(Database, FrameRanges, BoneName));
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeContactCurve(
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const FName BoneName,
	const float HeightThreshold,
	const float VelocityThreshold,
	const bool bFilterCurve,
	const int32 MajorityVoteFilterWidth,
	const bool bSmoothCurve,
	const float SmoothingAmount)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeContactCurve: Database is nullptr.");
		return FAnimDatabaseFrameAttribute();
	}

	FAnimDatabaseFrameAttribute Location, Velocity;
	MakeBoneGlobalLocationAndLinearVelocityFrameAttributeFromName(Location, Velocity, Database, FrameRanges, BoneName);
	
	FAnimDatabaseFrameAttribute ContactActive = FrameAttributeAnd(
		FrameAttributeLessThanFloat(FrameAttributeZ(Location), HeightThreshold),
		FrameAttributeLessThanFloat(FrameAttributeLength(Velocity), VelocityThreshold));

	if (bFilterCurve) { ContactActive = FrameAttributeFilterMajorityVote(ContactActive, MajorityVoteFilterWidth); }

	ContactActive = FrameAttributeBoolToFloat(ContactActive);

	if (bSmoothCurve) { ContactActive = FrameAttributeFilterGaussian(ContactActive, SmoothingAmount); }

	return ContactActive;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameAttributeLibrary::FrameAttributeContactRanges(
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const FName BoneName,
	const float HeightThreshold,
	const float VelocityThreshold,
	const bool bFilterCurve,
	const int32 MajorityVoteFilterWidth)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameAttributeContactRanges: Database is nullptr.");
		return FAnimDatabaseFrameRanges();
	}

	FAnimDatabaseFrameAttribute Location, Velocity;
	MakeBoneGlobalLocationAndLinearVelocityFrameAttributeFromName(Location, Velocity, Database, FrameRanges, BoneName);

	FAnimDatabaseFrameAttribute ContactActive = FrameAttributeAnd(
		FrameAttributeLessThanFloat(FrameAttributeZ(Location), HeightThreshold),
		FrameAttributeLessThanFloat(FrameAttributeLength(Velocity), VelocityThreshold));

	if (bFilterCurve) { ContactActive = FrameAttributeFilterMajorityVote(ContactActive, MajorityVoteFilterWidth); }

	return FrameAttributeBoolToFrameRanges(ContactActive);
}

FAnimDatabaseFrames UAnimDatabaseFrameAttributeLibrary::FrameAttributeContactFrames(
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const FName BoneName,
	const float HeightThreshold,
	const float VelocityThreshold,
	const bool bFilterCurve,
	const int32 MajorityVoteFilterWidth,
	const float FrameOffset,
	const bool bIgnoreFirstFrame)
{
	FAnimDatabaseFrames ContactFrames = UAnimDatabaseFrameRangesLibrary::MakeFramesAtFrameRangesStarts(FrameAttributeContactRanges(
		Database,
		FrameRanges,
		BoneName,
		HeightThreshold,
		VelocityThreshold,
		bFilterCurve,
		MajorityVoteFilterWidth));

	if (bIgnoreFirstFrame)
	{
		ContactFrames = UAnimDatabaseFramesLibrary::FramesDifference(ContactFrames, UAnimDatabaseFrameRangesLibrary::MakeFramesAtFrameRangesStarts(FrameRanges));
	}

	return UAnimDatabaseFramesLibrary::FramesBefore(Database, ContactFrames, FMath::RoundToInt(FrameOffset * Database->GetFrameRate().AsDecimal()), EAnimDatabaseFrameShiftBehavior::Clamp);
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeFitTransform(
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const TArray<FAnimDatabaseFrameAttribute>& FitLocations,
	const TArray<FAnimDatabaseFrameAttribute>& FitForwardDirections,
	const TArray<FAnimDatabaseFrameAttribute>& FitRightDirections,
	const TArray<FAnimDatabaseFrameAttribute>& FitUpDirections,
	const TArray<FAnimDatabaseFrameAttribute>& FitLocationWeights,
	const TArray<FAnimDatabaseFrameAttribute>& FitForwardDirectionWeights,
	const TArray<FAnimDatabaseFrameAttribute>& FitRightDirectionWeights,
	const TArray<FAnimDatabaseFrameAttribute>& FitUpDirectionWeights)
{
	return FrameAttributeFitTransformFromArrayViews(
		Database,
		FrameRanges,
		FitLocations,
		FitForwardDirections,
		FitRightDirections,
		FitUpDirections,
		FitLocationWeights,
		FitForwardDirectionWeights,
		FitRightDirectionWeights,
		FitUpDirectionWeights);
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeLibrary::FrameAttributeFitTransformFromArrayViews(
	const UAnimDatabase* Database,
	const FAnimDatabaseFrameRanges& FrameRanges,
	const TArrayView<const FAnimDatabaseFrameAttribute> FitLocations,
	const TArrayView<const FAnimDatabaseFrameAttribute> FitForwardDirections,
	const TArrayView<const FAnimDatabaseFrameAttribute> FitRightDirections,
	const TArrayView<const FAnimDatabaseFrameAttribute> FitUpDirections,
	const TArrayView<const FAnimDatabaseFrameAttribute> FitLocationWeights,
	const TArrayView<const FAnimDatabaseFrameAttribute> FitForwardDirectionWeights,
	const TArrayView<const FAnimDatabaseFrameAttribute> FitRightDirectionWeights,
	const TArrayView<const FAnimDatabaseFrameAttribute> FitUpDirectionWeights)
{
	const int32 LocationNum = FitLocations.Num();
	const int32 ForwardNum = FitForwardDirections.Num();
	const int32 RightNum = FitRightDirections.Num();
	const int32 UpNum = FitUpDirections.Num();

	if (LocationNum == 0 || ForwardNum == 0 || RightNum == 0 || UpNum == 0) { return FAnimDatabaseFrameAttribute(); }

	TArray<FAnimDatabaseFrameAttribute, TInlineAllocator<32>> ForwardAttributes;
	TArray<FAnimDatabaseFrameAttribute, TInlineAllocator<32>> RightAttributes;
	TArray<FAnimDatabaseFrameAttribute, TInlineAllocator<32>> UpAttributes;

	ForwardAttributes.Reserve(ForwardNum);
	RightAttributes.Reserve(RightNum);
	UpAttributes.Reserve(UpNum);

	TArray<FAnimDatabaseFrameAttribute> LocationWeights; LocationWeights.SetNum(LocationNum);
	TArray<FAnimDatabaseFrameAttribute> ForwardWeights; ForwardWeights.SetNum(ForwardNum);
	TArray<FAnimDatabaseFrameAttribute> RightWeights; RightWeights.SetNum(RightNum);
	TArray<FAnimDatabaseFrameAttribute> UpWeights; UpWeights.SetNum(UpNum);

	const FAnimDatabaseFrameAttribute ConstantZero = UAnimDatabaseFrameAttributeLibrary::MakeFloatFrameAttributeFromConstant(FrameRanges, 0.0f);
	const FAnimDatabaseFrameAttribute ConstantOne = UAnimDatabaseFrameAttributeLibrary::MakeFloatFrameAttributeFromConstant(FrameRanges, 1.0f);

	FAnimDatabaseFrameAttribute LocationWeightSum = ConstantZero;
	for (int32 LocationIdx = 0; LocationIdx < LocationNum; LocationIdx++)
	{
		LocationWeights[LocationIdx] = LocationIdx < FitLocationWeights.Num() ? FitLocationWeights[LocationIdx] : ConstantOne;
		LocationWeightSum = UAnimDatabaseFrameAttributeLibrary::FrameAttributeAdd(LocationWeightSum, LocationWeights[LocationIdx]);
	}

	for (int32 LocationIdx = 0; LocationIdx < LocationNum; LocationIdx++)
	{
		LocationWeights[LocationIdx] = UAnimDatabaseFrameAttributeLibrary::FrameAttributeDivide(LocationWeights[LocationIdx], UAnimDatabaseFrameAttributeLibrary::FrameAttributeMaxFloat(LocationWeightSum, UE_SMALL_NUMBER));
	}

	FAnimDatabaseFrameAttribute ForwardWeightSum = ConstantZero;
	for (int32 DirectionIdx = 0; DirectionIdx < ForwardNum; DirectionIdx++)
	{
		ForwardWeights[DirectionIdx] = DirectionIdx < FitForwardDirectionWeights.Num() ? FitForwardDirectionWeights[DirectionIdx] : ConstantOne;
		ForwardWeightSum = UAnimDatabaseFrameAttributeLibrary::FrameAttributeAdd(ForwardWeightSum, ForwardWeights[DirectionIdx]);
		ForwardAttributes.Add(UAnimDatabaseFrameAttributeLibrary::FrameAttributeDirectionToLocation(FitForwardDirections[DirectionIdx]));
	}

	for (int32 DirectionIdx = 0; DirectionIdx < ForwardNum; DirectionIdx++)
	{
		ForwardWeights[DirectionIdx] = UAnimDatabaseFrameAttributeLibrary::FrameAttributeDivide(ForwardWeights[DirectionIdx], UAnimDatabaseFrameAttributeLibrary::FrameAttributeMaxFloat(ForwardWeightSum, UE_SMALL_NUMBER));
	}

	FAnimDatabaseFrameAttribute RightWeightSum = ConstantZero;
	for (int32 DirectionIdx = 0; DirectionIdx < RightNum; DirectionIdx++)
	{
		RightWeights[DirectionIdx] = DirectionIdx < FitRightDirectionWeights.Num() ? FitRightDirectionWeights[DirectionIdx] : ConstantOne;
		RightWeightSum = UAnimDatabaseFrameAttributeLibrary::FrameAttributeAdd(RightWeightSum, RightWeights[DirectionIdx]);
		RightAttributes.Add(UAnimDatabaseFrameAttributeLibrary::FrameAttributeDirectionToLocation(FitRightDirections[DirectionIdx]));
	}

	for (int32 DirectionIdx = 0; DirectionIdx < RightNum; DirectionIdx++)
	{
		RightWeights[DirectionIdx] = UAnimDatabaseFrameAttributeLibrary::FrameAttributeDivide(RightWeights[DirectionIdx], UAnimDatabaseFrameAttributeLibrary::FrameAttributeMaxFloat(RightWeightSum, UE_SMALL_NUMBER));
	}

	FAnimDatabaseFrameAttribute UpWeightSum = ConstantZero;
	for (int32 DirectionIdx = 0; DirectionIdx < UpNum; DirectionIdx++)
	{
		UpWeights[DirectionIdx] = DirectionIdx < FitUpDirectionWeights.Num() ? FitUpDirectionWeights[DirectionIdx] : ConstantOne;
		UpWeightSum = UAnimDatabaseFrameAttributeLibrary::FrameAttributeAdd(UpWeightSum, UpWeights[DirectionIdx]);
		UpAttributes.Add(UAnimDatabaseFrameAttributeLibrary::FrameAttributeDirectionToLocation(FitUpDirections[DirectionIdx]));
	}

	for (int32 DirectionIdx = 0; DirectionIdx < UpNum; DirectionIdx++)
	{
		UpWeights[DirectionIdx] = UAnimDatabaseFrameAttributeLibrary::FrameAttributeDivide(UpWeights[DirectionIdx], UAnimDatabaseFrameAttributeLibrary::FrameAttributeMaxFloat(UpWeightSum, UE_SMALL_NUMBER));
	}

	const FAnimDatabaseFrameAttribute LocationAverage = UAnimDatabaseFrameAttributeLibrary::FrameAttributeDynamicWeightedSumFromArrayView(FitLocations, LocationWeights);
	const FAnimDatabaseFrameAttribute ForwardAverage = UAnimDatabaseFrameAttributeLibrary::FrameAttributeLocationToDirection(UAnimDatabaseFrameAttributeLibrary::FrameAttributeDynamicWeightedSumFromArrayView(ForwardAttributes, ForwardWeights));
	const FAnimDatabaseFrameAttribute RightAverage = UAnimDatabaseFrameAttributeLibrary::FrameAttributeLocationToDirection(UAnimDatabaseFrameAttributeLibrary::FrameAttributeDynamicWeightedSumFromArrayView(RightAttributes, RightWeights));
	const FAnimDatabaseFrameAttribute UpAverage = UAnimDatabaseFrameAttributeLibrary::FrameAttributeLocationToDirection(UAnimDatabaseFrameAttributeLibrary::FrameAttributeDynamicWeightedSumFromArrayView(UpAttributes, UpWeights));

	const FAnimDatabaseFrameAttribute RotationAverage = UAnimDatabaseFrameAttributeLibrary::MakeRotationFrameAttributeFromBasisDirections(ForwardAverage, RightAverage, UpAverage);

	return UAnimDatabaseFrameAttributeLibrary::MakeTransformFrameAttributeNoScale(LocationAverage, RotationAverage);
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeFunction::MakeFrameAttribute_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameAttributeLibrary::MakeEmptyFrameAttribute();
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeFunction_FrameRanges::MakeFrameAttribute_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameAttributeLibrary::MakeBoolFrameAttributeFromActiveRanges(UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFunction(Database, FrameRanges, FrameRangesFunction), FrameRanges);
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeFunction_Filter::MakeFrameAttribute_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameAttributeLibrary::MakeFrameAttributeFromFunction(Database, UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFunction(Database, FrameRanges, FrameRangesFunction), FrameAttributeFunction);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_Outliers::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameAttributeLibrary::FrameAttributeOutlierFrameRanges(
		Database,
		ExcludedRanges ?
			UAnimDatabaseFrameRangesLibrary::FrameRangesDifference(FrameRanges, UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFunction(Database, FrameRanges, ExcludedRanges)) :
			FrameRanges,
		Padding,
		Threshold,
		LocationMinimumDifference,
		RotationMinimumDifference,
		ScaleMinimumDifference,
		LinearVelocityMinimumDifference,
		AngularVelocityMinimumDifference,
		ScalarVelocityMinimumDifference);
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeFunction_ContactCurve::MakeFrameAttribute_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameAttributeLibrary::FrameAttributeContactCurve(
		Database,
		FrameRanges,
		BoneName,
		HeightThreshold,
		VelocityThreshold,
		bFilterCurve,
		MajorityVoteFilterWidth,
		bSmoothCurve,
		SmoothingAmount);
}

FAnimDatabaseFrames UAnimDatabaseFramesFunction_ContactFrames::MakeFrames_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	FAnimDatabaseFrames ContactFrames = UAnimDatabaseFrameAttributeLibrary::FrameAttributeContactFrames(
		Database,
		FrameRanges,
		BoneName,
		HeightThreshold,
		VelocityThreshold,
		bFilterCurve,
		MajorityVoteFilterWidth,
		EventOffset);

	if (EventFilter)
	{
		ContactFrames = UAnimDatabaseFrameRangesLibrary::FramesFrameRangesIntersection(ContactFrames, UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFunction(Database, FrameRanges, EventFilter));
	}

	return ContactFrames;
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeFunction_ContactEvent::MakeFrameAttribute_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	FAnimDatabaseFrames ContactFrames = UAnimDatabaseFrameAttributeLibrary::FrameAttributeContactFrames(
		Database,
		FrameRanges,
		BoneName,
		HeightThreshold,
		VelocityThreshold,
		bFilterCurve,
		MajorityVoteFilterWidth,
		EventOffset);

	if (EventFilter)
	{
		ContactFrames = UAnimDatabaseFrameRangesLibrary::FramesFrameRangesIntersection(ContactFrames, UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFunction(Database, FrameRanges, EventFilter));
	}

	return UAnimDatabaseFrameAttributeLibrary::MakeEventFrameAttribute(ContactFrames, FrameRanges, Database->GetFrameRate());
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeFunction_FitTransform::MakeFrameAttribute_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	TArray<FAnimDatabaseFrameAttribute> LocationAttributes, ForwardAttributes, RightAttributes, UpAttributes, LocationWeightAttributes, ForwardWeightAttributes, RightWeightAttributes, UpWeightAttributes;
	UAnimDatabaseFrameAttributeLibrary::MakeFrameAttributesFromFunctions(LocationAttributes, Database, FrameRanges, FitLocations);
	UAnimDatabaseFrameAttributeLibrary::MakeFrameAttributesFromFunctions(ForwardAttributes, Database, FrameRanges, FitForwardDirections);
	UAnimDatabaseFrameAttributeLibrary::MakeFrameAttributesFromFunctions(RightAttributes, Database, FrameRanges, FitRightDirections);
	UAnimDatabaseFrameAttributeLibrary::MakeFrameAttributesFromFunctions(UpAttributes, Database, FrameRanges, FitUpDirections);
	UAnimDatabaseFrameAttributeLibrary::MakeFrameAttributesFromFunctions(LocationWeightAttributes, Database, FrameRanges, FitLocationWeights);
	UAnimDatabaseFrameAttributeLibrary::MakeFrameAttributesFromFunctions(ForwardWeightAttributes, Database, FrameRanges, FitForwardDirectionWeights);
	UAnimDatabaseFrameAttributeLibrary::MakeFrameAttributesFromFunctions(RightWeightAttributes, Database, FrameRanges, FitRightDirectionWeights);
	UAnimDatabaseFrameAttributeLibrary::MakeFrameAttributesFromFunctions(UpWeightAttributes, Database, FrameRanges, FitUpDirectionWeights);

	return UAnimDatabaseFrameAttributeLibrary::FrameAttributeFitTransformFromArrayViews(
		Database,
		FrameRanges,
		LocationAttributes,
		ForwardAttributes,
		RightAttributes,
		UpAttributes,
		LocationWeightAttributes,
		ForwardWeightAttributes,
		RightWeightAttributes,
		UpWeightAttributes);
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeFunction_BoneGlobalTransform::MakeFrameAttribute_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameAttributeLibrary::FrameAttributeTransformMultiply(LocalOffset, UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalTransformFrameAttributeFromName(Database, FrameRanges, BoneName));
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeFunction_BoneRelativeTransform::MakeFrameAttribute_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	TArray<FAnimDatabaseFrameAttribute, TInlineAllocator<2>> BoneTransforms; BoneTransforms.SetNum(2);
	UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalTransformFrameAttributesFromNamesArrayView(BoneTransforms, Database, FrameRanges, { BoneName, RelativeBoneName });
	return UAnimDatabaseFrameAttributeLibrary::FrameAttributeDivide(BoneTransforms[0], BoneTransforms[1]);
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeFunction_BoneRelativeAngularVelocity::MakeFrameAttribute_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	const FAnimDatabaseFrameAttribute RelativeRotation = UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalRotationFrameAttributeFromName(Database, FrameRanges, RelativeBoneName);
	const FAnimDatabaseFrameAttribute AngularVelocity = UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalAngularVelocityFrameAttributeFromName(Database, FrameRanges, BoneName);
	return UAnimDatabaseFrameAttributeLibrary::FrameAttributeUnrotate(RelativeRotation, AngularVelocity);
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeFunction_BoneGlobalLocation::MakeFrameAttribute_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameAttributeLibrary::FrameAttributeApplyTransformLocation(UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalTransformFrameAttributeFromName(Database, FrameRanges, BoneName), LocalOffset);
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeFunction_BoneGlobalDirection::MakeFrameAttribute_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalDirectionFrameAttributeFromName(Database, FrameRanges, BoneName, LocalDirection);
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeFunction_BoneGlobalLinearVelocityMagnitude::MakeFrameAttribute_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameAttributeLibrary::FrameAttributeBoneSpeed(Database, FrameRanges, BoneName);
}

FAnimDatabaseFrameAttribute UAnimDatabaseFrameAttributeFunction_GaussianSmoothingFilter::MakeFrameAttribute_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	if (!Database) { return FAnimDatabaseFrameAttribute(); }

	return UAnimDatabaseFrameAttributeLibrary::FrameAttributeFilterGaussian(
		UAnimDatabaseFrameAttributeLibrary::MakeFrameAttributeFromFunction(Database, FrameRanges, FilterFrameAttribute),
		FMath::RoundToInt(StandardDeviation * Database->GetFrameRate().AsDecimal()));
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_MajorityVoteFilter::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameAttributeLibrary::FrameAttributeBoolToFrameRanges(
		UAnimDatabaseFrameAttributeLibrary::FrameAttributeFilterMajorityVote(
			UAnimDatabaseFrameAttributeLibrary::FrameAttributeFrameRangesToBool(UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFunction(Database, FrameRanges, FilterFrameRanges), FrameRanges),
			MajorityVoteFilterWidth));
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_MovingInDirection::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	const FAnimDatabaseFrameAttribute MovingDirection = UAnimDatabaseFrameAttributeLibrary::MakeRootDirectionFrameAttribute(Database, FrameRanges, 0.0f, Direction);
	const FAnimDatabaseFrameAttribute RootVelocity = UAnimDatabaseFrameAttributeLibrary::MakeRootLinearVelocityFrameAttribute(Database, FrameRanges);
	const FAnimDatabaseFrameAttribute RootVelocityDirection = UAnimDatabaseFrameAttributeLibrary::FrameAttributeVelocityToDirection(RootVelocity);

	return UAnimDatabaseFrameAttributeLibrary::FrameAttributeBoolToFrameRanges(
		UAnimDatabaseFrameAttributeLibrary::FrameAttributeAnd(
			UAnimDatabaseFrameAttributeLibrary::FrameAttributeGreaterThanFloat(UAnimDatabaseFrameAttributeLibrary::FrameAttributeDot(MovingDirection, RootVelocityDirection), FMath::Cos(FMath::DegreesToRadians(AngleThreshold))),
			UAnimDatabaseFrameAttributeLibrary::FrameAttributeGreaterThanFloat(UAnimDatabaseFrameAttributeLibrary::FrameAttributeLength(RootVelocity), VelocityThreshold)));
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_RootLinearVelocityBelowThreshold::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameAttributeLibrary::FrameAttributeBoolToFrameRanges(UAnimDatabaseFrameAttributeLibrary::FrameAttributeLessThanFloat(
		UAnimDatabaseFrameAttributeLibrary::FrameAttributeLength(UAnimDatabaseFrameAttributeLibrary::MakeRootLinearVelocityFrameAttribute(Database, FrameRanges)), Threshold));
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_RootLinearVelocityAboveThreshold::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameAttributeLibrary::FrameAttributeBoolToFrameRanges(UAnimDatabaseFrameAttributeLibrary::FrameAttributeGreaterThanFloat(
		UAnimDatabaseFrameAttributeLibrary::FrameAttributeLength(UAnimDatabaseFrameAttributeLibrary::MakeRootLinearVelocityFrameAttribute(Database, FrameRanges)), Threshold));
}

#undef UE_ANIMDATABASE_ISPC
#undef LOCTEXT_NAMESPACE