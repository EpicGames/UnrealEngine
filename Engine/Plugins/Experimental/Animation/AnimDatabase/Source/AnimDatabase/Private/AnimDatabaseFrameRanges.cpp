// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimDatabaseFrameRanges.h"

#include "AnimDatabase.h"
#include "AnimDatabaseFrameAttribute.h"

#include "LearningFrameSet.h"
#include "LearningFrameRangeSet.h"

#include "UObject/Package.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimNotifies/AnimNotify.h"
#include "Animation/AnimNotifies/AnimNotifyState.h"

#if WITH_EDITOR
#include "Misc/PackageName.h"
#include "HAL/FileManager.h"
#endif

#include "LearningRandom.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimDatabaseFrameRanges)


bool FAnimDatabaseFrames::IsValid() const
{
	return FrameSet.IsValid();
}

bool FAnimDatabaseFrames::Serialize(FArchive& Ar)
{
	static constexpr int32 Magic = 0x6c9100dc;
	static constexpr int32 Version = 0;

	if (Ar.IsSaving())
	{
		int32 SaveMagic = Magic;
		int32 SaveVersion = Version;
		bool bSaveIsValid = FrameSet.IsValid();

		Ar << SaveMagic;
		Ar << SaveVersion;
		Ar << bSaveIsValid;
		if (bSaveIsValid)
		{
			UE::Learning::Array::Serialize(Ar, FrameSet->EntrySequences);
			UE::Learning::Array::Serialize(Ar, FrameSet->EntryFrameOffsets);
			UE::Learning::Array::Serialize(Ar, FrameSet->EntryFrameNums);
			UE::Learning::Array::Serialize(Ar, FrameSet->Frames);
		}
	}
	else if (Ar.IsLoading())
	{
		int64 Offset = Ar.Tell();
		int32 LoadMagic = 0;
		int32 LoadVersion = 0;
		bool bLoadIsValid = true;

		Ar << LoadMagic;
		if (!ensure(LoadMagic == Magic)) { Ar.Seek(Offset); FrameSet.Reset(); return false; }
		Ar << LoadVersion;
		if (!ensure(LoadVersion == Version)) { Ar.Seek(Offset); FrameSet.Reset(); return false; }
		Ar << bLoadIsValid;
		if (bLoadIsValid)
		{
			FrameSet = MakeShared<UE::Learning::FFrameSet>();
			UE::Learning::Array::Serialize(Ar, FrameSet->EntrySequences);
			UE::Learning::Array::Serialize(Ar, FrameSet->EntryFrameOffsets);
			UE::Learning::Array::Serialize(Ar, FrameSet->EntryFrameNums);
			UE::Learning::Array::Serialize(Ar, FrameSet->Frames);
		}
		else
		{
			FrameSet.Reset();
		}
	}

	return true;
}

bool operator==(const FAnimDatabaseFrames& Lhs, const FAnimDatabaseFrames& Rhs)
{
	if (Lhs.IsValid() && Rhs.IsValid())
	{
		return UE::Learning::FrameSet::Equal(*Lhs.FrameSet, *Rhs.FrameSet);
	}
	else if (!Lhs.IsValid() && !Rhs.IsValid())
	{
		return true;
	}
	else
	{
		return false;
	}
}

bool operator==(const FAnimDatabaseFramesEntry& Lhs, const FAnimDatabaseFramesEntry& Rhs)
{
	return Lhs.Name == Rhs.Name && Lhs.Frames == Rhs.Frames;
}

bool FAnimDatabaseFrameRanges::IsValid() const
{
	return FrameRangeSet.IsValid();
}

bool FAnimDatabaseFrameRanges::Serialize(FArchive& Ar)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAnimDatabaseFrameRanges::Serialize);

	static constexpr int32 Magic = 0xb825abc6;
	static constexpr int32 Version = 0;

	if (Ar.IsSaving())
	{
		int32 SaveMagic = Magic;
		int32 SaveVersion = Version;
		bool bSaveIsValid = FrameRangeSet.IsValid();

		Ar << SaveMagic;
		Ar << SaveVersion;
		Ar << bSaveIsValid;
		if (bSaveIsValid)
		{
			UE::Learning::Array::Serialize(Ar, FrameRangeSet->EntrySequences);
			UE::Learning::Array::Serialize(Ar, FrameRangeSet->EntryRangeOffsets);
			UE::Learning::Array::Serialize(Ar, FrameRangeSet->EntryRangeNums);
			UE::Learning::Array::Serialize(Ar, FrameRangeSet->RangeStarts);
			UE::Learning::Array::Serialize(Ar, FrameRangeSet->RangeLengths);
			UE::Learning::Array::Serialize(Ar, FrameRangeSet->RangeOffsets);
		}
	}
	else if (Ar.IsLoading())
	{
		int64 Offset = Ar.Tell();
		int32 LoadMagic = 0;
		int32 LoadVersion = 0;
		bool bLoadIsValid = false;

		Ar << LoadMagic;
		if (!ensure(LoadMagic == Magic)) { Ar.Seek(Offset); FrameRangeSet.Reset(); return false; }
		Ar << LoadVersion;
		if (!ensure(LoadVersion == Version)) { Ar.Seek(Offset); FrameRangeSet.Reset(); return false; }
		Ar << bLoadIsValid;
		if (bLoadIsValid)
		{
			FrameRangeSet = MakeShared<UE::Learning::FFrameRangeSet>();
			UE::Learning::Array::Serialize(Ar, FrameRangeSet->EntrySequences);
			UE::Learning::Array::Serialize(Ar, FrameRangeSet->EntryRangeOffsets);
			UE::Learning::Array::Serialize(Ar, FrameRangeSet->EntryRangeNums);
			UE::Learning::Array::Serialize(Ar, FrameRangeSet->RangeStarts);
			UE::Learning::Array::Serialize(Ar, FrameRangeSet->RangeLengths);
			UE::Learning::Array::Serialize(Ar, FrameRangeSet->RangeOffsets);
		}
		else
		{
			FrameRangeSet.Reset();
		}
	}

	return true;
}

bool operator==(const FAnimDatabaseFrameRanges& Lhs, const FAnimDatabaseFrameRanges& Rhs)
{
	if (Lhs.IsValid() && Rhs.IsValid())
	{
		return UE::Learning::FrameRangeSet::Equal(*Lhs.FrameRangeSet, *Rhs.FrameRangeSet);
	}
	else if (!Lhs.IsValid() && !Rhs.IsValid())
	{
		return true;
	}
	else
	{
		return false;
	}
}

bool operator==(const FAnimDatabaseFrameRangesEntry& Lhs, const FAnimDatabaseFrameRangesEntry& Rhs)
{
	return Lhs.Name == Rhs.Name && Lhs.FrameRanges == Rhs.FrameRanges;
}

FAnimDatabaseFrames UAnimDatabaseFramesLibrary::MakeEmptyFrames()
{
	FAnimDatabaseFrames Out;
	Out.FrameSet = MakeShared<UE::Learning::FFrameSet>();
	return Out;
}

FAnimDatabaseFrames UAnimDatabaseFramesLibrary::MakeFramesFromFunction(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const UAnimDatabaseFramesFunction* Function)
{
	if (!Function)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFramesFromFunction: Function is null.");
		return FAnimDatabaseFrames();
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFramesFromFunction: Invalid FrameRanges Object.");
		return FAnimDatabaseFrames();
	}

	const FAnimDatabaseFrames OutFrames = Function->MakeFrames(Database, FrameRanges);

	if (!OutFrames.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFramesFromFunction: Invalid Frames Object.");
		return FAnimDatabaseFrames();
	}

	return OutFrames;
}

FAnimDatabaseFrames UAnimDatabaseFramesLibrary::MakeFramesFromClass(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TSubclassOf<UAnimDatabaseFramesFunction> Class)
{
	if (!Class)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFramesFromClass: Class is null.");
		return FAnimDatabaseFrames();
	}

	return MakeFramesFromFunction(Database, FrameRanges, Class->GetDefaultObject<UAnimDatabaseFramesFunction>());
}

FAnimDatabaseFrames UAnimDatabaseFramesLibrary::MakeFramesFromAnimNotify(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TSubclassOf<UAnimNotify> Notify)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFramesLibrary::MakeFramesFromAnimNotify);

	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFramesFromAnimNotify: Database is nullptr.");
		return FAnimDatabaseFrames();
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFramesFromAnimNotify: Invalid FrameRanges Object.");
		return FAnimDatabaseFrames();
	}

	if (!Notify)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFramesFromAnimNotify: Notify is nullptr.");
		return FAnimDatabaseFrames();
	}

	FAnimDatabaseFrames Out = MakeEmptyFrames();

	TArray<FAnimNotifyEventReference, TInlineAllocator<64>> SortedReferences;
	TArray<int32> FramesToAdd;

	const int32 EntryNum = FrameRanges.FrameRangeSet->GetEntryNum();

	for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
	{
		const int32 SequenceIdx = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);

		const UAnimSequence* Sequence = Database->GetAnimSequence(SequenceIdx);
		if (!Sequence) { continue; }

		const int32 SequenceFrameNum = FMath::Max(Database->GetSequenceFrameNum(SequenceIdx), 1);

		FAnimNotifyContext Notifies;
		Sequence->GetAnimNotifies(0.0f, Sequence->GetPlayLength(), Notifies);

		SortedReferences = Notifies.ActiveNotifies;
		SortedReferences.Sort([](const FAnimNotifyEventReference& A, const FAnimNotifyEventReference& B) { return A.GetNotify()->GetTime() < B.GetNotify()->GetTime(); });

		int32 PreviousFramesTime = -1;

		FramesToAdd.Reset();
		for (const FAnimNotifyEventReference& AnimNotifyRef : SortedReferences)
		{
			if (AnimNotifyRef.GetNotify()->Notify && AnimNotifyRef.GetNotify()->Notify->IsA(Notify))
			{
				const int32 AnimNotifyStart = FMath::Clamp(
					FMath::RoundToInt(AnimNotifyRef.GetNotify()->GetTime() * Database->GetFrameRate().AsDecimal()),
					0, SequenceFrameNum - 1);

				if (AnimNotifyStart == PreviousFramesTime)
				{
					UE_LOGFMT(LogAnimDatabase, Warning, "MakeFramesFromAnimNotify: Anim Notify {Notify} is at same time as Previous in Anim Sequence {Name} Time {Time} Frame {Frame}", *Notify->GetName(), *Sequence->GetName(), AnimNotifyRef.GetNotify()->GetTime(), AnimNotifyStart);
				}
				else
				{
					if (FrameRanges.FrameRangeSet->EntryContains(EntryIdx, AnimNotifyStart))
					{
						FramesToAdd.Add(AnimNotifyStart);
					}
					
					PreviousFramesTime = AnimNotifyStart;
				}
			}
		}

		Out.FrameSet->AddEntry(SequenceIdx, FramesToAdd);
	}

	return Out;
}

FAnimDatabaseFrames UAnimDatabaseFramesLibrary::MakeFramesFromAnimNotifiesOnTrack(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName TrackName)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFramesLibrary::MakeFramesFromAnimNotifiesOnTrack);

#if WITH_EDITOR

	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFramesFromAnimNotifiesOnTrack: Database is nullptr.");
		return FAnimDatabaseFrames();
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFramesFromAnimNotifiesOnTrack: Invalid FrameRanges Object.");
		return FAnimDatabaseFrames();
	}

	FAnimDatabaseFrames Out = MakeEmptyFrames();

	TArray<FAnimNotifyEventReference, TInlineAllocator<64>> SortedReferences;
	TArray<int32> FramesToAdd;

	const int32 EntryNum = FrameRanges.FrameRangeSet->GetEntryNum();

	for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
	{
		const int32 SequenceIdx = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);

		const UAnimSequence* Sequence = Database->GetAnimSequence(SequenceIdx);
		if (!Sequence) { continue; }

		int32 TrackIndex = INDEX_NONE;

		int32 TrackNum = Sequence->AnimNotifyTracks.Num();
		for (int32 TrackIdx = 0; TrackIdx < TrackNum; TrackIdx++)
		{
			if (Sequence->AnimNotifyTracks[TrackIdx].TrackName == TrackName)
			{
				TrackIndex = TrackIdx;
				break;
			}
		}

		if (TrackIndex == INDEX_NONE) { continue; }

		const int32 SequenceFrameNum = FMath::Max(Database->GetSequenceFrameNum(SequenceIdx), 1);

		FAnimNotifyContext Notifies;
		Sequence->GetAnimNotifies(0.0f, Sequence->GetPlayLength(), Notifies);

		SortedReferences = Notifies.ActiveNotifies;
		SortedReferences.Sort([](const FAnimNotifyEventReference& A, const FAnimNotifyEventReference& B) { return A.GetNotify()->GetTime() < B.GetNotify()->GetTime(); });

		int32 PreviousFramesTime = -1;

		FramesToAdd.Reset();
		for (const FAnimNotifyEventReference& AnimNotifyRef : SortedReferences)
		{
			if (AnimNotifyRef.GetNotify()->Notify && AnimNotifyRef.GetNotify()->TrackIndex == TrackIndex)
			{
				const int32 AnimNotifyStart = FMath::Clamp(
					FMath::RoundToInt(AnimNotifyRef.GetNotify()->GetTime() * Database->GetFrameRate().AsDecimal()),
					0, SequenceFrameNum - 1);

				if (AnimNotifyStart == PreviousFramesTime)
				{
					UE_LOGFMT(LogAnimDatabase, Warning, "MakeFramesFromAnimNotifiesOnTrack: Anim Notify {Notify} is at same time as Previous in Anim Sequence {Name} Time {Time} Frame {Frame}", *AnimNotifyRef.GetNotify()->Notify->GetName(), *Sequence->GetName(), AnimNotifyRef.GetNotify()->GetTime(), AnimNotifyStart);
				}
				else
				{
					if (FrameRanges.FrameRangeSet->EntryContains(EntryIdx, AnimNotifyStart))
					{
						FramesToAdd.Add(AnimNotifyStart);
					}

					PreviousFramesTime = AnimNotifyStart;
				}
			}
		}

		Out.FrameSet->AddEntry(SequenceIdx, FramesToAdd);
	}

	return Out;

#else

	UE_LOGFMT(LogAnimDatabase, Error, "MakeFramesFromAnimNotifiesOnTrack: Track Names are only available in Editor.");
	return FAnimDatabaseFrames();

#endif
}

FAnimDatabaseFrames UAnimDatabaseFramesLibrary::MakeFramesFromSyncMarker(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName SyncMarkerName)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFramesLibrary::MakeFramesFromSyncMarker);

	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFramesFromSyncMarker: Database is nullptr.");
		return FAnimDatabaseFrames();
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFramesFromSyncMarker: Invalid FrameRanges Object.");
		return FAnimDatabaseFrames();
	}

	FAnimDatabaseFrames Out = MakeEmptyFrames();

	TArray<FAnimSyncMarker, TInlineAllocator<64>> SortedSyncMarkers;
	TArray<int32> FramesToAdd;

	const int32 EntryNum = FrameRanges.FrameRangeSet->GetEntryNum();

	for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
	{
		const int32 SequenceIdx = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);

		const UAnimSequence* Sequence = Database->GetAnimSequence(SequenceIdx);
		if (!Sequence) { continue; }

		const int32 SequenceFrameNum = FMath::Max(Database->GetSequenceFrameNum(SequenceIdx), 1);

		SortedSyncMarkers = Sequence->AuthoredSyncMarkers;
		SortedSyncMarkers.Sort([](const FAnimSyncMarker& A, const FAnimSyncMarker& B) { return A.Time < B.Time; });

		int32 PreviousFramesTime = -1;

		FramesToAdd.Reset();
		for (const FAnimSyncMarker& SyncMarker : SortedSyncMarkers)
		{
			if (SyncMarker.MarkerName == SyncMarkerName)
			{
				const int32 SyncMarkerStart = FMath::Clamp(
					FMath::RoundToInt(SyncMarker.Time * Database->GetFrameRate().AsDecimal()),
					0, SequenceFrameNum - 1);

				if (SyncMarkerStart == PreviousFramesTime)
				{
					UE_LOGFMT(LogAnimDatabase, Warning, "MakeFramesFromSyncMarker: Sync Marker {SyncMarkerName} is at same time as Previous in Anim Sequence {Name} Time {Time} Frame {Frame}", *SyncMarkerName.ToString(), *Sequence->GetName(), SyncMarker.Time, SyncMarkerStart);
				}
				else
				{
					if (FrameRanges.FrameRangeSet->EntryContains(EntryIdx, SyncMarkerStart))
					{
						FramesToAdd.Add(SyncMarkerStart);
					}

					PreviousFramesTime = SyncMarkerStart;
				}
			}
		}

		Out.FrameSet->AddEntry(SequenceIdx, FramesToAdd);
	}

	return Out;
}

FAnimDatabaseFrames UAnimDatabaseFramesLibrary::MakeFramesFromAnimNotifyStateStart(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TSubclassOf<UAnimNotifyState> State)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFramesLibrary::MakeFramesFromAnimNotifyStateStart);

	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFramesFromAnimNotifyStateStart: Database is nullptr.");
		return FAnimDatabaseFrames();
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFramesFromAnimNotify: Invalid FrameRanges Object.");
		return FAnimDatabaseFrames();
	}

	if (!State)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFramesFromAnimNotifyStateStart: State is nullptr.");
		return FAnimDatabaseFrames();
	}

	FAnimDatabaseFrames Out = MakeEmptyFrames();

	TArray<int32> FramesToAdd;

	const int32 EntryNum = FrameRanges.FrameRangeSet->GetEntryNum();

	for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
	{
		const int32 SequenceIdx = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);

		const UAnimSequence* Sequence = Database->GetAnimSequence(SequenceIdx);
		if (!Sequence) { continue; }

		const int32 SequenceFrameNum = FMath::Max(Database->GetSequenceFrameNum(SequenceIdx), 1);

		FAnimNotifyContext Notifies;
		Sequence->GetAnimNotifies(0.0f, Sequence->GetPlayLength(), Notifies);

		int32 PreviousFramesTime = -1;

		FramesToAdd.Reset();
		for (const FAnimNotifyEventReference& AnimNotifyRef : Notifies.ActiveNotifies)
		{
			if (AnimNotifyRef.GetNotify()->NotifyStateClass && AnimNotifyRef.GetNotify()->NotifyStateClass->IsA(State))
			{
				const int32 AnimNotifyStart = FMath::Clamp(
					FMath::RoundToInt(AnimNotifyRef.GetNotify()->GetTime() * Database->GetFrameRate().AsDecimal()),
					0, SequenceFrameNum - 1);

				if (AnimNotifyStart == PreviousFramesTime)
				{
					UE_LOGFMT(LogAnimDatabase, Warning, "MakeFramesFromAnimNotifyStateStart: Anim Notify State {State} Start is at same time as Previous in Anim Sequence {Name} Time {Time} Frame {Frame}", *State->GetName(), *Sequence->GetName(), AnimNotifyRef.GetNotify()->GetTime(), AnimNotifyStart);
				}
				else
				{
					PreviousFramesTime = AnimNotifyStart;
					
					if (FrameRanges.FrameRangeSet->EntryContains(EntryIdx, AnimNotifyStart))
					{
						FramesToAdd.Add(AnimNotifyStart);
					}
				}
			}
		}

		Out.FrameSet->AddEntry(SequenceIdx, FramesToAdd);
	}

	return Out;
}


FAnimDatabaseFrames UAnimDatabaseFramesLibrary::MakeFramesFromAnimNotifyStateEnd(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TSubclassOf<UAnimNotifyState> State)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFramesLibrary::MakeFramesFromAnimNotifyStateEnd);

	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFramesFromAnimNotifyStateEnd: Database is nullptr.");
		return FAnimDatabaseFrames();
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFramesFromAnimNotify: Invalid FrameRanges Object.");
		return FAnimDatabaseFrames();
	}

	if (!State)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFramesFromAnimNotifyStateEnd: State is nullptr.");
		return FAnimDatabaseFrames();
	}

	FAnimDatabaseFrames Out = MakeEmptyFrames();

	TArray<int32> FramesToAdd;

	const int32 EntryNum = FrameRanges.FrameRangeSet->GetEntryNum();

	for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
	{
		const int32 SequenceIdx = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);

		const UAnimSequence* Sequence = Database->GetAnimSequence(SequenceIdx);
		if (!Sequence) { continue; }

		const int32 SequenceFrameNum = FMath::Max(Database->GetSequenceFrameNum(SequenceIdx), 1);

		FAnimNotifyContext Notifies;
		Sequence->GetAnimNotifies(0.0f, Sequence->GetPlayLength(), Notifies);

		int32 PreviousFramesTime = -1;

		FramesToAdd.Reset();
		for (const FAnimNotifyEventReference& AnimNotifyRef : Notifies.ActiveNotifies)
		{
			if (AnimNotifyRef.GetNotify()->NotifyStateClass && AnimNotifyRef.GetNotify()->NotifyStateClass->IsA(State))
			{
				const int32 AnimNotifyStop = FMath::Clamp(
					FMath::RoundToInt((AnimNotifyRef.GetNotify()->GetTime() + AnimNotifyRef.GetNotify()->GetDuration()) * Database->GetFrameRate().AsDecimal()) - 1,
					0, SequenceFrameNum - 1);

				if (!FrameRanges.FrameRangeSet->EntryContains(EntryIdx, AnimNotifyStop)) { continue; }

				if (AnimNotifyStop == PreviousFramesTime)
				{
					UE_LOGFMT(LogAnimDatabase, Warning, "MakeFramesFromAnimNotifyStateEnd: Anim Notify State {State} End is at same time as Previous in Anim Sequence {Name} Time {Time} Frame {Frame}", *State->GetName(), *Sequence->GetName(), AnimNotifyRef.GetNotify()->GetTime(), AnimNotifyStop);
				}
				else
				{
					PreviousFramesTime = AnimNotifyStop;

					if (FrameRanges.FrameRangeSet->EntryContains(EntryIdx, AnimNotifyStop))
					{
						FramesToAdd.Add(AnimNotifyStop);
					}
				}
			}
		}

		Out.FrameSet->AddEntry(SequenceIdx, FramesToAdd);
	}

	return Out;
}


FAnimDatabaseFrames UAnimDatabaseFramesLibrary::MakeFramesFromAnimNotifyUnion(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArray<TSubclassOf<UAnimNotify>>& Notifies)
{
	return MakeFramesFromAnimNotifyArrayViewUnion(Database, FrameRanges, Notifies);
}

FAnimDatabaseFrames UAnimDatabaseFramesLibrary::MakeFramesFromAnimNotifyArrayViewUnion(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const TSubclassOf<UAnimNotify>> Notifies)
{
	TArray<FAnimDatabaseFrames, TInlineAllocator<32>> Frames;
	Frames.Reserve(Notifies.Num());
	for (const TSubclassOf<UAnimNotify>& Notify : Notifies)
	{
		Frames.Emplace(MakeFramesFromAnimNotify(Database, FrameRanges, Notify));
	}

	return FramesUnionFromArrayView(Frames);
}

FAnimDatabaseFrames UAnimDatabaseFramesLibrary::MakeFramesFromSequenceAndTime(const UAnimDatabase* Database, const int32 SequenceIdx, const float Time)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFramesFromSequenceAndTime: Database is nullptr.");
		return FAnimDatabaseFrames();
	}

	if (SequenceIdx == INDEX_NONE || SequenceIdx < 0 || SequenceIdx >= Database->GetSequenceNum())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFramesFromSequenceAndTime: Invalid Sequence Index.");
		return FAnimDatabaseFrames();
	}

	FAnimDatabaseFrames Out = MakeEmptyFrames();
	Out.FrameSet->AddEntry(SequenceIdx,
		{ (int32)FMath::Clamp(
			FMath::RoundToInt(Time * Database->GetFrameRate().AsDecimal()),
			0, FMath::Max(Database->GetSequenceFrameNum(SequenceIdx), 1) - 1) });

	return Out;
}

FAnimDatabaseFrames UAnimDatabaseFramesLibrary::FramesNone()
{
	return MakeEmptyFrames();
}

bool UAnimDatabaseFramesLibrary::FramesIsEmpty(const FAnimDatabaseFrames& Frames)
{
	if (!Frames.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FramesIsEmpty: Invalid Frames.");
		return true;
	}

	return Frames.FrameSet->IsEmpty();
}

FAnimDatabaseFrames UAnimDatabaseFramesLibrary::FramesBefore(const UAnimDatabase* Database, const FAnimDatabaseFrames& Frames, const int32 FramesBefore, const EAnimDatabaseFrameShiftBehavior ShiftBehavior)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FramesBefore: Database is nullptr.");
		return FAnimDatabaseFrames();
	}

	if (!Frames.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FramesBefore: Invalid Frames.");
		return FAnimDatabaseFrames();
	}

	if (FramesBefore == 0) { return Frames; }

	FAnimDatabaseFrames Out = MakeEmptyFrames();
	
	TArray<int32, TInlineAllocator<32>> AddedFrames;
	
	const int32 EntryNum = Frames.FrameSet->GetEntryNum();
	for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
	{
		const int32 SequenceIdx = Frames.FrameSet->GetEntrySequence(EntryIdx);
		const int32 SequenceFrameNum = Database->GetSequenceFrameNum(SequenceIdx);

		const int32 FrameNum = Frames.FrameSet->GetEntryFrameNum(EntryIdx);

		AddedFrames.Reset();
		for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
		{
			const int32 Frame = Frames.FrameSet->GetEntryFrame(EntryIdx, FrameIdx) - FramesBefore;
			switch (ShiftBehavior)
			{
			case EAnimDatabaseFrameShiftBehavior::Remove:
			{
				if (Frame >= 0 && Frame < SequenceFrameNum)
				{
					AddedFrames.Add(Frame);
				}
				break;
			}
			case EAnimDatabaseFrameShiftBehavior::Clamp:
			{
				AddedFrames.Add(FMath::Clamp(Frame, 0, FMath::Max(SequenceFrameNum - 1, 0)));
				break;
			}
			default:
			{
				checkNoEntry();
			}
			}
		}

		Out.FrameSet->AddEntry(SequenceIdx, AddedFrames);
	}

	Out.FrameSet->Check();

	return Out;
}

FAnimDatabaseFrames UAnimDatabaseFramesLibrary::FramesAfter(const UAnimDatabase* Database, const FAnimDatabaseFrames& Frames, const int32 FramesAfter, const EAnimDatabaseFrameShiftBehavior ShiftBehavior)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FramesAfter: Database is nullptr.");
		return FAnimDatabaseFrames();
	}

	if (!Frames.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FramesAfter: Invalid Frames.");
		return FAnimDatabaseFrames();
	}

	return FramesBefore(Database, Frames, -FramesAfter, ShiftBehavior);
}

bool UAnimDatabaseFramesLibrary::FramesEqual(const FAnimDatabaseFrames& A, const FAnimDatabaseFrames& B)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFramesLibrary::FramesEqual);

	if (!A.IsValid() || !B.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FramesEqual: Invalid Frames.");
		return false;
	}

	return UE::Learning::FrameSet::Equal(*A.FrameSet, *B.FrameSet);
}

FAnimDatabaseFrames UAnimDatabaseFramesLibrary::FramesUnion(FAnimDatabaseFrames A, FAnimDatabaseFrames B)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFramesLibrary::FramesUnion);

	if (!A.IsValid() || !B.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FramesUnion: Invalid Frames.");
		return FAnimDatabaseFrames();
	}

	FAnimDatabaseFrames Out = MakeEmptyFrames();
	UE::Learning::FrameSet::Union(*Out.FrameSet, *A.FrameSet, *B.FrameSet);
	return Out;
}

FAnimDatabaseFrames UAnimDatabaseFramesLibrary::FramesIntersection(FAnimDatabaseFrames A, FAnimDatabaseFrames B)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFramesLibrary::FramesIntersection);

	if (!A.IsValid() || !B.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FramesIntersection: Invalid Frames.");
		return FAnimDatabaseFrames();
	}

	FAnimDatabaseFrames Out = MakeEmptyFrames();
	UE::Learning::FrameSet::Intersection(*Out.FrameSet, *A.FrameSet, *B.FrameSet);
	return Out;
}

FAnimDatabaseFrames UAnimDatabaseFramesLibrary::FramesDifference(FAnimDatabaseFrames A, FAnimDatabaseFrames B)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFramesLibrary::FramesDifference);

	if (!A.IsValid() || !B.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FramesDifference: Invalid Frames.");
		return FAnimDatabaseFrames();
	}

	FAnimDatabaseFrames Out = MakeEmptyFrames();
	UE::Learning::FrameSet::Difference(*Out.FrameSet, *A.FrameSet, *B.FrameSet);
	return Out;
}

FAnimDatabaseFrames UAnimDatabaseFramesLibrary::FramesUnionFromArrayView(const TArrayView<const FAnimDatabaseFrames> Frames)
{
	const int32 FramesNum = Frames.Num();

	if (FramesNum == 0)
	{
		return MakeEmptyFrames();
	}
	else
	{
		FAnimDatabaseFrames Out = Frames[0];
		for (int32 FramesIdx = 1; FramesIdx < FramesNum; FramesIdx++)
		{
			Out = FramesUnion(Out, Frames[FramesIdx]);
		}
		return Out;
	}
}

FAnimDatabaseFrames UAnimDatabaseFramesLibrary::FramesUnionFromArray(const TArray<FAnimDatabaseFrames>& Frames)
{
	return FramesUnionFromArrayView(Frames);
}

FAnimDatabaseFrames UAnimDatabaseFramesLibrary::FramesIntersectionFromArrayView(const TArrayView<const FAnimDatabaseFrames> Frames)
{
	const int32 FramesNum = Frames.Num();

	if (FramesNum == 0)
	{
		return MakeEmptyFrames();
	}
	else
	{
		FAnimDatabaseFrames Out = Frames[0];
		for (int32 FramesIdx = 1; FramesIdx < FramesNum; FramesIdx++)
		{
			Out = FramesIntersection(Out, Frames[FramesIdx]);
		}
		return Out;
	}
}

FAnimDatabaseFrames UAnimDatabaseFramesLibrary::FramesIntersectionFromArray(const TArray<FAnimDatabaseFrames>& Frames)
{
	return FramesIntersectionFromArrayView(Frames);
}

FAnimDatabaseFrames UAnimDatabaseFramesLibrary::FramesDifferenceFromArrayView(const TArrayView<const FAnimDatabaseFrames> Frames)
{
	const int32 FramesNum = Frames.Num();

	if (FramesNum == 0)
	{
		return MakeEmptyFrames();
	}
	else
	{
		FAnimDatabaseFrames Out = Frames[0];
		for (int32 FramesIdx = 1; FramesIdx < FramesNum; FramesIdx++)
		{
			Out = FramesDifference(Out, Frames[FramesIdx]);
		}
		return Out;
	}
}

FAnimDatabaseFrames UAnimDatabaseFramesLibrary::FramesDifferenceFromArray(const TArray<FAnimDatabaseFrames>& Frames)
{
	return FramesDifferenceFromArrayView(Frames);
}

FString UAnimDatabaseFramesLibrary::FramesToString(const FAnimDatabaseFrames& Frames)
{
	if (!Frames.IsValid()) { return TEXT("Invalid"); }

	return FString::Printf(TEXT("EntryNum=%i TotalFrameNum=%i"), Frames.FrameSet->GetEntryNum(), Frames.FrameSet->GetTotalFrameNum());
}

FString UAnimDatabaseFramesLibrary::FramesToStringFormat(const UAnimDatabase* Database, const FAnimDatabaseFrames& Frames, const int32 Cutoff)
{
	if (!Frames.IsValid()) { return TEXT("Invalid"); }

	const int32 RoundedCuttoff = (Cutoff / 2) * 2 + 1;

	FString Output = FString::Printf(TEXT("EntryNum=%i TotalFrameNum=%i Data=[\n"), Frames.FrameSet->GetEntryNum(), Frames.FrameSet->GetTotalFrameNum());
	const int32 EntryNum = Frames.FrameSet->GetEntryNum();
	for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
	{
		const int32 SequenceIdx = Frames.FrameSet->GetEntrySequence(EntryIdx);
		const int32 FrameNum = Frames.FrameSet->GetEntryFrameNum(EntryIdx);
		
		if (Database)
		{
			Output += FString::Printf(TEXT("   %s%s ["),
				*Database->GetSequenceAssetName(SequenceIdx),
				Database->GetIsMirrored(SequenceIdx) ? TEXT(" (mirrored)") : TEXT(""));
		}
		else
		{
			Output += FString::Printf(TEXT("   Sequence %i ["), SequenceIdx);

		}

		if (Cutoff < 0 || FrameNum < RoundedCuttoff)
		{
			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				Output += FString::FromInt(Frames.FrameSet->GetEntryFrame(EntryIdx, FrameIdx));
				if (FrameIdx != FrameNum - 1) { Output += TEXT(" "); }
			}
		}
		else
		{
			for (int32 FrameIdx = 0; FrameIdx < RoundedCuttoff / 2; FrameIdx++)
			{
				Output += FString::FromInt(Frames.FrameSet->GetEntryFrame(EntryIdx, FrameIdx));
				Output += TEXT(" ");
			}

			Output += TEXT("... ");

			for (int32 FrameIdx = FrameNum - RoundedCuttoff / 2 - 1; FrameIdx < FrameNum; FrameIdx++)
			{
				Output += FString::FromInt(Frames.FrameSet->GetEntryFrame(EntryIdx, FrameIdx));
				if (FrameIdx != FrameNum - 1) { Output += TEXT(" "); }
			}
		}

		Output += TEXT("]\n");
	}
	Output += TEXT("]");

	return Output;
}

FAnimDatabaseFrames UAnimDatabaseFramesFunction::MakeFrames_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFramesLibrary::MakeEmptyFrames();
}

FAnimDatabaseFrames UAnimDatabaseFramesFunction_Empty::MakeFrames_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFramesLibrary::MakeEmptyFrames();
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_AnimSequence::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAnimSequence(Database, FrameRanges, AnimSequence);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_AnimSequences::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAnimSequences(Database, FrameRanges, AnimSequences);
}

FAnimDatabaseFrames UAnimDatabaseFramesFunction_AnimNotify::MakeFrames_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFramesLibrary::MakeFramesFromAnimNotify(Database, FrameRanges, AnimNotify);
}

FAnimDatabaseFrames UAnimDatabaseFramesFunction_AnimNotifiesOnTrack::MakeFrames_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFramesLibrary::MakeFramesFromAnimNotifiesOnTrack(Database, FrameRanges, TrackName);
}

FAnimDatabaseFrames UAnimDatabaseFramesFunction_SyncMarker::MakeFrames_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFramesLibrary::MakeFramesFromSyncMarker(Database, FrameRanges, SyncMarker);
}

FAnimDatabaseFrames UAnimDatabaseFramesFunction_FrameRangeStarts::MakeFrames_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameRangesLibrary::MakeFramesAtFrameRangesStarts(UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFunction(Database, FrameRanges, Function));
}

FAnimDatabaseFrames UAnimDatabaseFramesFunction_FrameRangeEnds::MakeFrames_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameRangesLibrary::MakeFramesAtFrameRangesEnds(UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFunction(Database, FrameRanges, Function));
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeEmptyFrameRanges()
{
	FAnimDatabaseFrameRanges Out;
	Out.FrameRangeSet = MakeShared<UE::Learning::FFrameRangeSet>();
	return Out;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromDatabase(const UAnimDatabase* Database)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromDatabase: Database is nullptr.");
		return FAnimDatabaseFrameRanges();
	}

	const int32 SequenceNum = Database->GetSequenceNum();

	FAnimDatabaseFrameRanges Out = MakeEmptyFrameRanges();

	for (int32 SequenceIdx = 0; SequenceIdx < SequenceNum; SequenceIdx++)
	{
		const int32 SequenceLength = Database->GetSequenceFrameNum(SequenceIdx);

		if (Database->GetAnimSequence(SequenceIdx) && SequenceLength > 0)
		{
			Out.FrameRangeSet->AddEntry(SequenceIdx, { 0 }, { SequenceLength });
		}
	}

	Out.FrameRangeSet->Check();
	return Out;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromNotMirrored(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromNotMirrored: Database is nullptr.");
		return FAnimDatabaseFrameRanges();
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromNotMirrored: Invalid FrameRanges Object.");
		return FAnimDatabaseFrameRanges();
	}

	if (!Database->GetMirrorDataTable())
	{
		return FrameRanges;
	}

	FAnimDatabaseFrameRanges Out = MakeEmptyFrameRanges();

	const int32 EntryNum = FrameRanges.FrameRangeSet->GetEntryNum();

	for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
	{
		const int32 SequenceIdx = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);

		if (Database->GetAnimSequence(SequenceIdx) && !Database->GetIsMirrored(SequenceIdx))
		{
			Out.FrameRangeSet->AddEntry(
				SequenceIdx,
				FrameRanges.FrameRangeSet->GetEntryRangeStarts(EntryIdx),
				FrameRanges.FrameRangeSet->GetEntryRangeLengths(EntryIdx));
		}
	}

	Out.FrameRangeSet->Check();
	return Out;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromMirrored(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromMirrored: Database is nullptr.");
		return FAnimDatabaseFrameRanges();
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromMirrored: Invalid FrameRanges Object.");
		return FAnimDatabaseFrameRanges();
	}

	if (!Database->GetMirrorDataTable())
	{
		return MakeEmptyFrameRanges();
	}

	FAnimDatabaseFrameRanges Out = MakeEmptyFrameRanges();

	const int32 EntryNum = FrameRanges.FrameRangeSet->GetEntryNum();

	for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
	{
		const int32 SequenceIdx = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);

		if (Database->GetAnimSequence(SequenceIdx) && Database->GetIsMirrored(SequenceIdx))
		{
			Out.FrameRangeSet->AddEntry(
				SequenceIdx,
				FrameRanges.FrameRangeSet->GetEntryRangeStarts(EntryIdx),
				FrameRanges.FrameRangeSet->GetEntryRangeLengths(EntryIdx));
		}
	}

	Out.FrameRangeSet->Check();
	return Out;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromRootMotionEnabled(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromRootMotionEnabled: Database is nullptr.");
		return FAnimDatabaseFrameRanges();
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromRootMotionEnabled: Invalid FrameRanges Object.");
		return FAnimDatabaseFrameRanges();
	}

	FAnimDatabaseFrameRanges Out = MakeEmptyFrameRanges();

	const int32 EntryNum = FrameRanges.FrameRangeSet->GetEntryNum();

	for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
	{
		const int32 SequenceIdx = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);

		if (const UAnimSequence* AnimSequence = Database->GetAnimSequence(SequenceIdx))
		{
			if (AnimSequence->bEnableRootMotion)
			{
				Out.FrameRangeSet->AddEntry(
					SequenceIdx,
					FrameRanges.FrameRangeSet->GetEntryRangeStarts(EntryIdx),
					FrameRanges.FrameRangeSet->GetEntryRangeLengths(EntryIdx));
			}
		}
	}

	Out.FrameRangeSet->Check();
	return Out;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromRootMotionDisabled(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromRootMotionDisabled: Database is nullptr.");
		return FAnimDatabaseFrameRanges();
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromRootMotionDisabled: Invalid FrameRanges Object.");
		return FAnimDatabaseFrameRanges();
	}

	FAnimDatabaseFrameRanges Out = MakeEmptyFrameRanges();

	const int32 EntryNum = FrameRanges.FrameRangeSet->GetEntryNum();

	for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
	{
		const int32 SequenceIdx = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);

		if (const UAnimSequence* AnimSequence = Database->GetAnimSequence(SequenceIdx))
		{
			if (!AnimSequence->bEnableRootMotion)
			{
				Out.FrameRangeSet->AddEntry(
					SequenceIdx,
					FrameRanges.FrameRangeSet->GetEntryRangeStarts(EntryIdx),
					FrameRanges.FrameRangeSet->GetEntryRangeLengths(EntryIdx));
			}
		}
	}

	Out.FrameRangeSet->Check();
	return Out;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromForceRootLockEnabled(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromRootMotionDisabled: Database is nullptr.");
		return FAnimDatabaseFrameRanges();
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromRootMotionDisabled: Invalid FrameRanges Object.");
		return FAnimDatabaseFrameRanges();
	}

	FAnimDatabaseFrameRanges Out = MakeEmptyFrameRanges();

	const int32 EntryNum = FrameRanges.FrameRangeSet->GetEntryNum();

	for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
	{
		const int32 SequenceIdx = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);

		if (const UAnimSequence* AnimSequence = Database->GetAnimSequence(SequenceIdx))
		{
			if (AnimSequence->bForceRootLock)
			{
				Out.FrameRangeSet->AddEntry(
					SequenceIdx,
					FrameRanges.FrameRangeSet->GetEntryRangeStarts(EntryIdx),
					FrameRanges.FrameRangeSet->GetEntryRangeLengths(EntryIdx));
			}
		}
	}

	Out.FrameRangeSet->Check();
	return Out;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromForceRootLockDisabled(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromRootMotionDisabled: Database is nullptr.");
		return FAnimDatabaseFrameRanges();
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromRootMotionDisabled: Invalid FrameRanges Object.");
		return FAnimDatabaseFrameRanges();
	}

	FAnimDatabaseFrameRanges Out = MakeEmptyFrameRanges();

	const int32 EntryNum = FrameRanges.FrameRangeSet->GetEntryNum();

	for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
	{
		const int32 SequenceIdx = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);

		if (const UAnimSequence* AnimSequence = Database->GetAnimSequence(SequenceIdx))
		{
			if (!AnimSequence->bForceRootLock)
			{
				Out.FrameRangeSet->AddEntry(
					SequenceIdx,
					FrameRanges.FrameRangeSet->GetEntryRangeStarts(EntryIdx),
					FrameRanges.FrameRangeSet->GetEntryRangeLengths(EntryIdx));
			}
		}
	}

	Out.FrameRangeSet->Check();
	return Out;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromNotLooped(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromNotLooped: Database is nullptr.");
		return FAnimDatabaseFrameRanges();
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromNotLooped: Invalid FrameRanges Object.");
		return FAnimDatabaseFrameRanges();
	}

	FAnimDatabaseFrameRanges Out = MakeEmptyFrameRanges();

	const int32 EntryNum = FrameRanges.FrameRangeSet->GetEntryNum();

	for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
	{
		const int32 SequenceIdx = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);

		if (const UAnimSequence* AnimSequence = Database->GetAnimSequence(SequenceIdx))
		{
			if (!AnimSequence->bLoop)
			{
				Out.FrameRangeSet->AddEntry(
					SequenceIdx,
					FrameRanges.FrameRangeSet->GetEntryRangeStarts(EntryIdx),
					FrameRanges.FrameRangeSet->GetEntryRangeLengths(EntryIdx));
			}
		}
	}

	Out.FrameRangeSet->Check();
	return Out;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromLooped(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromLooped: Database is nullptr.");
		return FAnimDatabaseFrameRanges();
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromLooped: Invalid FrameRanges Object.");
		return FAnimDatabaseFrameRanges();
	}

	FAnimDatabaseFrameRanges Out = MakeEmptyFrameRanges();

	const int32 EntryNum = FrameRanges.FrameRangeSet->GetEntryNum();

	for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
	{
		const int32 SequenceIdx = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);

		if (const UAnimSequence* AnimSequence = Database->GetAnimSequence(SequenceIdx))
		{
			if (AnimSequence->bLoop)
			{
				Out.FrameRangeSet->AddEntry(
					SequenceIdx,
					FrameRanges.FrameRangeSet->GetEntryRangeStarts(EntryIdx),
					FrameRanges.FrameRangeSet->GetEntryRangeLengths(EntryIdx));
			}
		}
	}

	Out.FrameRangeSet->Check();
	return Out;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromSequenceIndex(const UAnimDatabase* Database, const int32 SequenceIdx)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromSequenceIndex: Database is nullptr.");
		return FAnimDatabaseFrameRanges();
	}

	if (SequenceIdx == INDEX_NONE || SequenceIdx < 0 || SequenceIdx >= Database->GetSequenceNum())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromSequenceIndex: Could not make tag given invalid Sequence Index for Database.");
		return FAnimDatabaseFrameRanges();
	}

	FAnimDatabaseFrameRanges Out = MakeEmptyFrameRanges();

	const int32 SequenceLength = Database->GetSequenceFrameNum(SequenceIdx);

	if (Database->GetAnimSequence(SequenceIdx) && SequenceLength > 0)
	{
		Out.FrameRangeSet->AddEntry(SequenceIdx, { 0 }, { SequenceLength });
	}

	Out.FrameRangeSet->Check();
	return Out;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAnimSequence(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, UAnimSequence* AnimSequence)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromSequence: Database is nullptr.");
		return FAnimDatabaseFrameRanges();
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromSequence: Invalid FrameRanges Object.");
		return FAnimDatabaseFrameRanges();
	}

	if (!AnimSequence)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "Could not make FrameRanges from Null AnimSequence.");
		return FAnimDatabaseFrameRanges();
	}

	const int32 UnmirroredSequenceIdx = Database->FindSequenceIndex(AnimSequence, false);
	const int32 MirroredSequenceIdx = Database->FindSequenceIndex(AnimSequence, true);

	const bool bContainsUnmirrored = FrameRanges.FrameRangeSet->FindSequenceEntry(UnmirroredSequenceIdx) != INDEX_NONE;
	const bool bContainsMirrored = FrameRanges.FrameRangeSet->FindSequenceEntry(MirroredSequenceIdx) != INDEX_NONE;

	if (bContainsUnmirrored && bContainsMirrored)
	{
		return FrameRangesUnionFromArrayView({
			MakeFrameRangesFromSequenceIndex(Database, UnmirroredSequenceIdx),
			MakeFrameRangesFromSequenceIndex(Database, MirroredSequenceIdx)
			});
	}
	else if (bContainsUnmirrored)
	{
		return MakeFrameRangesFromSequenceIndex(Database, UnmirroredSequenceIdx);
	}
	else if (bContainsMirrored)
	{
		return MakeFrameRangesFromSequenceIndex(Database, MirroredSequenceIdx);
	}
	else
	{
		return MakeEmptyFrameRanges();
	}
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAnimSequences(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArray<UAnimSequence*>& AnimSequences)
{
	return MakeFrameRangesFromAnimSequencesArrayView(Database, FrameRanges, AnimSequences);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAnimSequencesArrayView(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<UAnimSequence* const> AnimSequences)
{
	TArray<FAnimDatabaseFrameRanges, TInlineAllocator<64>> FrameRangesSet;
	FrameRangesSet.Reserve(AnimSequences.Num());

	for (UAnimSequence* AnimSequence : AnimSequences)
	{
		FrameRangesSet.Add(MakeFrameRangesFromAnimSequence(Database, FrameRanges, AnimSequence));
	}

	return FrameRangesUnionFromArrayView(FrameRangesSet);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromSequenceRange(const UAnimDatabase* Database, const int32 SequenceIdx, const int32 RangeStart, const int32 RangeLength)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromSequenceRange: Database is nullptr.");
		return FAnimDatabaseFrameRanges();
	}

	if (SequenceIdx == INDEX_NONE || SequenceIdx < 0 || SequenceIdx >= Database->GetSequenceNum())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromSequenceRange: Could not make tag given invalid Sequence Index for Database.");
		return FAnimDatabaseFrameRanges();
	}

	if (!Database->GetAnimSequence(SequenceIdx))
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromSequenceRange: Sequence is null.");
		return FAnimDatabaseFrameRanges();
	}

	const int32 SequenceLength = Database->GetSequenceFrameNum(SequenceIdx);

	if (RangeStart < 0 || RangeStart + RangeLength > SequenceLength)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromSequenceRange: Invalid range for given sequence.");
		return FAnimDatabaseFrameRanges();
	}

	FAnimDatabaseFrameRanges Out = MakeEmptyFrameRanges();
	Out.FrameRangeSet->AddEntry(SequenceIdx, { RangeStart }, { RangeLength });
	Out.FrameRangeSet->Check();
	return Out;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAssetName(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FString& AssetName)
{
	return MakeFrameRangesFromAssetNameStringView(Database, FrameRanges, AssetName);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAssetNameStringView(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FStringView AssetName)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromAssetNameStringView: Database is nullptr.");
		return FAnimDatabaseFrameRanges();
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromAssetNameStringView: Invalid FrameRanges Object.");
		return FAnimDatabaseFrameRanges();
	}

	FAnimDatabaseFrameRanges Out = MakeEmptyFrameRanges();

	const int32 EntryNum = FrameRanges.FrameRangeSet->GetEntryNum();

	for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
	{
		const int32 SequenceIdx = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);

		if (const UAnimSequence* AnimSequence = Database->GetAnimSequence(SequenceIdx))
		{
			if (AnimSequence->GetName() == AssetName)
			{
				Out.FrameRangeSet->AddEntry(
					SequenceIdx,
					FrameRanges.FrameRangeSet->GetEntryRangeStarts(EntryIdx),
					FrameRanges.FrameRangeSet->GetEntryRangeLengths(EntryIdx));
			}
		}
	}

	Out.FrameRangeSet->Check();
	return Out;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAssetNameStartsWith(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FString& Prefix)
{
	return MakeFrameRangesFromAssetNameStartsWithStringView(Database, FrameRanges, Prefix);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAssetNameStartsWithStringView(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FStringView Prefix)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromAssetNameStartsWithStringView: Database is nullptr.");
		return FAnimDatabaseFrameRanges();
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromAssetNameStartsWithStringView: Invalid FrameRanges Object.");
		return FAnimDatabaseFrameRanges();
	}

	FAnimDatabaseFrameRanges Out = MakeEmptyFrameRanges();

	const int32 EntryNum = FrameRanges.FrameRangeSet->GetEntryNum();

	for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
	{
		const int32 SequenceIdx = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);

		if (const UAnimSequence* AnimSequence = Database->GetAnimSequence(SequenceIdx))
		{
			if (AnimSequence->GetName().StartsWith(Prefix))
			{
				Out.FrameRangeSet->AddEntry(
					SequenceIdx,
					FrameRanges.FrameRangeSet->GetEntryRangeStarts(EntryIdx),
					FrameRanges.FrameRangeSet->GetEntryRangeLengths(EntryIdx));
			}
		}
	}

	Out.FrameRangeSet->Check();
	return Out;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAssetNameEndsWith(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FString& Suffix)
{
	return MakeFrameRangesFromAssetNameEndsWithStringView(Database, FrameRanges, Suffix);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAssetNameEndsWithStringView(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FStringView Suffix)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromAssetNameEndsWithStringView: Database is nullptr.");
		return FAnimDatabaseFrameRanges();
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromAssetNameEndsWithStringView: Invalid FrameRanges Object.");
		return FAnimDatabaseFrameRanges();
	}

	FAnimDatabaseFrameRanges Out = MakeEmptyFrameRanges();

	const int32 EntryNum = FrameRanges.FrameRangeSet->GetEntryNum();

	for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
	{
		const int32 SequenceIdx = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);

		if (const UAnimSequence* AnimSequence = Database->GetAnimSequence(SequenceIdx))
		{
			if (AnimSequence->GetName().EndsWith(Suffix))
			{
				Out.FrameRangeSet->AddEntry(
					SequenceIdx,
					FrameRanges.FrameRangeSet->GetEntryRangeStarts(EntryIdx),
					FrameRanges.FrameRangeSet->GetEntryRangeLengths(EntryIdx));
			}
		}
	}

	Out.FrameRangeSet->Check();
	return Out;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAssetNameContains(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FString& Substring)
{
	return MakeFrameRangesFromAssetNameContainsStringView(Database, FrameRanges, Substring);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAssetNameContainsStringView(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FStringView Substring)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromAssetNameContainsStringView: Database is nullptr.");
		return FAnimDatabaseFrameRanges();
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromAssetNameEndsWithStringView: Invalid FrameRanges Object.");
		return FAnimDatabaseFrameRanges();
	}

	FAnimDatabaseFrameRanges Out = MakeEmptyFrameRanges();

	const int32 EntryNum = FrameRanges.FrameRangeSet->GetEntryNum();

	for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
	{
		const int32 SequenceIdx = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);

		if (const UAnimSequence* AnimSequence = Database->GetAnimSequence(SequenceIdx))
		{
			if (Substring.IsEmpty() || AnimSequence->GetName().Contains(Substring))
			{
				Out.FrameRangeSet->AddEntry(
					SequenceIdx,
					FrameRanges.FrameRangeSet->GetEntryRangeStarts(EntryIdx),
					FrameRanges.FrameRangeSet->GetEntryRangeLengths(EntryIdx));
			}
		}
	}

	Out.FrameRangeSet->Check();
	return Out;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAssetNameStartsWithOneOf(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArray<FString>& Prefixes)
{
	return MakeFrameRangesFromAssetNameStartsWithOneOfArrayView(Database, FrameRanges, Prefixes);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAssetNameStartsWithOneOfArrayView(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const FString> Prefixes)
{
	TArray<FAnimDatabaseFrameRanges, TInlineAllocator<64>> FrameRangesSet;
	FrameRangesSet.Reserve(Prefixes.Num());

	for (const FString& Prefix : Prefixes)
	{
		FrameRangesSet.Add(MakeFrameRangesFromAssetNameStartsWithStringView(Database, FrameRanges, Prefix));
	}

	return FrameRangesUnionFromArrayView(FrameRangesSet);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAssetNameEndsWithOneOf(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArray<FString>& Suffixes)
{
	return MakeFrameRangesFromAssetNameEndsWithOneOfArrayView(Database, FrameRanges, Suffixes);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAssetNameEndsWithOneOfArrayView(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const FString> Suffixes)
{
	TArray<FAnimDatabaseFrameRanges, TInlineAllocator<64>> FrameRangesSet;
	FrameRangesSet.Reserve(Suffixes.Num());

	for (const FString& Suffix : Suffixes)
	{
		FrameRangesSet.Add(MakeFrameRangesFromAssetNameEndsWithStringView(Database, FrameRanges, Suffix));
	}

	return FrameRangesUnionFromArrayView(FrameRangesSet);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAssetNameContainsOneOf(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArray<FString>& Substrings)
{
	return MakeFrameRangesFromAssetNameContainsOneOfArrayView(Database, FrameRanges, Substrings);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAssetNameContainsOneOfArrayView(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const FString> Substrings)
{
	TArray<FAnimDatabaseFrameRanges, TInlineAllocator<64>> FrameRangesSet;
	FrameRangesSet.Reserve(Substrings.Num());

	for (const FString& Substring : Substrings)
	{
		FrameRangesSet.Add(MakeFrameRangesFromAssetNameContainsStringView(Database, FrameRanges, Substring));
	}

	return FrameRangesUnionFromArrayView(FrameRangesSet);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAssetPackageContains(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FString& Substring)
{
	return MakeFrameRangesFromAssetPackageContainsStringView(Database, FrameRanges, Substring);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAssetPackageContainsStringView(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FStringView Substring)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromAssetPackageContainsStringView: Database is nullptr.");
		return FAnimDatabaseFrameRanges();
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromAssetPackageContainsStringView: Invalid FrameRanges Object.");
		return FAnimDatabaseFrameRanges();
	}

	FAnimDatabaseFrameRanges Out = MakeEmptyFrameRanges();

	const int32 EntryNum = FrameRanges.FrameRangeSet->GetEntryNum();

	for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
	{
		const int32 SequenceIdx = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);

		if (const UAnimSequence* AnimSequence = Database->GetAnimSequence(SequenceIdx))
		{
			if (Substring.IsEmpty() || AnimSequence->GetPathName().Contains(Substring))
			{
				Out.FrameRangeSet->AddEntry(
					SequenceIdx,
					FrameRanges.FrameRangeSet->GetEntryRangeStarts(EntryIdx),
					FrameRanges.FrameRangeSet->GetEntryRangeLengths(EntryIdx));
			}
		}
	}

	Out.FrameRangeSet->Check();
	return Out;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAssetPackageContainsOneOf(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArray<FString>& Substring)
{
	return MakeFrameRangesFromAssetPackageContainsOneOfArrayView(Database, FrameRanges, Substring);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAssetPackageContainsOneOfArrayView(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const FString> Substrings)
{
	TArray<FAnimDatabaseFrameRanges, TInlineAllocator<64>> FrameRangesSet;
	FrameRangesSet.Reserve(Substrings.Num());

	for (const FString& Substring : Substrings)
	{
		FrameRangesSet.Add(MakeFrameRangesFromAssetPackageContainsStringView(Database, FrameRanges, Substring));
	}

	return FrameRangesUnionFromArrayView(FrameRangesSet);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromSequences(const UAnimDatabase* Database, const TArray<UAnimSequence*>& AnimSequences)
{
	return MakeFrameRangesFromSequencesArrayView(Database, AnimSequences);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromSequencesArrayView(const UAnimDatabase* Database, const TArrayView<UAnimSequence* const> AnimSequences)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromSequencesArrayView: Database is nullptr.");
		return FAnimDatabaseFrameRanges();
	}

	const int32 SequenceNum = AnimSequences.Num();

	TArray<FAnimDatabaseFrameRanges, TInlineAllocator<64>> FrameRanges;
	FrameRanges.Reserve(Database->GetMirrorDataTable() ? SequenceNum * 2 : SequenceNum);

	for (UAnimSequence* AnimSequence : AnimSequences)
	{
		if (!AnimSequence)
		{
			UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromSequencesArrayView: Could not make FrameRanges from Null AnimSequence.");
			continue;
		}

		const int32 UnmirroredSequenceIdx = Database->FindSequenceIndex(AnimSequence, false);

		if (UnmirroredSequenceIdx == INDEX_NONE)
		{
			UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromSequencesArrayView: Could not find Anim Sequence '{Name}' in Database.", *AnimSequence->GetName());
			continue;
		}

		if (Database->GetMirrorDataTable())
		{
			const int32 MirroredSequenceIdx = Database->FindSequenceIndex(AnimSequence, true);
			check(MirroredSequenceIdx != INDEX_NONE);
			FrameRanges.Add(MakeFrameRangesFromSequenceIndex(Database, UnmirroredSequenceIdx));
			FrameRanges.Add(MakeFrameRangesFromSequenceIndex(Database, MirroredSequenceIdx));
		}
		else
		{
			FrameRanges.Add(MakeFrameRangesFromSequenceIndex(Database, UnmirroredSequenceIdx));
		}
	}

	return FrameRangesUnionFromArrayView(FrameRanges);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFunction(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const UAnimDatabaseFrameRangesFunction* Function)
{
	if (!Function)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromFunction: Function is null.");
		return FAnimDatabaseFrameRanges();
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromFunction: Invalid FrameRanges Object.");
		return FAnimDatabaseFrameRanges();
	}

	const FAnimDatabaseFrameRanges OutFrameRanges = Function->MakeFrameRanges(Database, FrameRanges);
	if (!OutFrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromFunction: Invalid FrameRanges Object.");
		return FAnimDatabaseFrameRanges();
	}

	return OutFrameRanges;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromClass(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TSubclassOf<UAnimDatabaseFrameRangesFunction> Class)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromClass: Database is nullptr.");
		return FAnimDatabaseFrameRanges();
	}

	if (!Class)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromClass: Class is null.");
		return FAnimDatabaseFrameRanges();
	}

	return MakeFrameRangesFromFunction(Database, FrameRanges, Class->GetDefaultObject<UAnimDatabaseFrameRangesFunction>());
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAnimNotifyState(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TSubclassOf<UAnimNotifyState> State)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAnimNotifyState);

	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromAnimNotifyState: Database is nullptr.");
		return FAnimDatabaseFrameRanges();
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromAnimNotifyState: Invalid FrameRanges Object.");
		return FAnimDatabaseFrameRanges();
	}

	if (!State)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromAnimNotifyState: State is nullptr.");
		return FAnimDatabaseFrameRanges();
	}

	UE::Learning::FFrameRangeSet Tmp;

	TArray<FAnimNotifyEventReference, TInlineAllocator<64>> SortedReferences;

	TArray<int32, TInlineAllocator<32>> RangeStartsToAdd;
	TArray<int32, TInlineAllocator<32>> RangeLengthsToAdd;

	const int32 EntryNum = FrameRanges.FrameRangeSet->GetEntryNum();

	for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
	{
		const int32 SequenceIdx = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);

		const UAnimSequence* Sequence = Database->GetAnimSequence(SequenceIdx);
		if (!Sequence) { continue; }

		const int32 SequenceFrameNum = FMath::Max(Database->GetSequenceFrameNum(SequenceIdx), 1);

		FAnimNotifyContext Notifies;
		Sequence->GetAnimNotifies(0.0f, Sequence->GetPlayLength(), Notifies);

		SortedReferences = Notifies.ActiveNotifies;
		SortedReferences.Sort([](const FAnimNotifyEventReference& A, const FAnimNotifyEventReference& B) { return A.GetNotify()->GetTime() < B.GetNotify()->GetTime(); });

		int32 PrevAnimNotifyStopFrame = -1;

		RangeStartsToAdd.Reset();
		RangeLengthsToAdd.Reset();
		for (const FAnimNotifyEventReference& AnimNotifyRef : SortedReferences)
		{
			if (AnimNotifyRef.GetNotify()->NotifyStateClass && AnimNotifyRef.GetNotify()->NotifyStateClass->IsA(State))
			{
				int32 AnimNotifyStart = FMath::Clamp(
					FMath::RoundToInt(AnimNotifyRef.GetNotify()->GetTime() * Database->GetFrameRate().AsDecimal()),
					0, SequenceFrameNum - 1);

				int32 AnimNotifyStop = FMath::Clamp(
					FMath::RoundToInt((AnimNotifyRef.GetNotify()->GetTime() + AnimNotifyRef.GetNotify()->GetDuration()) * Database->GetFrameRate().AsDecimal() + 1.0f),
					0, SequenceFrameNum);

				if (AnimNotifyStart < PrevAnimNotifyStopFrame)
				{
					UE_LOGFMT(LogAnimDatabase, Warning, "MakeFrameRangesFromAnimNotifyState: Anim Notify State {Name} Overlaps with Previous in Anim Sequence {Sequence} Time {Time}", *State->GetName(), *Sequence->GetName(), AnimNotifyRef.GetNotify()->GetTime());
					AnimNotifyStart = PrevAnimNotifyStopFrame;
					AnimNotifyStop = FMath::Max(AnimNotifyStop, AnimNotifyStart + 1);
				}

				if (AnimNotifyStop > AnimNotifyStart)
				{
					if (FrameRanges.FrameRangeSet->EntryIntersectsRange(EntryIdx, AnimNotifyStart, AnimNotifyStop - AnimNotifyStart))
					{
						RangeStartsToAdd.Add(AnimNotifyStart);
						RangeLengthsToAdd.Add(AnimNotifyStop - AnimNotifyStart);
					}

					PrevAnimNotifyStopFrame = AnimNotifyStop;
				}
				else
				{
					UE_LOGFMT(LogAnimDatabase, Warning, "MakeFrameRangesFromAnimNotifyState: Anim Notify State {Name} is less than a single frame in duration in Anim Sequence {Sequence} Time {Time}", *State->GetName(), *Sequence->GetName(), AnimNotifyRef.GetNotify()->GetTime());
				}
			}
		}

		Tmp.AddEntry(SequenceIdx, RangeStartsToAdd, RangeLengthsToAdd);
	}

	Tmp.Check();
	
	// Perform a final intersection since some of the added ranges might go outside the ranges of the input FrameRanges

	FAnimDatabaseFrameRanges Out = MakeEmptyFrameRanges();
	UE::Learning::FrameRangeSet::Intersection(*Out.FrameRangeSet, *FrameRanges.FrameRangeSet, Tmp);

	return Out;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAnimNotifyStateNotActive(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TSubclassOf<UAnimNotifyState> State)
{
	return FrameRangesDifference(FrameRanges, MakeFrameRangesFromAnimNotifyState(Database, FrameRanges, State));
}

void UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAnimNotifyStates(TArray<FAnimDatabaseFrameRanges>& OutFrameRanges, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArray<TSubclassOf<UAnimNotifyState>>& States)
{
	OutFrameRanges.SetNum(States.Num());
	MakeFrameRangesFromAnimNotifyStatesToArrayView(OutFrameRanges, Database, FrameRanges, States);
}

void UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAnimNotifyStatesToArrayView(const TArrayView<FAnimDatabaseFrameRanges> OutFrameRanges, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const TSubclassOf<UAnimNotifyState>> States)
{
	check(OutFrameRanges.Num() == States.Num());
	const int32 StateNum = States.Num();
	for (int32 StateIdx = 0; StateIdx < StateNum; StateIdx++)
	{
		OutFrameRanges[StateIdx] = MakeFrameRangesFromAnimNotifyState(Database, FrameRanges, States[StateIdx]);
	}
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAnimNotifyStatesUnion(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArray<TSubclassOf<UAnimNotifyState>>& States)
{
	return MakeFrameRangesFromAnimNotifyStatesArrayViewUnion(Database, FrameRanges, States);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAnimNotifyStatesArrayViewUnion(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const TSubclassOf<UAnimNotifyState>> States)
{
	TArray<FAnimDatabaseFrameRanges, TInlineAllocator<32>> FrameRangesSet;
	FrameRangesSet.Reserve(States.Num());
	for (const TSubclassOf<UAnimNotifyState>& State : States)
	{
		FrameRangesSet.Emplace(MakeFrameRangesFromAnimNotifyState(Database, FrameRanges, State));
	}

	return FrameRangesUnionFromArrayView(FrameRangesSet);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAnimNotifyStatesIntersection(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArray<TSubclassOf<UAnimNotifyState>>& States)
{
	return MakeFrameRangesFromAnimNotifyStatesArrayViewIntersection(Database, FrameRanges, States);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAnimNotifyStatesArrayViewIntersection(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const TSubclassOf<UAnimNotifyState>> States)
{
	TArray<FAnimDatabaseFrameRanges, TInlineAllocator<32>> FrameRangesSet;
	FrameRangesSet.Reserve(States.Num());
	for (const TSubclassOf<UAnimNotifyState>& State : States)
	{
		FrameRangesSet.Emplace(MakeFrameRangesFromAnimNotifyState(Database, FrameRanges, State));
	}

	return FrameRangesIntersectionFromArrayView(FrameRangesSet);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFrames(const FAnimDatabaseFrames& Frames)
{
	if (!Frames.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromFrames: Invalid Frames.");
		return FAnimDatabaseFrameRanges();
	}

	FAnimDatabaseFrameRanges Out = MakeEmptyFrameRanges();
	UE::Learning::FrameRangeSet::MakeFromFrameSet(*Out.FrameRangeSet, *Frames.FrameSet);
	Out.FrameRangeSet->Check();
	return Out;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAfterFrames(const FAnimDatabaseFrameRanges& FrameRanges, const FAnimDatabaseFrames& Frames)
{
	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromAfterFrames: Invalid FrameRanges.");
		return FAnimDatabaseFrameRanges();
	}

	if (!Frames.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromAfterFrames: Invalid Frames.");
		return FAnimDatabaseFrameRanges();
	}

	FAnimDatabaseFrameRanges Out = MakeEmptyFrameRanges();
	UE::Learning::FrameRangeSet::RangesAfterFrameSet(*Out.FrameRangeSet, *FrameRanges.FrameRangeSet, *Frames.FrameSet);
	Out.FrameRangeSet->Check();
	return Out;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromBeforeFrames(const FAnimDatabaseFrameRanges& FrameRanges, const FAnimDatabaseFrames& Frames)
{
	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromAfterFrames: Invalid FrameRanges.");
		return FAnimDatabaseFrameRanges();
	}

	if (!Frames.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFrameRangesFromAfterFrames: Invalid Frames.");
		return FAnimDatabaseFrameRanges();
	}

	FAnimDatabaseFrameRanges Out = MakeEmptyFrameRanges();
	UE::Learning::FrameRangeSet::RangesBeforeFrameSet(*Out.FrameRangeSet, *FrameRanges.FrameRangeSet, *Frames.FrameSet);
	Out.FrameRangeSet->Check();
	return Out;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAnimNotify(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TSubclassOf<UAnimNotify> Notify)
{
	return MakeFrameRangesFromFrames(UAnimDatabaseFramesLibrary::MakeFramesFromAnimNotify(Database, FrameRanges, Notify));
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromSyncMarker(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName SyncMarkerName)
{
	return MakeFrameRangesFromFrames(UAnimDatabaseFramesLibrary::MakeFramesFromSyncMarker(Database, FrameRanges, SyncMarkerName));
}

FAnimDatabaseFrames UAnimDatabaseFrameRangesLibrary::MakeFramesAtFrameRangesStarts(const FAnimDatabaseFrameRanges& FrameRanges)
{
	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFramesAtFrameRangesStarts: Invalid FrameRanges.");
		return FAnimDatabaseFrames();
	}

	FAnimDatabaseFrames Out = UAnimDatabaseFramesLibrary::MakeEmptyFrames();
	UE::Learning::FrameRangeSet::MakeFrameSetFromRangeStarts(*Out.FrameSet, *FrameRanges.FrameRangeSet);
	Out.FrameSet->Check();
	return Out;
}

FAnimDatabaseFrames UAnimDatabaseFrameRangesLibrary::MakeFramesAtFrameRangesEnds(const FAnimDatabaseFrameRanges& FrameRanges)
{
	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFramesAtFrameRangesEnds: Invalid FrameRanges.");
		return FAnimDatabaseFrames();
	}

	FAnimDatabaseFrames Out = UAnimDatabaseFramesLibrary::MakeEmptyFrames();
	UE::Learning::FrameRangeSet::MakeFrameSetFromRangeEnds(*Out.FrameSet, *FrameRanges.FrameRangeSet);
	Out.FrameSet->Check();
	return Out;
}

FAnimDatabaseFrames UAnimDatabaseFrameRangesLibrary::MakeFramesFromFrameRangesInterval(const FAnimDatabaseFrameRanges& FrameRanges, const int32 IntervalFrames)
{
	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFramesFromFrameRangesInterval: Invalid FrameRanges.");
		return FAnimDatabaseFrames();
	}

	if (IntervalFrames <= 0)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "MakeFramesFromFrameRangesInterval: Invalid IntervalFrames, got {IntervalFrames}, must be >0.", IntervalFrames);
		return FAnimDatabaseFrames();
	}

	FAnimDatabaseFrames Out = UAnimDatabaseFramesLibrary::MakeEmptyFrames();
	UE::Learning::FrameRangeSet::MakeFrameSetFromRangeInterval(*Out.FrameSet, *FrameRanges.FrameRangeSet, IntervalFrames);
	Out.FrameSet->Check();
	return Out;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::FrameRangesNone()
{
	return MakeEmptyFrameRanges();
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::FrameRangesAll(const UAnimDatabase* Database)
{
	return MakeFrameRangesFromDatabase(Database);
}

bool UAnimDatabaseFrameRangesLibrary::FrameRangesIsEmpty(const FAnimDatabaseFrameRanges& FrameRanges)
{
	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameRangesIsEmpty: Invalid FrameRanges.");
		return false;
	}

	return FrameRanges.FrameRangeSet->IsEmpty();
}

bool UAnimDatabaseFrameRangesLibrary::FrameRangesEqual(const FAnimDatabaseFrameRanges& A, const FAnimDatabaseFrameRanges& B)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameRangesLibrary::FrameRangesEqual);

	if (!A.IsValid() || !B.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameRangesEqual: Invalid FrameRanges.");
		return false;
	}

	return UE::Learning::FrameRangeSet::Equal(*A.FrameRangeSet, *B.FrameRangeSet);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::FrameRangesUnion(FAnimDatabaseFrameRanges Lhs, const FAnimDatabaseFrameRanges Rhs)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameRangesLibrary::FrameRangesUnion);

	if (!Lhs.IsValid() || !Rhs.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameRangesUnion: Invalid FrameRanges.");
		return FAnimDatabaseFrameRanges();
	}

	FAnimDatabaseFrameRanges Out = MakeEmptyFrameRanges();
	UE::Learning::FrameRangeSet::Union(*Out.FrameRangeSet, *Lhs.FrameRangeSet, *Rhs.FrameRangeSet);
	Out.FrameRangeSet->Check();
	return Out;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::FrameRangesUnionFromArrayView(const TArrayView<const FAnimDatabaseFrameRanges> FrameRanges)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameRangesLibrary::FrameRangesUnionFromArrayView);

	const int32 FrameRangesNum = FrameRanges.Num();

	if (FrameRangesNum == 0)
	{
		return MakeEmptyFrameRanges();
	}
	else
	{
		FAnimDatabaseFrameRanges Out = FrameRanges[0];
		for (int32 FrameRangesIdx = 1; FrameRangesIdx < FrameRangesNum; FrameRangesIdx++)
		{
			Out = FrameRangesUnion(Out, FrameRanges[FrameRangesIdx]);
		}
		return Out;
	}
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::FrameRangesUnionFromArray(const TArray<FAnimDatabaseFrameRanges>& FrameRanges)
{
	return FrameRangesUnionFromArrayView(FrameRanges);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::FrameRangesIntersection(FAnimDatabaseFrameRanges Lhs, FAnimDatabaseFrameRanges Rhs)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameRangesLibrary::FrameRangesIntersection);

	if (!Lhs.IsValid() || !Rhs.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameRangesIntersection: Invalid FrameRanges.");
		return FAnimDatabaseFrameRanges();
	}

	FAnimDatabaseFrameRanges Out = MakeEmptyFrameRanges();
	UE::Learning::FrameRangeSet::Intersection(*Out.FrameRangeSet, *Lhs.FrameRangeSet, *Rhs.FrameRangeSet);
	Out.FrameRangeSet->Check();
	return Out;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::FrameRangesIntersectionFromArrayView(const TArrayView<const FAnimDatabaseFrameRanges> FrameRanges)
{
	const int32 FrameRangesNum = FrameRanges.Num();

	if (FrameRangesNum == 0)
	{
		return MakeEmptyFrameRanges();
	}
	else
	{
		FAnimDatabaseFrameRanges Out = FrameRanges[0];
		for (int32 FrameRangesIdx = 1; FrameRangesIdx < FrameRangesNum; FrameRangesIdx++)
		{
			Out = FrameRangesIntersection(Out, FrameRanges[FrameRangesIdx]);
		}
		return Out;
	}
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::FrameRangesIntersectionFromArray(const TArray<FAnimDatabaseFrameRanges>& FrameRanges)
{
	return FrameRangesIntersectionFromArrayView(FrameRanges);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::FrameRangesDifference(FAnimDatabaseFrameRanges Lhs, FAnimDatabaseFrameRanges Rhs)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameRangesLibrary::FrameRangesDifference);

	if (!Lhs.IsValid() || !Rhs.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameRangesDifference: Invalid FrameRanges.");
		return FAnimDatabaseFrameRanges();
	}

	FAnimDatabaseFrameRanges Out = MakeEmptyFrameRanges();
	UE::Learning::FrameRangeSet::Difference(*Out.FrameRangeSet, *Lhs.FrameRangeSet, *Rhs.FrameRangeSet);
	Out.FrameRangeSet->Check();
	return Out;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::FrameRangesDifferenceFromArrayView(const TArrayView<const FAnimDatabaseFrameRanges> FrameRanges)
{
	const int32 FrameRangesNum = FrameRanges.Num();

	if (FrameRangesNum == 0)
	{
		return MakeEmptyFrameRanges();
	}

	FAnimDatabaseFrameRanges Out = FrameRanges[0];
	for (int32 FrameRangesIdx = 1; FrameRangesIdx < FrameRangesNum; FrameRangesIdx++)
	{
		Out = FrameRangesDifference(Out, FrameRanges[FrameRangesIdx]);
	}
	return Out;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::FrameRangesDifferenceFromArray(const TArray<FAnimDatabaseFrameRanges>& FrameRanges)
{
	return FrameRangesDifferenceFromArrayView(FrameRanges);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::FrameRangesNot(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges)
{
	return FrameRangesDifference(MakeFrameRangesFromDatabase(Database), FrameRanges);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::FrameRangesBounds(const FAnimDatabaseFrameRanges& FrameRanges)
{
	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameRangesBounds: Invalid FrameRanges.");
		return FAnimDatabaseFrameRanges();
	}

	FAnimDatabaseFrameRanges Out = MakeEmptyFrameRanges();
	UE::Learning::FrameRangeSet::Bounds(*Out.FrameRangeSet, *FrameRanges.FrameRangeSet);
	Out.FrameRangeSet->Check();
	return Out;
}


FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::FrameRangesBoundsFrames(const FAnimDatabaseFrames& Frames)
{
	if (!Frames.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameRangesBoundsFrames: Invalid Frames.");
		return FAnimDatabaseFrameRanges();
	}

	FAnimDatabaseFrameRanges Out = MakeEmptyFrameRanges();
	UE::Learning::FrameRangeSet::Bounds(*Out.FrameRangeSet, *Frames.FrameSet);
	Out.FrameRangeSet->Check();
	return Out;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::FrameRangesIntersects(const FAnimDatabaseFrameRanges& A, const FAnimDatabaseFrameRanges& B)
{
	if (!A.IsValid() || !B.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameRangesIntersects: Invalid Frame Ranges.");
		return FAnimDatabaseFrameRanges();
	}

	FAnimDatabaseFrameRanges Out = MakeEmptyFrameRanges();
	UE::Learning::FrameRangeSet::Intersects(*Out.FrameRangeSet, *A.FrameRangeSet, *B.FrameRangeSet);
	Out.FrameRangeSet->Check();
	return Out;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::FrameRangesIntersectsFrames(const FAnimDatabaseFrameRanges& A, const FAnimDatabaseFrames& B)
{
	if (!A.IsValid() || !B.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameRangesIntersectsFrames: Invalid Frame Ranges or Frames.");
		return FAnimDatabaseFrameRanges();
	}

	FAnimDatabaseFrameRanges Out = MakeEmptyFrameRanges();
	UE::Learning::FrameRangeSet::Intersects(*Out.FrameRangeSet, *A.FrameRangeSet, *B.FrameSet);
	Out.FrameRangeSet->Check();
	return Out;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::FrameRangesTrim(const FAnimDatabaseFrameRanges& A, const int32 TrimStartFrames, const int32 TrimEndFrames)
{
	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameRangesTrim: Invalid FrameRanges.");
		return FAnimDatabaseFrameRanges();
	}

	FAnimDatabaseFrameRanges Out = MakeEmptyFrameRanges();
	UE::Learning::FrameRangeSet::Trim(*Out.FrameRangeSet, *A.FrameRangeSet, FMath::Max(TrimStartFrames, 0), FMath::Max(TrimEndFrames, 0));
	Out.FrameRangeSet->Check();
	return Out;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::FrameRangesTrimStart(const FAnimDatabaseFrameRanges& A, const int32 TrimFrames)
{
	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameRangesTrimStart: Invalid FrameRanges.");
		return FAnimDatabaseFrameRanges();
	}

	FAnimDatabaseFrameRanges Out = MakeEmptyFrameRanges();
	UE::Learning::FrameRangeSet::TrimStart(*Out.FrameRangeSet, *A.FrameRangeSet, FMath::Max(TrimFrames, 0));
	Out.FrameRangeSet->Check();
	return Out;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::FrameRangesTrimEnd(const FAnimDatabaseFrameRanges& A, const int32 TrimFrames)
{
	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameRangesTrimEnd: Invalid FrameRanges.");
		return FAnimDatabaseFrameRanges();
	}

	FAnimDatabaseFrameRanges Out = MakeEmptyFrameRanges();
	UE::Learning::FrameRangeSet::TrimEnd(*Out.FrameRangeSet, *A.FrameRangeSet, FMath::Max(TrimFrames, 0));
	Out.FrameRangeSet->Check();
	return Out;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::FrameRangesPad(const FAnimDatabaseFrameRanges& A, const int32 PadStartFrames, const int32 PadEndFrames)
{
	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameRangesPad: Invalid FrameRanges.");
		return FAnimDatabaseFrameRanges();
	}

	FAnimDatabaseFrameRanges Out = MakeEmptyFrameRanges();
	UE::Learning::FrameRangeSet::Pad(*Out.FrameRangeSet, *A.FrameRangeSet, FMath::Max(PadStartFrames, 0), FMath::Max(PadEndFrames, 0));
	Out.FrameRangeSet->Check();
	return Out;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::FrameRangesPadStart(const FAnimDatabaseFrameRanges& A, const int32 PadFrames)
{
	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameRangesPadStart: Invalid FrameRanges.");
		return FAnimDatabaseFrameRanges();
	}

	FAnimDatabaseFrameRanges Out = MakeEmptyFrameRanges();
	UE::Learning::FrameRangeSet::PadStart(*Out.FrameRangeSet, *A.FrameRangeSet, FMath::Max(PadFrames, 0));
	return Out;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::FrameRangesPadEnd(const FAnimDatabaseFrameRanges& A, const int32 PadFrames)
{
	if (!A.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameRangesPadEnd: Invalid FrameRanges.");
		return FAnimDatabaseFrameRanges();
	}

	FAnimDatabaseFrameRanges Out = MakeEmptyFrameRanges();
	UE::Learning::FrameRangeSet::PadEnd(*Out.FrameRangeSet, *A.FrameRangeSet, FMath::Max(PadFrames, 0));
	Out.FrameRangeSet->Check();
	return Out;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::FrameRangesGatherRanges(const FAnimDatabaseFrameRanges& FrameRanges, const TArray<int32>& RangeIndices)
{
	return FrameRangesGatherRangesFromIndexSet(FrameRanges, RangeIndices);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::FrameRangesGatherRangesFromIndexSet(const FAnimDatabaseFrameRanges& FrameRanges, const UE::Learning::FIndexSet RangeIndices)
{
	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameRangesGatherRangesFromIndexSet: Invalid FrameRanges.");
		return FAnimDatabaseFrameRanges();
	}

	FAnimDatabaseFrameRanges Out = MakeEmptyFrameRanges();
	UE::Learning::FrameRangeSet::GatherRanges(*Out.FrameRangeSet, *FrameRanges.FrameRangeSet, RangeIndices);
	Out.FrameRangeSet->Check();
	return Out;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::FramesFrameRangesUnion(const FAnimDatabaseFrames& Frames, const FAnimDatabaseFrameRanges& FrameRanges)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameRangesLibrary::FramesFrameRangesUnion);

	if (!Frames.IsValid() || !FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FramesFrameRangesUnion: Invalid Frames or FrameRanges.");
		return FAnimDatabaseFrameRanges();
	}

	FAnimDatabaseFrameRanges Out = MakeEmptyFrameRanges();
	UE::Learning::FrameRangeSet::Union(*Out.FrameRangeSet, *Frames.FrameSet, *FrameRanges.FrameRangeSet);
	Out.FrameRangeSet->Check();
	return Out;
}

FAnimDatabaseFrames UAnimDatabaseFrameRangesLibrary::FramesFrameRangesIntersection(const FAnimDatabaseFrames& Frames, const FAnimDatabaseFrameRanges& FrameRanges)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameRangesLibrary::FramesFrameRangesIntersection);

	if (!Frames.IsValid() || !FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FramesFrameRangesIntersection: Invalid Frames or FrameRanges.");
		return FAnimDatabaseFrames();
	}

	FAnimDatabaseFrames Out = UAnimDatabaseFramesLibrary::MakeEmptyFrames();
	UE::Learning::FrameRangeSet::Intersection(*Out.FrameSet, *Frames.FrameSet, *FrameRanges.FrameRangeSet);
	Out.FrameSet->Check();
	return Out;
}

FAnimDatabaseFrames UAnimDatabaseFrameRangesLibrary::FramesFrameRangesDifference(const FAnimDatabaseFrames& Frames, const FAnimDatabaseFrameRanges& FrameRanges)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameRangesLibrary::FramesFrameRangesDifference);

	if (!Frames.IsValid() || !FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FramesFrameRangesDifference: Invalid Frames or FrameRanges.");
		return FAnimDatabaseFrames();
	}

	FAnimDatabaseFrames Out = UAnimDatabaseFramesLibrary::MakeEmptyFrames();
	UE::Learning::FrameRangeSet::Difference(*Out.FrameSet, *Frames.FrameSet, *FrameRanges.FrameRangeSet);
	Out.FrameSet->Check();
	return Out;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::FrameRangesFramesDifference(const FAnimDatabaseFrameRanges& FrameRanges, const FAnimDatabaseFrames& Frames)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameRangesLibrary::FrameRangesFramesDifference);

	if (!Frames.IsValid() || !FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameRangesFramesDifference: Invalid Frames or FrameRanges.");
		return FAnimDatabaseFrameRanges();
	}

	FAnimDatabaseFrameRanges Out = MakeEmptyFrameRanges();
	UE::Learning::FrameRangeSet::Difference(*Out.FrameRangeSet, *FrameRanges.FrameRangeSet, *Frames.FrameSet);
	Out.FrameRangeSet->Check();
	return Out;
}

void UAnimDatabaseFrameRangesLibrary::FrameRangesAnimSequences(TArray<UAnimSequence*>& OutAnimSequences, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameRangesAnimSequences: Database is nullptr.");
		OutAnimSequences.Empty();
		return;
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameRangesAnimSequences: Invalid FrameRanges.");
		OutAnimSequences.Empty();
		return;
	}

	const int32 EntryNum = FrameRanges.FrameRangeSet->GetEntryNum();
	
	OutAnimSequences.Empty(EntryNum);
	for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
	{
		OutAnimSequences.AddUnique(Database->GetAnimSequence(FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx)));
	}
}

void UAnimDatabaseFrameRangesLibrary::FrameRangesAnimSequenceAssets(TArray<UAnimSequence*>& OutAnimSequenceAssets, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameRangesAnimSequenceAssets: Database is nullptr.");
		OutAnimSequenceAssets.Empty();
		return;
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameRangesAnimSequenceAssets: Invalid FrameRanges.");
		OutAnimSequenceAssets.Empty();
		return;
	}

	const int32 EntryNum = FrameRanges.FrameRangeSet->GetEntryNum();

	OutAnimSequenceAssets.Empty(EntryNum);
	for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
	{
		OutAnimSequenceAssets.AddUnique(Database->GetAnimSequence(FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx)));
	}
}

void UAnimDatabaseFrameRangesLibrary::FrameRangesStatistics(FAnimDatabaseFrameRangesStatistics& OutStatistics, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameRangesStatistics: Database is nullptr.");
		OutStatistics = FAnimDatabaseFrameRangesStatistics();
		return;
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameRangesStatistics: Invalid FrameRanges.");
		OutStatistics = FAnimDatabaseFrameRangesStatistics();
		return;
	}

	OutStatistics = FAnimDatabaseFrameRangesStatistics();
	OutStatistics.TotalFrameNum = 0;
	OutStatistics.TotalSequenceNum = 0;
	OutStatistics.TotalRangeNum = 0;
	OutStatistics.MinimumRangeDuration = FrameRanges.FrameRangeSet->GetTotalRangeNum() > 0 ? +UE_MAX_FLT : 0.0f;
	OutStatistics.MaximumRangeDuration = FrameRanges.FrameRangeSet->GetTotalRangeNum() > 0 ? -UE_MAX_FLT : 0.0f;

	const int32 EntryNum = FrameRanges.FrameRangeSet->GetEntryNum();
	const float DeltaTime = 1.0f / FMath::Max(Database->GetFrameRate().AsDecimal(), UE_SMALL_NUMBER);

#if WITH_EDITOR
	IFileManager& FileManager = IFileManager::Get();
#endif

	TSet<const UAnimSequence*> ProcessedSequences;
	ProcessedSequences.Reserve(EntryNum);

	for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
	{
		const int32 RangeNum = FrameRanges.FrameRangeSet->GetEntryRangeNum(EntryIdx);
		const int32 SequenceIdx = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);

		const UAnimSequence* Sequence = Database->GetAnimSequence(SequenceIdx);
		const float TotalSequenceDuration = Database->GetSequenceDuration(SequenceIdx);

		int32 UncompressedSize = 0, CompressedSize = 0, DiskSize = 0;
		if (Sequence && !ProcessedSequences.Contains(Sequence))
		{
#if WITH_EDITOR
			FString PackageFilename;
			if (FPackageName::DoesPackageExist(Sequence->GetOutermost() ? Sequence->GetOutermost()->GetName() : TEXT(""), &PackageFilename))
			{
				DiskSize = FileManager.FileSize(*PackageFilename) / 1000;
			}

			UncompressedSize = Sequence->GetUncompressedRawSize() / 1000;
#endif
			CompressedSize = Sequence->GetApproxCompressedSize() / 1000;
				
			ProcessedSequences.Add(Sequence);
		}

		for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
		{
			const int32 RangeFrameNum = FrameRanges.FrameRangeSet->GetEntryRangeLength(EntryIdx, RangeIdx);
			const float RangeDuration = FrameRanges.FrameRangeSet->GetEntryRangeDuration(EntryIdx, RangeIdx, DeltaTime);

			OutStatistics.TotalDuration += RangeDuration;
			OutStatistics.AverageRangeDuration += RangeDuration;
			OutStatistics.MinimumRangeDuration = FMath::Min(OutStatistics.MinimumRangeDuration, RangeDuration);
			OutStatistics.MaximumRangeDuration = FMath::Max(OutStatistics.MaximumRangeDuration, RangeDuration);

			OutStatistics.DiskSize += DiskSize * (RangeDuration / FMath::Max(TotalSequenceDuration, UE_SMALL_NUMBER));
			OutStatistics.UncompressedSize += UncompressedSize * (RangeDuration / FMath::Max(TotalSequenceDuration, UE_SMALL_NUMBER));
			OutStatistics.CompressedSize += CompressedSize * (RangeDuration / FMath::Max(TotalSequenceDuration, UE_SMALL_NUMBER));
			OutStatistics.TotalFrameNum += RangeFrameNum;
			OutStatistics.TotalRangeNum++;
		}

		OutStatistics.TotalSequenceNum++;
	}

	if (OutStatistics.TotalRangeNum > 0)
	{
		OutStatistics.AverageRangeDuration /= OutStatistics.TotalRangeNum;
	}

	FindAnimNotifyClassesInFrameRanges(OutStatistics.AnimNotifies, Database, FrameRanges);
	FindAnimNotifyStateClassesInFrameRanges(OutStatistics.AnimNotifyStates, Database, FrameRanges);
}

int32 UAnimDatabaseFrameRangesLibrary::FrameRangesContentHash(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAnimDatabaseFrameRangesLibrary::FrameRangesContentHash);

	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameRangesContentHash: Database is nullptr.");
		return 0;
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameRangesContentHash: Invalid FrameRanges.");
		return 0;
	}

	int32 FrameRangesHash = 0x3e360b83;

	const int32 FrameRateHash = CityHash32((const char*)&Database->FrameRate, sizeof(Database->FrameRate));

	const int32 EntryNum = FrameRanges.FrameRangeSet->GetEntryNum();
	for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
	{
		const int32 SequenceIdx = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);
		int32 SequenceHash = 0x93ca112a;

		if (UAnimSequence* Sequence = Database->GetAnimSequence(SequenceIdx))
		{
#if WITH_EDITOR
			if (Sequence->IsCompressedDataOutOfDate()) { Sequence->WaitOnExistingCompression(true); }
			const FIoHash EntryIoHash = Sequence->GetDerivedDataKeyHash(nullptr);
#else
			const FIoHash EntryIoHash = FIoHash();
#endif	
			SequenceHash = CityHash32((const char*)&EntryIoHash, sizeof(EntryIoHash));
		}

		const int32 MirrorHash = Database->GetIsMirrored(SequenceIdx) ? 0xe24e5cc5 : 0xd11c53a0;

		const int32 RangeNum = FrameRanges.FrameRangeSet->GetEntryRangeNum(EntryIdx);

		for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
		{
			const int32 RangeProperties[5] =
			{
				SequenceHash,
				MirrorHash,
				FrameRateHash,
				FrameRanges.FrameRangeSet->GetEntryRangeStart(EntryIdx, RangeIdx),
				FrameRanges.FrameRangeSet->GetEntryRangeLength(EntryIdx, RangeIdx),
			};

			FrameRangesHash ^= CityHash32((const char*)&RangeProperties, sizeof(RangeProperties));
		}
	}

	return FrameRangesHash;
}

bool UAnimDatabaseFrameRangesLibrary::FrameRangesContainsSequence(const FAnimDatabaseFrameRanges& FrameRanges, const int32 SequenceIdx)
{
	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameRangesContainsSequence: Invalid FrameRanges.");
		return false;
	}

	return FrameRanges.FrameRangeSet->ContainsSequence(SequenceIdx);
}

bool UAnimDatabaseFrameRangesLibrary::FrameRangesContains(const FAnimDatabaseFrameRanges& FrameRanges, const int32 SequenceIdx, const int32 FrameIdx)
{
	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameRangesContains: Invalid FrameRanges.");
		return false;
	}

	return FrameRanges.FrameRangeSet->Contains(SequenceIdx, FrameIdx);
}

bool UAnimDatabaseFrameRangesLibrary::FrameRangesContainsTime(const FAnimDatabaseFrameRanges& FrameRanges, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate)
{
	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameRangesContainsTime: Invalid FrameRanges.");
		return false;
	}

	return FrameRanges.FrameRangeSet->ContainsTime(SequenceIdx, SequenceTime, 1.0f / FrameRate.AsDecimal());
}

int32 UAnimDatabaseFrameRangesLibrary::FrameRangesTotalRangeNum(const FAnimDatabaseFrameRanges& FrameRanges)
{
	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameRangesTotalRangeNum: Invalid FrameRanges.");
		return 0;
	}

	return FrameRanges.FrameRangeSet->GetTotalRangeNum();
}

int32 UAnimDatabaseFrameRangesLibrary::FrameRangesTotalFrameNum(const FAnimDatabaseFrameRanges& FrameRanges)
{
	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameRangesTotalFrameNum: Invalid FrameRanges.");
		return 0;
	}

	return FrameRanges.FrameRangeSet->GetTotalFrameNum();
}

float UAnimDatabaseFrameRangesLibrary::FrameRangesAverageFrameNum(const FAnimDatabaseFrameRanges& FrameRanges)
{
	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameRangesAverageFrameNum: Invalid FrameRanges.");
		return 0;
	}

	return FrameRanges.FrameRangeSet->GetAverageFrameNum();
}

int32 UAnimDatabaseFrameRangesLibrary::FrameRangesMinFrameNum(const FAnimDatabaseFrameRanges& FrameRanges)
{
	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameRangesMinFrameNum: Invalid FrameRanges.");
		return 0;
	}

	return FrameRanges.FrameRangeSet->GetMinFrameNum();
}

int32 UAnimDatabaseFrameRangesLibrary::FrameRangesMaxFrameNum(const FAnimDatabaseFrameRanges& FrameRanges)
{
	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameRangesMaxFrameNum: Invalid FrameRanges.");
		return 0;
	}

	return FrameRanges.FrameRangeSet->GetMaxFrameNum();
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesLibrary::FrameRangesRangeAtIndex(const FAnimDatabaseFrameRanges& FrameRanges, const int32 RangeIdx)
{
	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameRangesRangeAtIndex: Invalid FrameRanges.");
		return FAnimDatabaseFrameRanges();
	}

	if (RangeIdx < 0 || RangeIdx >= FrameRanges.FrameRangeSet->GetTotalRangeNum())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FrameRangesRangeAtIndex: Index {RangeIdx} out of bounds. Must be >= 0 and < {TotalNum}.", RangeIdx, FrameRanges.FrameRangeSet->GetTotalRangeNum());
		return FAnimDatabaseFrameRanges();
	}

	int32 EntryIdx = INDEX_NONE;
	int32 EntryRangeIdx = INDEX_NONE;
	FrameRanges.FrameRangeSet->FindTotalRange(EntryIdx, EntryRangeIdx, RangeIdx);
	check(EntryIdx != INDEX_NONE && EntryRangeIdx != INDEX_NONE);

	FAnimDatabaseFrameRanges Out = MakeEmptyFrameRanges();
	Out.FrameRangeSet->AddEntry(
		FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx),
		{ FrameRanges.FrameRangeSet->GetEntryRangeStart(EntryIdx, EntryRangeIdx) },
		{ FrameRanges.FrameRangeSet->GetEntryRangeLength(EntryIdx, EntryRangeIdx) });

	return Out;
}

FString UAnimDatabaseFrameRangesLibrary::FrameRangesToString(const FAnimDatabaseFrameRanges& FrameRanges)
{
	if (!FrameRanges.IsValid()) { return TEXT("Invalid"); }

	return FString::Printf(TEXT("EntryNum=%i TotalRangeNum=%i TotalFrameNum=%i"), 
		FrameRanges.FrameRangeSet->GetEntryNum(), FrameRanges.FrameRangeSet->GetTotalRangeNum(), FrameRanges.FrameRangeSet->GetTotalFrameNum());
}

FString UAnimDatabaseFrameRangesLibrary::FrameRangesToStringFormat(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const int32 Cutoff)
{
	if (!FrameRanges.IsValid()) { return TEXT("Invalid"); }

	const int32 RoundedCuttoff = (Cutoff / 2) * 2 + 1;

	FString Output = FString::Printf(TEXT("EntryNum=%i TotalRangeNum=%i TotalFrameNum=%i Data=[\n"),
		FrameRanges.FrameRangeSet->GetEntryNum(), FrameRanges.FrameRangeSet->GetTotalRangeNum(), FrameRanges.FrameRangeSet->GetTotalFrameNum());

	const int32 EntryNum = FrameRanges.FrameRangeSet->GetEntryNum();
	for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
	{
		const int32 SequenceIdx = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);
		const int32 RangeNum = FrameRanges.FrameRangeSet->GetEntryRangeNum(EntryIdx);

		if (Database)
		{
			Output += FString::Printf(TEXT("   %s%s ["),
				*Database->GetSequenceAssetName(SequenceIdx),
				Database->GetIsMirrored(SequenceIdx) ? TEXT(" (mirrored)") : TEXT(""));
		}
		else
		{
			Output += FString::Printf(TEXT("   Sequence %i ["), SequenceIdx);

		}
		
		if (Cutoff < 0 || RangeNum < RoundedCuttoff)
		{
			for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
			{
				Output += FString::Printf(TEXT("%i:%i"),
					FrameRanges.FrameRangeSet->GetEntryRangeStart(EntryIdx, RangeIdx),
					FrameRanges.FrameRangeSet->GetEntryRangeStop(EntryIdx, RangeIdx));
				if (RangeIdx != RangeNum - 1) { Output += TEXT(" "); }
			}
		}
		else
		{
			for (int32 RangeIdx = 0; RangeIdx < RoundedCuttoff / 2; RangeIdx++)
			{
				Output += FString::Printf(TEXT("%i:%i"),
					FrameRanges.FrameRangeSet->GetEntryRangeStart(EntryIdx, RangeIdx),
					FrameRanges.FrameRangeSet->GetEntryRangeStop(EntryIdx, RangeIdx));
				Output += TEXT(" ");
			}

			Output += TEXT("... ");

			for (int32 RangeIdx = RangeNum - RoundedCuttoff / 2 - 1; RangeIdx < RangeNum; RangeIdx++)
			{
				Output += FString::Printf(TEXT("%i:%i"),
					FrameRanges.FrameRangeSet->GetEntryRangeStart(EntryIdx, RangeIdx),
					FrameRanges.FrameRangeSet->GetEntryRangeStop(EntryIdx, RangeIdx));
				if (RangeIdx != RangeNum - 1) { Output += TEXT(" "); }
			}
		}

		Output += TEXT("]\n");
	}
	Output += TEXT("]");

	return Output;
}

void UAnimDatabaseFrameRangesLibrary::FindAnimNotifyClassesInFrameRanges(TArray<TSubclassOf<UAnimNotify>>& OutAnimNotifyClasses, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FindAnimNotifyClassesInFrameRanges: Database is nullptr.");
		OutAnimNotifyClasses.Empty();
		return;
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FindAnimNotifyClassesInFrameRanges: Invalid FrameRanges.");
		OutAnimNotifyClasses.Empty();
		return;
	}

	OutAnimNotifyClasses.Reset();

	const int32 EntryNum = FrameRanges.FrameRangeSet->GetEntryNum();
	for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
	{
		const int32 SequenceIdx = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);
		const UAnimSequence* Sequence = Database->GetAnimSequence(SequenceIdx);
		if (!Sequence) { continue; }

		const int32 SequenceFrameNum = FMath::Max(Database->GetSequenceFrameNum(SequenceIdx), 1);

		FAnimNotifyContext Notifies;
		Sequence->GetAnimNotifies(0.0f, Sequence->GetPlayLength(), Notifies);

		for (const FAnimNotifyEventReference& AnimNotifyRef : Notifies.ActiveNotifies)
		{
			if (AnimNotifyRef.GetNotify()->Notify)
			{
				const int32 AnimNotifyStart = FMath::Clamp(
					FMath::RoundToInt(AnimNotifyRef.GetNotify()->GetTime() * Database->GetFrameRate().AsDecimal()),
					0, SequenceFrameNum - 1);

				if (FrameRanges.FrameRangeSet->Contains(SequenceIdx, AnimNotifyStart))
				{
					OutAnimNotifyClasses.AddUnique(AnimNotifyRef.GetNotify()->Notify->GetClass());
				}
			}
		}
	}
}

void UAnimDatabaseFrameRangesLibrary::FindAnimNotifyStateClassesInFrameRanges(TArray<TSubclassOf<UAnimNotifyState>>& OutAnimNotifyStateClasses, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FindAnimNotifyStateClassesInFrameRanges: Database is nullptr.");
		OutAnimNotifyStateClasses.Empty();
		return;
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FindAnimNotifyStateClassesInFrameRanges: Invalid FrameRanges.");
		OutAnimNotifyStateClasses.Empty();
		return;
	}

	OutAnimNotifyStateClasses.Reset();

	const int32 EntryNum = FrameRanges.FrameRangeSet->GetEntryNum();
	for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
	{
		const int32 SequenceIdx = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);
		const UAnimSequence* Sequence = Database->GetAnimSequence(SequenceIdx);
		if (!Sequence) { continue; }

		const int32 SequenceFrameNum = FMath::Max(Database->GetSequenceFrameNum(SequenceIdx), 1);

		FAnimNotifyContext Notifies;
		Sequence->GetAnimNotifies(0.0f, Sequence->GetPlayLength(), Notifies);

		for (const FAnimNotifyEventReference& AnimNotifyRef : Notifies.ActiveNotifies)
		{
			if (AnimNotifyRef.GetNotify()->NotifyStateClass)
			{
				const int32 AnimNotifyStart = FMath::Clamp(
					FMath::RoundToInt(AnimNotifyRef.GetNotify()->GetTime() * Database->GetFrameRate().AsDecimal()),
					0, SequenceFrameNum - 1);

				const int32 AnimNotifyStop = FMath::Clamp(
					FMath::RoundToInt((AnimNotifyRef.GetNotify()->GetTime() + AnimNotifyRef.GetNotify()->GetDuration()) * Database->GetFrameRate().AsDecimal() + 1.0f),
					0, SequenceFrameNum);

				if (AnimNotifyStop > AnimNotifyStart && 
					FrameRanges.FrameRangeSet->IntersectsRange(SequenceIdx, AnimNotifyStart, AnimNotifyStop - AnimNotifyStart))
				{
					OutAnimNotifyStateClasses.AddUnique(AnimNotifyRef.GetNotify()->NotifyStateClass->GetClass());
				}
			}
		}
	}
}

void UAnimDatabaseFrameRangesLibrary::FindCurvesInFrameRanges(TArray<FName>& OutCurves, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FindCurvesInFrameRanges: Database is nullptr.");
		OutCurves.Empty();
		return;
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FindCurvesInFrameRanges: Invalid FrameRanges.");
		OutCurves.Empty();
		return;
	}

	OutCurves.Reset();

	const int32 EntryNum = FrameRanges.FrameRangeSet->GetEntryNum();
	for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
	{
		const int32 SequenceIdx = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);
		UAnimSequence* Sequence = Database->GetAnimSequence(SequenceIdx);
		if (!Sequence) { continue; }

		const int32 CurveNum = Sequence->GetCurveData().FloatCurves.Num();
		for (int32 CurveIdx = 0; CurveIdx < CurveNum; CurveIdx++)
		{
			OutCurves.AddUnique(Sequence->GetCurveData().FloatCurves[CurveIdx].GetName());
		}
	}
}

void UAnimDatabaseFrameRangesLibrary::FindSyncMarkersInFrameRanges(TArray<FName>& SyncMarkers, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges)
{
	if (!Database)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FindSyncMarkersInFrameRanges: Database is nullptr.");
		SyncMarkers.Empty();
		return;
	}

	if (!FrameRanges.IsValid())
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FindSyncMarkersInFrameRanges: Invalid FrameRanges.");
		SyncMarkers.Empty();
		return;
	}

	SyncMarkers.Reset();

	const int32 EntryNum = FrameRanges.FrameRangeSet->GetEntryNum();
	for (int32 EntryIdx = 0; EntryIdx < EntryNum; EntryIdx++)
	{
		const int32 SequenceIdx = FrameRanges.FrameRangeSet->GetEntrySequence(EntryIdx);
		const UAnimSequence* Sequence = Database->GetAnimSequence(SequenceIdx);
		if (!Sequence) { continue; }

		const int32 SequenceFrameNum = FMath::Max(Database->GetSequenceFrameNum(SequenceIdx), 1);

		for (const FAnimSyncMarker& SyncMarker : Sequence->AuthoredSyncMarkers)
		{
			const int32 SyncMarkerStartStart = FMath::Clamp(
				FMath::RoundToInt(SyncMarker.Time * Database->GetFrameRate().AsDecimal()),
				0, SequenceFrameNum - 1);

			if (FrameRanges.FrameRangeSet->Contains(SequenceIdx, SyncMarkerStartStart))
			{
				SyncMarkers.AddUnique(SyncMarker.MarkerName);
			}
		}
	}
}

void UAnimDatabaseFrameRangesLibrary::FindAllAnimNotifyStateClasses(TArray<TSubclassOf<UAnimNotifyState>>& OutClasses, const TSubclassOf<UAnimNotifyState> BaseClass)
{
	if (!BaseClass)
	{
		OutClasses.Empty();
		return;
	}

	TArray<UClass*> Subclasses;
	GetDerivedClasses(BaseClass, Subclasses);

	const int32 SubclassNum = Subclasses.Num();
	OutClasses.SetNum(Subclasses.Num());
	for (int32 SubclassIdx = 0; SubclassIdx < SubclassNum; SubclassIdx++)
	{
		OutClasses[SubclassIdx] = Subclasses[SubclassIdx];
	}
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameRangesLibrary::MakeEmptyFrameRanges();
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_Empty::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameRangesLibrary::MakeEmptyFrameRanges();
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_Identity::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return FrameRanges;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_Frames::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFrames(UAnimDatabaseFramesLibrary::MakeFramesFromFunction(Database, FrameRanges, Frames));
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_All::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameRangesLibrary::FrameRangesAll(Database);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_Mirrored::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromMirrored(Database, FrameRanges);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_NotMirrored::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromNotMirrored(Database, FrameRanges);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_RootMotionEnabled::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromRootMotionEnabled(Database, FrameRanges);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_RootMotionDisabled::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromRootMotionDisabled(Database, FrameRanges);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_ForceRootLockEnabled::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromForceRootLockEnabled(Database, FrameRanges);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_ForceRootLockDisabled::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromForceRootLockDisabled(Database, FrameRanges);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_Looped::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromLooped(Database, FrameRanges);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_NotLooped::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromNotLooped(Database, FrameRanges);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_AnimNotify::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAnimNotify(Database, FrameRanges, AnimNotify);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_SyncMarker::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromSyncMarker(Database, FrameRanges, SyncMarker);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_AnimNotifyState::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAnimNotifyState(Database, FrameRanges, AnimNotifyState);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_AnimNotifyStateNotActive::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAnimNotifyStateNotActive(Database, FrameRanges, AnimNotifyState);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_AnimNotifyStateUnion::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAnimNotifyStatesArrayViewUnion(Database, FrameRanges, AnimNotifyStates);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_AnimNotifyStateIntersection::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAnimNotifyStatesArrayViewIntersection(Database, FrameRanges, AnimNotifyStates);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_AnimNotifyStateWithMirrored::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameRangesLibrary::FrameRangesUnionFromArrayView({
		UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAnimNotifyState(Database, UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromNotMirrored(Database, FrameRanges), NotMirroredAnimNotifyState),
		UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAnimNotifyState(Database, UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromMirrored(Database, FrameRanges), MirroredAnimNotifyState)
		});
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_AssetNameContains::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAssetNameContainsStringView(Database, FrameRanges, AssetNameSubstring);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_AssetNameContainsOneOf::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAssetNameContainsOneOfArrayView(Database, FrameRanges, AssetNameSubstrings);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_AssetPackageContains::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAssetPackageContainsStringView(Database, FrameRanges, AssetPackageSubstring);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_AssetPackageContainsOneOf::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAssetPackageContainsOneOfArrayView(Database, FrameRanges, AssetPackageSubstrings);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_AssetNameStartsWith::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAssetNameStartsWithStringView(Database, FrameRanges, AssetNamePrefix);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_AssetNameStartsWithOneOf::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAssetNameStartsWithOneOfArrayView(Database, FrameRanges, AssetNamePrefixes);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_AssetNameEndsWith::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAssetNameEndsWithStringView(Database, FrameRanges, AssetNameSuffix);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_AssetNameEndsWithOneOf::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAssetNameEndsWithOneOfArrayView(Database, FrameRanges, AssetNameSuffixes);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_Filter::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	FAnimDatabaseFrameRanges OutFrameRanges = FrameRanges;

	for (const TObjectPtr<UAnimDatabaseFrameRangesFunction>& Function : Functions)
	{
		if (Function)
		{
			OutFrameRanges = UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFunction(Database, OutFrameRanges, Function);
		}
	}

	return OutFrameRanges;
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_Not::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameRangesLibrary::FrameRangesDifference(FrameRanges, UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFunction(Database, FrameRanges, Function));
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_Union::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	TArray<FAnimDatabaseFrameRanges, TInlineAllocator<8>> FrameRangesSet;
	FrameRangesSet.Reserve(Functions.Num());
	for (const TObjectPtr<UAnimDatabaseFrameRangesFunction>& Function : Functions)
	{
		if (Function)
		{
			FrameRangesSet.Add(UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFunction(Database, FrameRanges, Function));
		}
	}

	return UAnimDatabaseFrameRangesLibrary::FrameRangesUnionFromArrayView(FrameRangesSet);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_Intersection::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	TArray<FAnimDatabaseFrameRanges, TInlineAllocator<8>> FrameRangesSet;
	FrameRangesSet.Reserve(Functions.Num());
	for (const TObjectPtr<UAnimDatabaseFrameRangesFunction>& Function : Functions)
	{
		if (Function)
		{
			FrameRangesSet.Add(UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFunction(Database, FrameRanges, Function));
		}
	}

	return UAnimDatabaseFrameRangesLibrary::FrameRangesIntersectionFromArrayView(FrameRangesSet);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_Difference::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	TArray<FAnimDatabaseFrameRanges, TInlineAllocator<8>> FrameRangesSet;
	FrameRangesSet.Reserve(Functions.Num());
	for (const TObjectPtr<UAnimDatabaseFrameRangesFunction>& Function : Functions)
	{
		if (Function)
		{
			FrameRangesSet.Add(UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFunction(Database, FrameRanges, Function));
		}
	}

	return UAnimDatabaseFrameRangesLibrary::FrameRangesDifferenceFromArrayView(FrameRangesSet);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_Bounds::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameRangesLibrary::FrameRangesBounds(UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFunction(Database, FrameRanges, Function));
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_BoundsFrames::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameRangesLibrary::FrameRangesBoundsFrames(UAnimDatabaseFramesLibrary::MakeFramesFromFunction(Database, FrameRanges, Function));
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_Intersects::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameRangesLibrary::FrameRangesIntersects(FrameRanges, UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFunction(Database, FrameRanges, Function));
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_IntersectsFrames::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameRangesLibrary::FrameRangesIntersectsFrames(FrameRanges, UAnimDatabaseFramesLibrary::MakeFramesFromFunction(Database, FrameRanges, Function));
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_Trim::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameRangesLibrary::FrameRangesTrim(UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFunction(Database, FrameRanges, Function), TrimStart, TrimEnd);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_Pad::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameRangesLibrary::FrameRangesPad(UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFunction(Database, FrameRanges, Function), PadStart, PadEnd);
}

FAnimDatabaseFrameRanges UAnimDatabaseFrameRangesFunction_Interval::MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const
{
	return UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFrames(UAnimDatabaseFrameRangesLibrary::MakeFramesFromFrameRangesInterval(FrameRanges, Interval));
}
