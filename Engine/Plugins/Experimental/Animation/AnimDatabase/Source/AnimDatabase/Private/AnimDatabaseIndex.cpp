// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimDatabaseIndex.h"

#include "AnimDatabase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimDatabaseIndex)

#if WITH_EDITOR

void UAnimDatabaseIndexFunction::BuildIndex_Implementation(
	TMap<FName, FAnimDatabaseFrames>& OutIndexFrames,
	TMap<FName, FAnimDatabaseFrameRanges>& OutIndexFrameRanges,
	TMap<FName, FAnimDatabaseFrameAttribute>& OutIndexFrameAttributes,
	UAnimDatabase* InDatabase,
	const FAnimDatabaseFrameRanges& InFrameRanges) {}

void UAnimDatabaseIndex::Build()
{
	TObjectPtr<UAnimDatabase> BuildDatabase = Database.LoadSynchronous();

	if (!BuildDatabase || !Function)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "UAnimDatabaseIndex: Invalid Database or Build Function.");
		return;
	}

	const FAnimDatabaseFrameRanges FrameRangesObject = !FrameRanges ?
		UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromDatabase(BuildDatabase) :
		UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFunction(BuildDatabase, UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromDatabase(BuildDatabase),FrameRanges);

	IndexDatabase = BuildDatabase;
	IndexContentHash = BuildDatabase->GetContentHash();

	IndexFrames.Reset();
    IndexFrameRanges.Reset();
    IndexFrameAttributes.Reset();

	Function->BuildIndex(
        IndexFrames,
        IndexFrameRanges,
        IndexFrameAttributes,
		BuildDatabase,
		FrameRangesObject);

	Modify(true);
}    

void UAnimDatabaseIndexFunction_Basic::BuildIndex_Implementation(
	TMap<FName, FAnimDatabaseFrames>& OutIndexFrames,
	TMap<FName, FAnimDatabaseFrameRanges>& OutIndexFrameRanges,
	TMap<FName, FAnimDatabaseFrameAttribute>& OutIndexFrameAttributes,
	UAnimDatabase* InDatabase,
	const FAnimDatabaseFrameRanges& InFrameRanges)
{
	const int32 FrameNum = Frames.Num();

	for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
	{
		OutIndexFrames.Add(Frames[FrameIdx].Name,
			UAnimDatabaseFramesLibrary::MakeFramesFromFunction(InDatabase, InFrameRanges, Frames[FrameIdx].Frames));
	}

	const int32 FrameRangeNum = FrameRanges.Num();

	for (int32 FrameRangeIdx = 0; FrameRangeIdx < FrameRangeNum; FrameRangeIdx++)
	{
		OutIndexFrameRanges.Add(FrameRanges[FrameRangeIdx].Name,
			UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFunction(InDatabase, InFrameRanges, FrameRanges[FrameRangeIdx].FrameRanges));
	}

	const int32 FrameAttributeNum = FrameAttributes.Num();

	for (int32 FrameAttributeIdx = 0; FrameAttributeIdx < FrameAttributeNum; FrameAttributeIdx++)
	{
		OutIndexFrameAttributes.Add(FrameAttributes[FrameAttributeIdx].Name,
			UAnimDatabaseFrameAttributeLibrary::MakeFrameAttributeFromFunction(InDatabase, InFrameRanges, FrameAttributes[FrameAttributeIdx].FrameAttribute));
	}
}

void UAnimDatabaseIndexFunction_TransitionPose::BuildIndex_Implementation(
	TMap<FName, FAnimDatabaseFrames>& OutIndexFrames,
	TMap<FName, FAnimDatabaseFrameRanges>& OutIndexFrameRanges,
	TMap<FName, FAnimDatabaseFrameAttribute>& OutIndexFrameAttributes,
	UAnimDatabase* InDatabase,
	const FAnimDatabaseFrameRanges& InFrameRanges)
{
	const FAnimDatabaseFrameRanges TransRanges = UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFunction(InDatabase, InFrameRanges, TransitionRanges);

	OutIndexFrameRanges.Add(TEXT("TransitionRanges"), TransRanges);

	FAnimDatabaseFrameAttribute RootLocation, RootRotation;
	UAnimDatabaseFrameAttributeLibrary::MakeRootLocationAndRotationFrameAttribute(RootLocation, RootRotation, InDatabase, TransRanges);

	FAnimDatabaseFrameAttribute LeftToeLocation, LeftToeVelocity;
	UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalLocationAndLinearVelocityFrameAttributeFromName(LeftToeLocation, LeftToeVelocity, InDatabase, TransRanges, LeftToeBoneName);

	FAnimDatabaseFrameAttribute RightToeLocation, RightToeVelocity;
	UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalLocationAndLinearVelocityFrameAttributeFromName(RightToeLocation, RightToeVelocity, InDatabase, TransRanges, RightToeBoneName);

	LeftToeLocation = UAnimDatabaseFrameAttributeLibrary::FrameAttributeSubtractAndUnrotate(LeftToeLocation, RootLocation, RootRotation);
	LeftToeVelocity = UAnimDatabaseFrameAttributeLibrary::FrameAttributeUnrotate(RootRotation, LeftToeVelocity);

	RightToeLocation = UAnimDatabaseFrameAttributeLibrary::FrameAttributeSubtractAndUnrotate(RightToeLocation, RootLocation, RootRotation);
	RightToeVelocity = UAnimDatabaseFrameAttributeLibrary::FrameAttributeUnrotate(RootRotation, RightToeVelocity);

	OutIndexFrameAttributes.Add(TEXT("LeftToeLocation"), LeftToeLocation);
	OutIndexFrameAttributes.Add(TEXT("LeftToeVelocity"), LeftToeVelocity);
	OutIndexFrameAttributes.Add(TEXT("RightToeLocation"), RightToeLocation);
	OutIndexFrameAttributes.Add(TEXT("RightToeVelocity"), RightToeVelocity);

	const int32 StyleRangeNum = StyleRanges.Num();

	for (int32 StyleRangeIdx = 0; StyleRangeIdx < StyleRangeNum; StyleRangeIdx++)
	{
		OutIndexFrameRanges.Add(StyleRanges[StyleRangeIdx].Name, 
			UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFunction(InDatabase, InFrameRanges, StyleRanges[StyleRangeIdx].FrameRanges));
	}
}

void UAnimDatabaseIndexFunction_AttachBone::BuildIndex_Implementation(
	TMap<FName, FAnimDatabaseFrames>& OutIndexFrames,
	TMap<FName, FAnimDatabaseFrameRanges>& OutIndexFrameRanges,
	TMap<FName, FAnimDatabaseFrameAttribute>& OutIndexFrameAttributes,
	UAnimDatabase* InDatabase,
	const FAnimDatabaseFrameRanges& InFrameRanges)
{
	FAnimDatabaseFrameAttribute RootLocation, RootRotation;
	UAnimDatabaseFrameAttributeLibrary::MakeRootLocationAndRotationFrameAttribute(RootLocation, RootRotation, InDatabase, InFrameRanges);
	
	const FAnimDatabaseFrameAttribute RootLinearVelocity = UAnimDatabaseFrameAttributeLibrary::MakeRootLinearVelocityFrameAttribute(InDatabase, InFrameRanges);

	const FAnimDatabaseFrameAttribute AttachLocation = UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalLocationFrameAttributeFromName(InDatabase, InFrameRanges, AttachBoneName);
	const FAnimDatabaseFrameAttribute AttachDirection = UAnimDatabaseFrameAttributeLibrary::MakeBoneGlobalDirectionFrameAttributeFromName(InDatabase, InFrameRanges, AttachBoneName, AttachForwardVector);

	OutIndexFrameAttributes.Add(TEXT("AttachLocation"), UAnimDatabaseFrameAttributeLibrary::FrameAttributeSubtractAndUnrotate(AttachLocation, RootLocation, RootRotation));
	OutIndexFrameAttributes.Add(TEXT("AttachDirection"), UAnimDatabaseFrameAttributeLibrary::FrameAttributeUnrotate(RootRotation, AttachDirection));
	OutIndexFrameAttributes.Add(TEXT("LocalRootVelocity"), UAnimDatabaseFrameAttributeLibrary::FrameAttributeUnrotate(RootRotation, RootLinearVelocity));
}

#endif

bool UAnimDatabaseIndexLibrary::FindTransitionPoseMatch(
	UAnimSequence*& OutAnimSequence,
	float& OutAnimSequenceTime,
	bool& bOutAnimSequenceMirrored,
	float& OutMinimumValue,
	const UAnimDatabaseIndex* Index,
	const FVector LeftToeBoneLocation,
	const FVector LeftToeBoneVelocity,
	const FVector RightToeBoneLocation,
	const FVector RightToeBoneVelocity,
	const FTransform& RootTransform,
	int32& RandomState,
	const FName StyleName,
	const float BlendTime,
	const float LocationScale,
	const float VelocityScale,
	const float LocationWeight,
	const float VelocityWeight,
	const float RandomWeight)
{
	if (!Index || !Index->IndexDatabase)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FindTransitionPoseMatch: Index or Index Database is nullptr.");
		OutAnimSequence = nullptr;
		OutAnimSequenceTime = 0.0f;
		bOutAnimSequenceMirrored = false;
		OutMinimumValue = 0.0f;
		return false;
	}

	FAnimDatabaseFrameRanges SearchRangesRanges;
	if (const FAnimDatabaseFrameRanges* SearchRangesRangesPtr = Index->IndexFrameRanges.Find(StyleName == NAME_None ? TEXT("TransitionRanges") : StyleName))
	{
		SearchRangesRanges = *SearchRangesRangesPtr;
	}
	else
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FindTransitionPoseMatch: No ranges found.");
		OutAnimSequence = nullptr;
		OutAnimSequenceTime = 0.0f;
		bOutAnimSequenceMirrored = false;
		OutMinimumValue = 0.0f;
		return false;
	}

	const FAnimDatabaseFrameAttribute* LeftToeLocationAttributePtr = Index->IndexFrameAttributes.Find(TEXT("LeftToeLocation"));
	const FAnimDatabaseFrameAttribute* LeftToeVelocityAttributePtr = Index->IndexFrameAttributes.Find(TEXT("LeftToeVelocity"));
	const FAnimDatabaseFrameAttribute* RightToeLocationAttributePtr = Index->IndexFrameAttributes.Find(TEXT("RightToeLocation"));
	const FAnimDatabaseFrameAttribute* RightToeVelocityAttributePtr = Index->IndexFrameAttributes.Find(TEXT("RightToeVelocity"));

	if (!LeftToeLocationAttributePtr || !LeftToeVelocityAttributePtr || !RightToeLocationAttributePtr || !RightToeVelocityAttributePtr)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FindTransitionPoseMatch: Attributes not found.");
		OutAnimSequence = nullptr;
		OutAnimSequenceTime = 0.0f;
		bOutAnimSequenceMirrored = false;
		OutMinimumValue = 0.0f;
		return false;
	}

	const FAnimDatabaseFrameAttribute LeftFootDistance = UAnimDatabaseFrameAttributeLibrary::FrameAttributeInertializationMatchingDistance(
		UAnimDatabaseFrameAttributeLibrary::FrameAttributeIntersection(*LeftToeLocationAttributePtr, SearchRangesRanges),
		UAnimDatabaseFrameAttributeLibrary::FrameAttributeIntersection(*LeftToeVelocityAttributePtr, SearchRangesRanges),
		RootTransform.InverseTransformPosition(LeftToeBoneLocation),
		RootTransform.InverseTransformVector(LeftToeBoneVelocity),
		BlendTime,
		LocationScale,
		VelocityScale,
		LocationWeight,
		VelocityWeight);

	const FAnimDatabaseFrameAttribute RightFootDistance = UAnimDatabaseFrameAttributeLibrary::FrameAttributeInertializationMatchingDistance(
		UAnimDatabaseFrameAttributeLibrary::FrameAttributeIntersection(*RightToeLocationAttributePtr, SearchRangesRanges),
		UAnimDatabaseFrameAttributeLibrary::FrameAttributeIntersection(*RightToeVelocityAttributePtr, SearchRangesRanges),
		RootTransform.InverseTransformPosition(RightToeBoneLocation),
		RootTransform.InverseTransformVector(RightToeBoneVelocity),
		BlendTime,
		LocationScale,
		VelocityScale,
		LocationWeight,
		VelocityWeight);

	const FAnimDatabaseFrameAttribute RandomAttribute = UAnimDatabaseFrameAttributeLibrary::MakeFloatFrameAttributeFromUniformRandom(SearchRangesRanges, RandomState, 0.0f, RandomWeight);

	const FAnimDatabaseFrameAttribute Total = UAnimDatabaseFrameAttributeLibrary::FrameAttributeSumFromArrayView({ LeftFootDistance, RightFootDistance, RandomAttribute });

	UAnimDatabaseFrameAttributeLibrary::FindFrameAttributeMinimumSequence(
		OutAnimSequence,
		OutAnimSequenceTime,
		bOutAnimSequenceMirrored,
		OutMinimumValue,
		Total,
		Index->IndexDatabase);

	return true;
}

bool UAnimDatabaseIndexLibrary::FindAttachBoneMatch(
	UAnimSequence*& OutAnimSequence,
	float& OutAnimSequenceTime,
	bool& bOutAnimSequenceMirrored,
	float& OutMinimumValue,
	const UAnimDatabaseIndex* Index,
	const FVector AttachBoneLocation,
	const FVector AttachBoneDirection,
	const FVector RootLinearVelocity,
	const FTransform& RootTransform,
	const float LocationScale,
	const float VelocityScale,
	const float LocationWeight,
	const float DirectionWeight,
	const float VelocityWeight)
{
	if (!Index || !Index->IndexDatabase)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FindAttachBoneMatch: Index or Index Database is nullptr.");
		OutAnimSequence = nullptr;
		OutAnimSequenceTime = 0.0f;
		bOutAnimSequenceMirrored = false;
		OutMinimumValue = 0.0f;
		return false;
	}

	const FAnimDatabaseFrameAttribute* AttachLocationAttributePtr = Index->IndexFrameAttributes.Find(TEXT("AttachLocation"));
	const FAnimDatabaseFrameAttribute* AttachDirectionAttributePtr = Index->IndexFrameAttributes.Find(TEXT("AttachDirection"));
	const FAnimDatabaseFrameAttribute* LocalRootVelocityAttributePtr = Index->IndexFrameAttributes.Find(TEXT("LocalRootVelocity"));

	if (!AttachLocationAttributePtr || !AttachDirectionAttributePtr || !LocalRootVelocityAttributePtr)
	{
		UE_LOGFMT(LogAnimDatabase, Error, "FindAttachBoneMatch: Attributes not found.");
		OutAnimSequence = nullptr;
		OutAnimSequenceTime = 0.0f;
		bOutAnimSequenceMirrored = false;
		OutMinimumValue = 0.0f;
		return false;
	}

	const FAnimDatabaseFrameAttribute LocationDistance = UAnimDatabaseFrameAttributeLibrary::FrameAttributeLocationMatchingDistance(
		*AttachLocationAttributePtr, 
		RootTransform.InverseTransformPosition(AttachBoneLocation),
		LocationScale, 
		LocationWeight);

	const FAnimDatabaseFrameAttribute DirectionDistance = UAnimDatabaseFrameAttributeLibrary::FrameAttributeDirectionMatchingDistance(
		*AttachDirectionAttributePtr, 
		RootTransform.InverseTransformVectorNoScale(AttachBoneDirection),
		DirectionWeight);

	const FAnimDatabaseFrameAttribute VelocityDistance = UAnimDatabaseFrameAttributeLibrary::FrameAttributeVelocityMatchingDistance(
		*LocalRootVelocityAttributePtr, 
		RootTransform.InverseTransformVector(RootLinearVelocity), 
		VelocityScale, 
		VelocityWeight);

	const FAnimDatabaseFrameAttribute Total = UAnimDatabaseFrameAttributeLibrary::FrameAttributeSumFromArrayView({ LocationDistance, DirectionDistance, VelocityDistance });

	UAnimDatabaseFrameAttributeLibrary::FindFrameAttributeMinimumSequence(
		OutAnimSequence,
		OutAnimSequenceTime,
		bOutAnimSequenceMirrored,
		OutMinimumValue,
		Total,
		Index->IndexDatabase);

	return true;
}

