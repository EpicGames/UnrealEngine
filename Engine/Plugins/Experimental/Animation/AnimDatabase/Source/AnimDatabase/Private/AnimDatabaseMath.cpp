// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimDatabaseMath.h"

#include "Math/Vector.h"
#include "Math/Quat.h"
#include "Math/Transform.h"

#include "LearningRandom.h"

#ifndef UE_ANIMDATABASE_ISPC
#define UE_ANIMDATABASE_ISPC INTEL_ISPC
//#define UE_ANIMDATABASE_ISPC 0
#endif

#if UE_ANIMDATABASE_ISPC
#include "AnimDatabase.ispc.generated.h"
#endif

namespace UE::AnimDatabase::Math
{
	FVector3f VectorMax(const FVector3f V, const float W)
	{
		return FVector3f(
			FMath::Max(V.X, W),
			FMath::Max(V.Y, W),
			FMath::Max(V.Z, W));
	}

	FVector3f VectorMin(const FVector3f V, const float W)
	{
		return FVector3f(
			FMath::Min(V.X, W),
			FMath::Min(V.Y, W),
			FMath::Min(V.Z, W));
	}

	FVector3f VectorMax(const FVector3f V, const FVector3f W)
	{
		return FVector3f(
			FMath::Max(V.X, W.X),
			FMath::Max(V.Y, W.Y),
			FMath::Max(V.Z, W.Z));
	}

	FVector3f VectorMin(const FVector3f V, const FVector3f W)
	{
		return FVector3f(
			FMath::Min(V.X, W.X),
			FMath::Min(V.Y, W.Y),
			FMath::Min(V.Z, W.Z));
	}

	FVector3f VectorDivMax(const float V, const FVector3f W, const float Epsilon)
	{
		return FVector3f(
			V / FMath::Max(W.X, Epsilon),
			V / FMath::Max(W.Y, Epsilon),
			V / FMath::Max(W.Z, Epsilon));
	}

	FVector3f VectorDivMax(const FVector3f V, const FVector3f W, const float Epsilon)
	{
		return FVector3f(
			V.X / FMath::Max(W.X, Epsilon),
			V.Y / FMath::Max(W.Y, Epsilon),
			V.Z / FMath::Max(W.Z, Epsilon));
	}

	FVector3f VectorInvExpApprox(const FVector3f V)
	{
		return FVector3f(
			FMath::InvExpApprox(V.X),
			FMath::InvExpApprox(V.Y),
			FMath::InvExpApprox(V.Z));
	}

	FVector3f VectorEerp(const FVector3f V, const FVector3f W, const float Alpha, const float Epsilon)
	{
		if (FVector3f::DistSquared(V, W) < Epsilon)
		{
			return FVector3f(
				FMath::Lerp(FMath::Max(V.X, Epsilon), FMath::Max(W.X, Epsilon), Alpha),
				FMath::Lerp(FMath::Max(V.Y, Epsilon), FMath::Max(W.Y, Epsilon), Alpha),
				FMath::Lerp(FMath::Max(V.Z, Epsilon), FMath::Max(W.Z, Epsilon), Alpha));
		}
		else
		{
			return FVector3f(
				FMath::Pow(FMath::Max(V.X, Epsilon), (1.0f - Alpha)) * FMath::Pow(FMath::Max(W.X, Epsilon), Alpha),
				FMath::Pow(FMath::Max(V.Y, Epsilon), (1.0f - Alpha)) * FMath::Pow(FMath::Max(W.Y, Epsilon), Alpha),
				FMath::Pow(FMath::Max(V.Z, Epsilon), (1.0f - Alpha)) * FMath::Pow(FMath::Max(W.Z, Epsilon), Alpha));
		}
	}

	FVector3f VectorLog(const FVector3f V)
	{
		return FVector3f(
			FMath::Loge(V.X),
			FMath::Loge(V.Y),
			FMath::Loge(V.Z));
	}

	FVector3f VectorExp(const FVector3f V)
	{
		return FVector3f(
			FMath::Exp(V.X),
			FMath::Exp(V.Y),
			FMath::Exp(V.Z));
	}

	FVector3f VectorLogSafe(const FVector3f V, const float Epsilon)
	{
		return FVector3f(
			FMath::Loge(FMath::Max(V.X, Epsilon)),
			FMath::Loge(FMath::Max(V.Y, Epsilon)),
			FMath::Loge(FMath::Max(V.Z, Epsilon)));
	}

	FVector3f VectorExpSafe(const FVector3f V, const float Max)
	{
		return FVector3f(
			FMath::Exp(FMath::Min(V.X, Max)),
			FMath::Exp(FMath::Min(V.Y, Max)),
			FMath::Exp(FMath::Min(V.Z, Max)));
	}

	FVector3f VectorSqrt(const FVector3f A)
	{
		return FVector3f(FMath::Sqrt(A.X), FMath::Sqrt(A.Y), FMath::Sqrt(A.Z));
	}

	FVector3f VectorGt(const FVector3f A, const float W)
	{
		return FVector3f(A.X > W, A.Y > W, A.Z > W);
	}

	FVector3f VectorSign(const FVector3f A)
	{
		return FVector3f(FMath::Sign(A.X), FMath::Sign(A.Y), FMath::Sign(A.Z));
	}

	FVector3f VectorAbs(const FVector3f A)
	{
		return FVector3f(FMath::Abs(A.X), FMath::Abs(A.Y), FMath::Abs(A.Z));
	}

	FVector3f VectorEq(const FVector3f A, const FVector3f W)
	{
		return FVector3f(A.X == W.X, A.Y == W.Y, A.Z == W.Z);
	}

	float VectorLength(const FVector4f X)
	{
		return FMath::Sqrt(X.X * X.X + X.Y * X.Y + X.Z * X.Z + X.W * X.W);
	}

	FVector4f VectorNormalize(const FVector4f X)
	{
		return X / VectorLength(X);
	}

	FVector VectorLogSafe(const FVector V, const double Epsilon)
	{
		return FVector(
			FMath::Loge(FMath::Max(V.X, Epsilon)),
			FMath::Loge(FMath::Max(V.Y, Epsilon)),
			FMath::Loge(FMath::Max(V.Z, Epsilon)));
	}

	FVector VectorExpSafe(const FVector V, const double Max)
	{
		return FVector(
			FMath::Exp(FMath::Min(V.X, Max)),
			FMath::Exp(FMath::Min(V.Y, Max)),
			FMath::Exp(FMath::Min(V.Z, Max)));
	}

	FQuat4f MakeQuatFromMatrixRightHanded(const FMatrix44f Matrix)
	{
		if (Matrix.M[2][2] < 0.0f)
		{
			if (Matrix.M[0][0] > Matrix.M[1][1])
			{
				return FQuat4f(
					1.0 + Matrix.M[0][0] - Matrix.M[1][1] - Matrix.M[2][2],
					Matrix.M[0][1] + Matrix.M[1][0],
					Matrix.M[2][0] + Matrix.M[0][2],
					Matrix.M[1][2] - Matrix.M[2][1]);
			}
			else
			{
				return FQuat4f(
					Matrix.M[0][1] + Matrix.M[1][0],
					1.0 - Matrix.M[0][0] + Matrix.M[1][1] - Matrix.M[2][2],
					Matrix.M[1][2] + Matrix.M[2][1],
					Matrix.M[2][0] - Matrix.M[0][2]);
			}
		}
		else
		{
			if (Matrix.M[0][0] < -Matrix.M[1][1])
			{
				return FQuat4f(
					Matrix.M[2][0] + Matrix.M[0][2],
					Matrix.M[1][2] + Matrix.M[2][1],
					1.0 - Matrix.M[0][0] - Matrix.M[1][1] + Matrix.M[2][2],
					Matrix.M[0][1] - Matrix.M[1][0]);
			}
			else
			{
				return FQuat4f(
					Matrix.M[1][2] - Matrix.M[2][1],
					Matrix.M[2][0] - Matrix.M[0][2],
					Matrix.M[0][1] - Matrix.M[1][0],
					1.0 + Matrix.M[0][0] + Matrix.M[1][1] + Matrix.M[2][2]);
			}
		}
	}

	void LerpToTargetInplace(
		const TLearningArrayView<1, float> InOut,
		const TLearningArrayView<1, const float> Target,
		const float Alpha)
	{
		check(InOut.Num() == Target.Num());

		const int64 ColNum = InOut.Num();

		if (UE_ANIMDATABASE_ISPC && InOut.NumBytes() < MAX_int32)
		{
#if UE_ANIMDATABASE_ISPC
			ispc::AnimDatabaseLerpToTargetInplace(
				InOut.GetData(),
				Target.GetData(),
				Alpha,
				1,
				ColNum);
#endif
		}
		else
		{
			for (int64 ColIdx = 0; ColIdx < ColNum; ColIdx++)
			{
				InOut[ColIdx] = FMath::Lerp(InOut[ColIdx], Target[ColIdx], Alpha);
			}
		}
	}

	void LerpToTargetInplace(
		const TLearningArrayView<2, float> InOut,
		const TLearningArrayView<2, const float> Target,
		const float Alpha)
	{
		check(InOut.Num<0>() == Target.Num<0>());
		check(InOut.Num<1>() == Target.Num<1>());

		const int64 RowNum = InOut.Num<0>();
		const int64 ColNum = InOut.Num<1>();

		if (UE_ANIMDATABASE_ISPC && InOut.NumBytes() < MAX_int32)
		{
#if UE_ANIMDATABASE_ISPC
			ispc::AnimDatabaseLerpToTargetInplace(
				InOut.GetData(),
				Target.GetData(),
				Alpha,
				RowNum,
				ColNum);
#endif
		}
		else
		{
			for (int64 RowIdx = 0; RowIdx < RowNum; RowIdx++)
			{
				for (int64 ColIdx = 0; ColIdx < ColNum; ColIdx++)
				{
					InOut[RowIdx][ColIdx] = FMath::Lerp(InOut[RowIdx][ColIdx], Target[RowIdx][ColIdx], Alpha);
				}
			}
		}
	}

	float HalfLifeToDamping(const float HalfLife)
	{
		return (4.0f * UE_LN2) / FMath::Max(HalfLife, UE_SMALL_NUMBER);
	}

	float DampingToHalfLife(const float Damping)
	{
		return (4.0 * UE_LN2) / FMath::Max(Damping, UE_SMALL_NUMBER);
	}

	void CriticalSpringUpdate(
		FVector& InOutValue,
		FVector& InOutVelocity,
		const FVector& DesiredValue,
		const float HalfLife,
		const float DeltaTime)
	{
		const float Y = HalfLifeToDamping(HalfLife) / 2.0f;

		const FVector J0 = InOutValue - DesiredValue;
		const FVector J1 = InOutVelocity + J0 * Y;

		const float Eydt = FMath::InvExpApprox(Y * DeltaTime);

		InOutValue = Eydt * (J0 + J1 * DeltaTime) + DesiredValue;
		InOutVelocity = Eydt * (InOutVelocity - J1 * Y * DeltaTime);
	}

	void CriticalSpringUpdate(
		FQuat& InOutValue,
		FVector& InOutVelocity,
		const FQuat& DesiredValue,
		const float HalfLife,
		const float DeltaTime)
	{
		const float Y = HalfLifeToDamping(HalfLife) / 2.0f;

		const FVector J0 = (InOutValue * DesiredValue.Inverse()).GetShortestArcWith(FQuat::Identity).ToRotationVector();
		const FVector J1 = InOutVelocity + J0 * Y;

		const float Eydt = FMath::InvExpApprox(Y * DeltaTime);

		InOutValue = FQuat::MakeFromRotationVector(Eydt * (J0 + J1 * DeltaTime)) * DesiredValue;
		InOutVelocity = Eydt * (InOutVelocity - J1 * Y * DeltaTime);
	}

	void CriticalSpringUpdate(
		FRotator& InOutValue,
		FVector& InOutVelocity,
		const FRotator& DesiredValue,
		const float HalfLife,
		const float DeltaTime)
	{
		FQuat InValueQuat = InOutValue.Quaternion();
		CriticalSpringUpdate(InValueQuat, InOutVelocity, DesiredValue.Quaternion(), HalfLife, DeltaTime);
		InOutValue = InValueQuat.Rotator();
	}

	void CriticalSpringUpdatePositionFromVelocity(
		FVector& InOutPosition,
		FVector& InOutVelocity,
		FVector& InOutAcceleration,
		const FVector& DesiredVelocity,
		const float HalfLife,
		const float DeltaTime)
	{
		const float Y = HalfLifeToDamping(HalfLife) / 2.0f;
		const FVector J0 = InOutVelocity - DesiredVelocity;
		const FVector J1 = InOutAcceleration + J0 * Y;
		const float Eydt = FMath::InvExpApprox(Y * DeltaTime);

		InOutPosition = Eydt * (((-J1) / (Y * Y)) + ((-J0 - J1 * DeltaTime) / Y)) +
			(J1 / (Y * Y)) + J0 / Y + DesiredVelocity * DeltaTime + InOutPosition;
		InOutVelocity = Eydt * (J0 + J1 * DeltaTime) + DesiredVelocity;
		InOutAcceleration = Eydt * (InOutAcceleration - J1 * Y * DeltaTime);
	}

	void ExtrapolateTranslation(
		FVector3f& OutTranslation,
		FVector3f& OutVelocity,
		const FVector3f Translation,
		const FVector3f Velocity,
		const float Time,
		const FVector3f DecayHalflife,
		const float Epsilon)
	{
		if (Velocity.SquaredLength() > Epsilon)
		{
			const FVector3f C = VectorDivMax(UE_LN2, DecayHalflife, Epsilon);
			OutTranslation = Translation + (VectorDivMax(Velocity, C, Epsilon) * (FVector3f::OneVector - VectorInvExpApprox(C * Time)));
			OutVelocity = Velocity * VectorInvExpApprox(C * Time);
		}
		else
		{
			OutTranslation = Translation;
			OutVelocity = FVector3f::ZeroVector;
		}
	}

	void ExtrapolateTranslation(
		FVector& OutTranslation,
		FVector3f& OutVelocity,
		const FVector Translation,
		const FVector3f Velocity,
		const float Time,
		const FVector3f DecayHalflife,
		const float Epsilon)
	{
		if (Velocity.SquaredLength() > Epsilon)
		{
			const FVector3f C = VectorDivMax(UE_LN2, DecayHalflife, Epsilon);
			OutTranslation = Translation + (FVector)(VectorDivMax(Velocity, C, Epsilon) * (FVector3f::OneVector - VectorInvExpApprox(C * Time)));
			OutVelocity = Velocity * VectorInvExpApprox(C * Time);
		}
		else
		{
			OutTranslation = Translation;
			OutVelocity = FVector3f::ZeroVector;
		}
	}

	void ExtrapolateRotation(
		FQuat4f& OutRotation,
		FVector3f& OutVelocity,
		const FQuat4f Rotation,
		const FVector3f Velocity,
		const float Time,
		const FVector3f DecayHalflife,
		const float Epsilon)
	{
		if (Velocity.SquaredLength() > Epsilon)
		{
			const FVector3f C = VectorDivMax(UE_LN2, DecayHalflife, Epsilon);
			OutRotation = FQuat4f::MakeFromRotationVector((VectorDivMax(Velocity, C, Epsilon) * (FVector3f::OneVector - VectorInvExpApprox(C * Time)))) * Rotation;
			check(OutRotation.IsNormalized());
			OutVelocity = Velocity * VectorInvExpApprox(C * Time);
		}
		else
		{
			OutRotation = Rotation;
			check(OutRotation.IsNormalized());
			OutVelocity = FVector3f::ZeroVector;
		}
	}

	void ExtrapolateScale(
		FVector3f& OutScale,
		FVector3f& OutVelocity,
		const FVector3f Scale,
		const FVector3f Velocity,
		const float Time,
		const FVector3f DecayHalflife,
		const float Epsilon)
	{
		if (Velocity.SquaredLength() > Epsilon)
		{
			const FVector3f C = VectorDivMax(UE_LN2, DecayHalflife, Epsilon);
			OutScale = VectorExp((VectorDivMax(Velocity, C, Epsilon) * (FVector3f::OneVector - VectorInvExpApprox(C * Time)))) * Scale;
			OutVelocity = Velocity * VectorInvExpApprox(C * Time);
		}
		else
		{
			OutScale = Scale;
			OutVelocity = FVector3f::ZeroVector;
		}
	}

	void ExtrapolateCurve(
		float& OutCurve,
		float& OutVelocity,
		const float Curve,
		const float Velocity,
		const float Time,
		const float DecayHalflife,
		const float Epsilon)
	{
		if (FMath::Square(Velocity) > Epsilon)
		{
			const float C = UE_LN2 / FMath::Max(DecayHalflife, Epsilon);
			OutCurve = Curve + FMath::Max(Velocity / C, Epsilon) * (1.0f - FMath::InvExpApprox(C * Time));
			OutVelocity = Velocity * FMath::InvExpApprox(C * Time);
		}
		else
		{
			OutCurve = Curve;
			OutVelocity = 0.0f;
		}
	}

	void DecayCubic(
		FVector3f& OutPosition,
		FVector3f& OutVelocity,
		const FVector3f Position,
		const FVector3f Velocity,
		const float Time,
		const float DecayDuration)
	{
		const float T = FMath::Clamp(Time / FMath::Max(DecayDuration, UE_SMALL_NUMBER), 0.0f, 1.0f);

		const float W0 = 2.0f * T * T * T - 3.0f * T * T + 1.0f;
		const float W1 = (T * T * T - 2.0f * T * T + T) * DecayDuration;
		const float W2 = (6.0f * T * T - 6.0f * T) / FMath::Max(DecayDuration, UE_SMALL_NUMBER);
		const float W3 = 3.0f * T * T - 4.0f * T + 1.0f;

		OutPosition = W0 * Position + W1 * Velocity;
		OutVelocity = W2 * Position + W3 * Velocity;
	}

	float ClipMagnitudeToGreaterThanEpsilon(const float X, const float Epsilon)
	{
		return
			X >= 0.0f && X < Epsilon ? Epsilon :
			X <  0.0f && X > -Epsilon ? -Epsilon : X;
	}

	float ComputeDecayHalfLifeFromDiffAndVelocity(
		const float SrcDstDiff,
		const float SrcVelocity,
		const float HalfLife,
		const float HalfLifeMin,
		const float HalfLifeMax,
		const float Epsilon)
	{
		// Essentially what this function does is compute a half-life based on the ratio between the velocity vector and
		// the vector from the source to the destination. This is then clamped to some min and max. If the signs are
		// different (i.e. the velocity and the vector from source to destination are in opposite directions) this will
		// produce a negative number that will get clamped to HalfLifeMin. If the signs match, this will produce a large
		// number when the velocity is small and the vector from source to destination is large, and a small number when
		// the velocity is large and the vector from source to destination is small. This will be clamped either way to 
		// be in the range given by HalfLifeMin and HalfLifeMax. Finally, since the velocity can be close to zero we 
		// have to clamp it to always be greater than some given magnitude (preserving the sign).

		return FMath::Clamp(HalfLife * (SrcDstDiff / ClipMagnitudeToGreaterThanEpsilon(SrcVelocity, Epsilon)), HalfLifeMin, HalfLifeMax);
	}

	FVector3f ComputeDecayHalfLifeFromDiffAndVelocity(
		const FVector3f SrcDstDiff,
		const FVector3f SrcVelocity,
		const float HalfLife,
		const float HalfLifeMin,
		const float HalfLifeMax,
		const float Epsilon)
	{
		return FVector3f(
			ComputeDecayHalfLifeFromDiffAndVelocity(SrcDstDiff.X, SrcVelocity.X, HalfLife, HalfLifeMin, HalfLifeMax, Epsilon),
			ComputeDecayHalfLifeFromDiffAndVelocity(SrcDstDiff.Y, SrcVelocity.Y, HalfLife, HalfLifeMin, HalfLifeMax, Epsilon),
			ComputeDecayHalfLifeFromDiffAndVelocity(SrcDstDiff.Z, SrcVelocity.Z, HalfLife, HalfLifeMin, HalfLifeMax, Epsilon));
	}

	void LocationInertializeCubicUpdate(
		FVector& OutLocation,
		FVector3f& OutLinearVelocity,
		float& InOutTimeSinceTransition,
		const FVector InLocation,
		const FVector3f InLinearVelocity,
		const FVector3f InLocationOffset,
		const FVector3f InLinearVelocityOffset,
		const float DeltaTime,
		const float BlendTime)
	{
		InOutTimeSinceTransition += DeltaTime;

		const float T = FMath::Clamp(InOutTimeSinceTransition / FMath::Max(BlendTime, UE_SMALL_NUMBER), 0.0f, 1.0f);
		const float W0 = 2.0f * T * T * T - 3.0f * T * T + 1.0f;
		const float W1 = (T * T * T - 2.0f * T * T + T) * BlendTime;
		const float W2 = (6.0f * T * T - 6.0f * T) / FMath::Max(BlendTime, UE_SMALL_NUMBER);
		const float W3 = 3.0f * T * T - 4.0f * T + 1.0f;

		OutLocation = InLocation + (FVector)(W0 * InLocationOffset + W1 * InLinearVelocityOffset);
		OutLinearVelocity = InLinearVelocity + W2 * InLocationOffset + W3 * InLinearVelocityOffset;
	}

	void LocationInertializeCubicTransition(
		FVector3f& InOutLocationOffset,
		FVector3f& InOutLinearVelocityOffset,
		float& InOutTimeSinceTransition,
		const FVector InSrcLocation,
		const FVector3f InSrcLinearVelocity,
		const FVector InDstLocation,
		const FVector3f InDstLinearVelocity,
		const float BlendTime)
	{
		const float T = FMath::Clamp(InOutTimeSinceTransition / FMath::Max(BlendTime, UE_SMALL_NUMBER), 0.0f, 1.0f);
		const float W0 = 2.0f * T * T * T - 3.0f * T * T + 1.0f;
		const float W1 = (T * T * T - 2.0f * T * T + T) * BlendTime;
		const float W2 = (6.0f * T * T - 6.0f * T) / FMath::Max(BlendTime, UE_SMALL_NUMBER);
		const float W3 = 3.0f * T * T - 4.0f * T + 1.0f;

		const FVector OffsetSrcLocation = InSrcLocation + (FVector)(W0 * InOutLocationOffset + W1 * InOutLinearVelocityOffset);
		const FVector3f OffsetSrcLinearVelocity = InSrcLinearVelocity + W2 * InOutLocationOffset + W3 * InOutLinearVelocityOffset;

		InOutLocationOffset = (FVector3f)(OffsetSrcLocation - InDstLocation);
		InOutLinearVelocityOffset = OffsetSrcLinearVelocity - InDstLinearVelocity;
		InOutTimeSinceTransition = 0.0f;
	}

	void HermiteValueInterpolate(float& OutValue, float& OutValueVelocity, const float P0, const float P1, const float V0, const float V1, const float X, const float FrameTime)
	{
		const float W1 = 3 * X * X - 2 * X * X * X;
		const float W2 = X * X * X - 2 * X * X + X;
		const float W3 = X * X * X - X * X;

		const float Q1 = 6 * X - 6 * X * X;
		const float Q2 = 3 * X * X - 4 * X + 1;
		const float Q3 = 3 * X * X - 2 * X;

		OutValue = W1 * (P1 - P0) + W2 * V0 + W3 * V1 + P0;
		OutValueVelocity = (Q1 * (P1 - P0) + Q2 * V0 + Q3 * V1) / FMath::Max(FrameTime, UE_SMALL_NUMBER);
	}

	void HermiteAngleInterpolate(float& OutAngle, float& OutAngularVelocity, const float A0, const float A1, const float V0, const float V1, const float X, const float FrameTime)
	{
		const float W1 = 3 * X * X - 2 * X * X * X;
		const float W2 = X * X * X - 2 * X * X + X;
		const float W3 = X * X * X - X * X;

		const float Q1 = 6 * X - 6 * X * X;
		const float Q2 = 3 * X * X - 4 * X + 1;
		const float Q3 = 3 * X * X - 2 * X;

		const float A1SubA0 = FMath::FindDeltaAngleRadians(A0, A1);

		OutAngle = FMath::Wrap(W1 * A1SubA0 + W2 * V0 + W3 * V1 + A0, -UE_PI, UE_PI);
		OutAngularVelocity = (Q1 * A1SubA0 + Q2 * V0 + Q3 * V1) / FMath::Max(FrameTime, UE_SMALL_NUMBER);
	}

	void HermiteLocationInterpolate(FVector& OutPosition, FVector3f& OutLinearVelocity, const FVector P0, const FVector P1, const FVector3f V0, const FVector3f V1, const float X, const float FrameTime)
	{
		const float W1 = 3 * X * X - 2 * X * X * X;
		const float W2 = X * X * X - 2 * X * X + X;
		const float W3 = X * X * X - X * X;

		const float Q1 = 6 * X - 6 * X * X;
		const float Q2 = 3 * X * X - 4 * X + 1;
		const float Q3 = 3 * X * X - 2 * X;

		OutPosition = W1 * (P1 - P0) + W2 * (FVector)V0 + W3 * (FVector)V1 + P0;
		OutLinearVelocity = (Q1 * (FVector3f)(P1 - P0) + Q2 * V0 + Q3 * V1) / FMath::Max(FrameTime, UE_SMALL_NUMBER);
	}

	void HermiteLocationInterpolate(FVector3f& OutPosition, FVector3f& OutLinearVelocity, const FVector3f P0, const FVector3f P1, const FVector3f V0, const FVector3f V1, const float X, const float FrameTime)
	{
		const float W1 = 3 * X * X - 2 * X * X * X;
		const float W2 = X * X * X - 2 * X * X + X;
		const float W3 = X * X * X - X * X;

		const float Q1 = 6 * X - 6 * X * X;
		const float Q2 = 3 * X * X - 4 * X + 1;
		const float Q3 = 3 * X * X - 2 * X;

		OutPosition = W1 * (P1 - P0) + W2 * V0 + W3 * V1 + P0;
		OutLinearVelocity = (Q1 * (P1 - P0) + Q2 * V0 + Q3 * V1) / FMath::Max(FrameTime, UE_SMALL_NUMBER);
	}

	void HermiteRotationInterpolate(FQuat4f& OutRotation, FVector3f& OutAngularVelocity, const FQuat4f R0, const FQuat4f R1, const FVector3f V0, const FVector3f V1, const float X, const float FrameTime)
	{
		const float W1 = 3 * X * X - 2 * X * X * X;
		const float W2 = X * X * X - 2 * X * X + X;
		const float W3 = X * X * X - X * X;

		const float Q1 = 6 * X - 6 * X * X;
		const float Q2 = 3 * X * X - 4 * X + 1;
		const float Q3 = 3 * X * X - 2 * X;

		const FVector3f R1SubR0 = (R1 * R0.Inverse()).GetShortestArcWith(FQuat4f::Identity).ToRotationVector();

		OutRotation = FQuat4f::MakeFromRotationVector(W1 * R1SubR0 + W2 * V0 + W3 * V1) * R0;
		check(OutRotation.IsNormalized());
		OutAngularVelocity = (Q1 * R1SubR0 + Q2 * V0 + Q3 * V1) / FMath::Max(FrameTime, UE_SMALL_NUMBER);
	}

	void HermiteScaleInterpolate(FVector3f& OutScale, FVector3f& OutScalarVelocity, const FVector3f S0, const FVector3f S1, const FVector3f V0, const FVector3f V1, const float X, const float FrameTime)
	{
		const float W1 = 3 * X * X - 2 * X * X * X;
		const float W2 = X * X * X - 2 * X * X + X;
		const float W3 = X * X * X - X * X;

		const float Q1 = 6 * X - 6 * X * X;
		const float Q2 = 3 * X * X - 4 * X + 1;
		const float Q3 = 3 * X * X - 2 * X;

		const FVector3f R1SubR0 = VectorLogSafe(VectorDivMax(S1, S0, UE_SMALL_NUMBER));

		OutScale = VectorExp(VectorMin(W1 * R1SubR0 + W2 * V0 + W3 * V1, 10.0f)) * S0;
		OutScalarVelocity = (Q1 * R1SubR0 + Q2 * V0 + Q3 * V1) / FMath::Max(FrameTime, UE_SMALL_NUMBER);
	}

	void HermiteArrayInterpolate(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<1, float> OutValueVelocities,
		const TLearningArrayView<1, const float> P0,
		const TLearningArrayView<1, const float> P1,
		const TLearningArrayView<1, const float> V0,
		const TLearningArrayView<1, const float> V1,
		const float X,
		const float FrameTime)
	{
		const float W1 = 3 * X * X - 2 * X * X * X;
		const float W2 = X * X * X - 2 * X * X + X;
		const float W3 = X * X * X - X * X;

		const float Q1 = 6 * X - 6 * X * X;
		const float Q2 = 3 * X * X - 4 * X + 1;
		const float Q3 = 3 * X * X - 2 * X;

		const int64 ValueNum = OutValues.Num();

		for (int64 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
		{
			OutValues[ValueIdx] = W1 * (P1[ValueIdx] - P0[ValueIdx]) + W2 * V0[ValueIdx] + W3 * V1[ValueIdx] + P0[ValueIdx];
			OutValueVelocities[ValueIdx] = (Q1 * (P1[ValueIdx] - P0[ValueIdx]) + Q2 * V0[ValueIdx] + Q3 * V1[ValueIdx]) / FMath::Max(FrameTime, UE_SMALL_NUMBER);
		}
	}

	void HermiteValueInterpolate(float& OutValue, const float P0, const float P1, const float V0, const float V1, const float X)
	{
		const float W1 = 3 * X * X - 2 * X * X * X;
		const float W2 = X * X * X - 2 * X * X + X;
		const float W3 = X * X * X - X * X;

		OutValue = W1 * (P1 - P0) + W2 * V0 + W3 * V1 + P0;
	}

	void HermiteAngleInterpolate(float& OutAngle, const float A0, const float A1, const float V0, const float V1, const float X)
	{
		const float W1 = 3 * X * X - 2 * X * X * X;
		const float W2 = X * X * X - 2 * X * X + X;
		const float W3 = X * X * X - X * X;

		const float A1SubA0 = FMath::FindDeltaAngleRadians(A0, A1);

		OutAngle = FMath::Wrap(W1 * A1SubA0 + W2 * V0 + W3 * V1 + A0, -UE_PI, UE_PI);
	}

	void HermiteLocationInterpolate(FVector& OutPosition, const FVector P0, const FVector P1, const FVector3f V0, const FVector3f V1, const float X)
	{
		const float W1 = 3 * X * X - 2 * X * X * X;
		const float W2 = X * X * X - 2 * X * X + X;
		const float W3 = X * X * X - X * X;

		OutPosition = W1 * (P1 - P0) + W2 * (FVector)V0 + W3 * (FVector)V1 + P0;
	}

	void HermiteLocationInterpolate(FVector3f& OutPosition, const FVector3f P0, const FVector3f P1, const FVector3f V0, const FVector3f V1, const float X)
	{
		const float W1 = 3 * X * X - 2 * X * X * X;
		const float W2 = X * X * X - 2 * X * X + X;
		const float W3 = X * X * X - X * X;

		OutPosition = W1 * (P1 - P0) + W2 * V0 + W3 * V1 + P0;
	}

	void HermiteRotationInterpolate(FQuat4f& OutRotation, const FQuat4f R0, const FQuat4f R1, const FVector3f V0, const FVector3f V1, const float X)
	{
		const float W1 = 3 * X * X - 2 * X * X * X;
		const float W2 = X * X * X - 2 * X * X + X;
		const float W3 = X * X * X - X * X;

		const FVector3f R1SubR0 = (R1 * R0.Inverse()).GetShortestArcWith(FQuat4f::Identity).ToRotationVector();

		OutRotation = FQuat4f::MakeFromRotationVector(W1 * R1SubR0 + W2 * V0 + W3 * V1) * R0;
		check(OutRotation.IsNormalized());
	}

	void HermiteScaleInterpolate(FVector3f& OutScale, const FVector3f S0, const FVector3f S1, const FVector3f V0, const FVector3f V1, const float X)
	{
		const float W1 = 3 * X * X - 2 * X * X * X;
		const float W2 = X * X * X - 2 * X * X + X;
		const float W3 = X * X * X - X * X;

		const FVector3f R1SubR0 = VectorLogSafe(VectorDivMax(S1, S0, UE_SMALL_NUMBER));

		OutScale = VectorExp(VectorMin(W1 * R1SubR0 + W2 * V0 + W3 * V1, 10.0f)) * S0;
	}

	void HermiteArrayInterpolate(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<1, const float> P0,
		const TLearningArrayView<1, const float> P1,
		const TLearningArrayView<1, const float> V0,
		const TLearningArrayView<1, const float> V1,
		const float X,
		const float FrameTime)
	{
		const float W1 = 3 * X * X - 2 * X * X * X;
		const float W2 = X * X * X - 2 * X * X + X;
		const float W3 = X * X * X - X * X;

		const int64 ValueNum = OutValues.Num();

		for (int64 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
		{
			OutValues[ValueIdx] = W1 * (P1[ValueIdx] - P0[ValueIdx]) + W2 * V0[ValueIdx] + W3 * V1[ValueIdx] + P0[ValueIdx];
		}
	}

	// Linear Interpolation

	void ValueInterpolateLinear(float& OutValue, float& OutValueVelocity, const float P0, const float P1, const float Alpha, const float FrameTime)
	{
		OutValue = FMath::Lerp(P0, P1, Alpha);
		OutValueVelocity = (P1 - P0) / FMath::Max(FrameTime, UE_SMALL_NUMBER);
	}

	void AngleInterpolateLinear(float& OutAngle, float& OutAngularVelocity, const float A0, const float A1, const float Alpha, const float FrameTime)
	{
		OutAngle = FMath::Wrap(A0 + FMath::FindDeltaAngleRadians(A0, A1) * Alpha, -UE_PI, UE_PI);
		OutAngularVelocity = FMath::FindDeltaAngleRadians(A0, A1) / FMath::Max(FrameTime, UE_SMALL_NUMBER);
	}

	void LocationInterpolateLinear(FVector& OutLocation, FVector3f& OutLinearVelocity, const FVector P0, const FVector P1, const float Alpha, const float FrameTime)
	{
		OutLocation = FMath::Lerp(P0, P1, Alpha);
		OutLinearVelocity = (FVector3f)((P1 - P0) / FMath::Max(FrameTime, UE_SMALL_NUMBER));
	}

	void LocationInterpolateLinear(FVector3f& OutLocation, FVector3f& OutLinearVelocity, const FVector3f P0, const FVector3f P1, const float Alpha, const float FrameTime)
	{
		OutLocation = FMath::Lerp(P0, P1, Alpha);
		OutLinearVelocity = (P1 - P0) / FMath::Max(FrameTime, UE_SMALL_NUMBER);
	}

	void RotationInterpolateLinear(FQuat4f& OutRotation, FVector3f& OutAngularVelocity, const FQuat4f R0, const FQuat4f R1, const float Alpha, const float FrameTime)
	{
		OutRotation = FQuat4f::Slerp(R0, R1, Alpha);
		check(OutRotation.IsNormalized());
		OutAngularVelocity = (R1 * R0.Inverse()).GetShortestArcWith(FQuat4f::Identity).ToRotationVector() / FMath::Max(FrameTime, UE_SMALL_NUMBER);
	}

	void ScaleInterpolateLinear(FVector3f& OutScale, FVector3f& OutScalarVelocity, const FVector3f S0, const FVector3f S1, const float Alpha, const float FrameTime)
	{
		const FVector3f S0Log = VectorLogSafe(S0);
		const FVector3f S1Log = VectorLogSafe(S1);

		OutScale = VectorExp(FMath::Lerp(S0Log, S1Log, Alpha));
		OutScalarVelocity = (S1Log - S0Log) / FMath::Max(FrameTime, UE_SMALL_NUMBER);
	}

	void ArrayInterpolateLinear(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<1, float> OutValueVelocities,
		const TLearningArrayView<1, const float> P0,
		const TLearningArrayView<1, const float> P1,
		const float Alpha,
		const float FrameTime)
	{
		const int64 ValueNum = OutValues.Num();

		for (int64 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
		{
			OutValues[ValueIdx] = FMath::Lerp(P0[ValueIdx], P1[ValueIdx], Alpha);
			OutValueVelocities[ValueIdx] = (P1[ValueIdx] - P0[ValueIdx]) / FMath::Max(FrameTime, UE_SMALL_NUMBER);
		}
	}

	void ValueInterpolateLinear(float& OutValue, const float P0, const float P1, const float Alpha)
	{
		OutValue = FMath::Lerp(P0, P1, Alpha);
	}

	void AngleInterpolateLinear(float& OutAngle, const float A0, const float A1, const float Alpha)
	{
		OutAngle = FMath::Wrap(A0 + FMath::FindDeltaAngleRadians(A0, A1) * Alpha, -UE_PI, UE_PI);
	}

	void LocationInterpolateLinear(FVector& OutLocation, const FVector P0, const FVector P1, const float Alpha)
	{
		OutLocation = FMath::Lerp(P0, P1, Alpha);
	}

	void LocationInterpolateLinear(FVector3f& OutLocation, const FVector3f P0, const FVector3f P1, const float Alpha)
	{
		OutLocation = FMath::Lerp(P0, P1, Alpha);
	}

	void RotationInterpolateLinear(FQuat4f& OutRotation, const FQuat4f R0, const FQuat4f R1, const float Alpha)
	{
		OutRotation = FQuat4f::Slerp(R0, R1, Alpha);
		check(OutRotation.IsNormalized());
	}

	void ScaleInterpolateLinear(FVector3f& OutScale, const FVector3f S0, const FVector3f S1, const float Alpha)
	{
		const FVector3f S0Log = VectorLogSafe(S0);
		const FVector3f S1Log = VectorLogSafe(S1);
		OutScale = VectorExp(FMath::Lerp(S0Log, S1Log, Alpha));
	}

	void DirectionInterpolateLinear(FVector3f& OutDirection, const FVector3f D0, const FVector3f D1, const float Alpha)
	{
		OutDirection = FMath::Lerp(D0, D1, Alpha).GetSafeNormal(UE_SMALL_NUMBER, FVector3f::ForwardVector);
	}

	void TransformInterpolateLinear(FTransform3f& OutTransform, const FTransform3f T0, const FTransform3f T1, const float Alpha)
	{
		FVector3f OutLocation;
		FQuat4f OutRotation;
		FVector3f OutScale;
		LocationInterpolateLinear(OutLocation, T0.GetLocation(), T1.GetLocation(), Alpha);
		RotationInterpolateLinear(OutRotation, T0.GetRotation(), T1.GetRotation(), Alpha);
		ScaleInterpolateLinear(OutScale, T0.GetScale3D(), T1.GetScale3D(), Alpha);
		OutTransform = FTransform3f(OutRotation, OutLocation, OutScale);
	}

	void ArrayInterpolateLinear(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<1, const float> P0,
		const TLearningArrayView<1, const float> P1,
		const float Alpha)
	{
		const int64 ValueNum = OutValues.Num();

		if (UE_ANIMDATABASE_ISPC && OutValues.NumBytes() < MAX_int32)
		{
#if UE_ANIMDATABASE_ISPC
			ispc::AnimDatabaseArrayInterpolateLinear(
				OutValues.GetData(),
				P0.GetData(),
				P1.GetData(),
				Alpha,
				ValueNum);
#endif
		}
		else
		{
			for (int64 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
			{
				OutValues[ValueIdx] = FMath::Lerp(P0[ValueIdx], P1[ValueIdx], Alpha);
			}
		}
	}

	// Cubic Interpolation

	void ValueInterpolateCubic(float& OutValue, float& OutValueVelocity, const float P0, const float P1, const float P2, const float P3, const float Alpha, const float FrameTime)
	{
		const float V1 = ((P1 - P0) + (P2 - P1)) / 2;
		const float V2 = ((P2 - P1) + (P3 - P2)) / 2;
		HermiteValueInterpolate(OutValue, OutValueVelocity, P1, P2, V1, V2, Alpha, FrameTime);
	}

	void AngleInterpolateCubic(float& OutAngle, float& OutAngularVelocity, const float A0, const float A1, const float A2, const float A3, const float Alpha, const float FrameTime)
	{
		const float A1A0Diff = FMath::FindDeltaAngleRadians(A0, A1);
		const float A2A1Diff = FMath::FindDeltaAngleRadians(A1, A2);
		const float A3A2Diff = FMath::FindDeltaAngleRadians(A2, A3);
		const float V1 = (A1A0Diff + A2A1Diff) / 2;
		const float V2 = (A2A1Diff + A3A2Diff) / 2;
		HermiteAngleInterpolate(OutAngle, OutAngularVelocity, A1, A2, V1, V2, Alpha, FrameTime);
	}

	void LocationInterpolateCubic(FVector& OutLocation, FVector3f& OutLinearVelocity, const FVector P0, const FVector P1, const FVector P2, const FVector P3, const float Alpha, const float FrameTime)
	{
		const FVector3f V1 = (FVector3f)((P1 - P0) + (P2 - P1)) / 2;
		const FVector3f V2 = (FVector3f)((P2 - P1) + (P3 - P2)) / 2;
		HermiteLocationInterpolate(OutLocation, OutLinearVelocity, P1, P2, V1, V2, Alpha, FrameTime);
	}

	void LocationInterpolateCubic(FVector3f& OutLocation, FVector3f& OutLinearVelocity, const FVector3f P0, const FVector3f P1, const FVector3f P2, const FVector3f P3, const float Alpha, const float FrameTime)
	{
		const FVector3f V1 = ((P1 - P0) + (P2 - P1)) / 2;
		const FVector3f V2 = ((P2 - P1) + (P3 - P2)) / 2;
		HermiteLocationInterpolate(OutLocation, OutLinearVelocity, P1, P2, V1, V2, Alpha, FrameTime);
	}

	void RotationInterpolateCubic(FQuat4f& OutRotation, FVector3f& OutAngularVelocity, const FQuat4f R0, const FQuat4f R1, const FQuat4f R2, const FQuat4f R3, const float Alpha, const float FrameTime)
	{
		const FVector3f R1R0Diff = (R1 * R0.Inverse()).GetShortestArcWith(FQuat4f::Identity).ToRotationVector();
		const FVector3f R2R1Diff = (R2 * R1.Inverse()).GetShortestArcWith(FQuat4f::Identity).ToRotationVector();
		const FVector3f R3R2Diff = (R3 * R2.Inverse()).GetShortestArcWith(FQuat4f::Identity).ToRotationVector();
		const FVector3f V1 = (R1R0Diff + R2R1Diff) / 2;
		const FVector3f V2 = (R2R1Diff + R3R2Diff) / 2;
		HermiteRotationInterpolate(OutRotation, OutAngularVelocity, R1, R2, V1, V2, Alpha, FrameTime);
	}

	void ScaleInterpolateCubic(FVector3f& OutScale, FVector3f& OutScalarVelocity, const FVector3f S0, const FVector3f S1, const FVector3f S2, const FVector3f S3, const float Alpha, const float FrameTime)
	{
		const FVector3f S1SubS0 = VectorLogSafe(VectorDivMax(S1, S0, UE_SMALL_NUMBER));
		const FVector3f S2SubS1 = VectorLogSafe(VectorDivMax(S2, S1, UE_SMALL_NUMBER));
		const FVector3f S3SubS2 = VectorLogSafe(VectorDivMax(S3, S2, UE_SMALL_NUMBER));
		const FVector3f V1 = (S1SubS0 + S2SubS1) / 2;
		const FVector3f V2 = (S2SubS1 + S3SubS2) / 2;
		HermiteScaleInterpolate(OutScale, OutScalarVelocity, S1, S2, V1, V2, Alpha, FrameTime);
	}

	void ArrayInterpolateCubic(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<1, float> OutValueVelocities,
		const TLearningArrayView<1, const float> P0,
		const TLearningArrayView<1, const float> P1,
		const TLearningArrayView<1, const float> P2,
		const TLearningArrayView<1, const float> P3,
		const float Alpha,
		const float FrameTime)
	{
		const int64 ValueNum = OutValues.Num();

		for (int64 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
		{
			const float V1 = ((P1[ValueIdx] - P0[ValueIdx]) + (P2[ValueIdx] - P1[ValueIdx])) / 2;
			const float V2 = ((P2[ValueIdx] - P1[ValueIdx]) + (P3[ValueIdx] - P2[ValueIdx])) / 2;
			HermiteValueInterpolate(OutValues[ValueIdx], OutValueVelocities[ValueIdx], P1[ValueIdx], P2[ValueIdx], V1, V2, Alpha, FrameTime);
		}
	}

	void ValueInterpolateCubic(float& OutValue, const float P0, const float P1, const float P2, const float P3, const float Alpha)
	{
		const float V1 = ((P1 - P0) + (P2 - P1)) / 2;
		const float V2 = ((P2 - P1) + (P3 - P2)) / 2;
		HermiteValueInterpolate(OutValue, P1, P2, V1, V2, Alpha);
	}

	void AngleInterpolateCubic(float& OutAngle, const float A0, const float A1, const float A2, const float A3, const float Alpha)
	{
		const float A1A0Diff = FMath::FindDeltaAngleRadians(A0, A1);
		const float A2A1Diff = FMath::FindDeltaAngleRadians(A1, A2);
		const float A3A2Diff = FMath::FindDeltaAngleRadians(A2, A3);
		const float V1 = (A1A0Diff + A2A1Diff) / 2;
		const float V2 = (A2A1Diff + A3A2Diff) / 2;
		HermiteAngleInterpolate(OutAngle, A1, A2, V1, V2, Alpha);
	}

	void LocationInterpolateCubic(FVector& OutLocation, const FVector P0, const FVector P1, const FVector P2, const FVector P3, const float Alpha)
	{
		const FVector3f V1 = (FVector3f)((P1 - P0) + (P2 - P1)) / 2;
		const FVector3f V2 = (FVector3f)((P2 - P1) + (P3 - P2)) / 2;
		HermiteLocationInterpolate(OutLocation, P1, P2, V1, V2, Alpha);
	}

	void LocationInterpolateCubic(FVector3f& OutLocation, const FVector3f P0, const FVector3f P1, const FVector3f P2, const FVector3f P3, const float Alpha)
	{
		const FVector3f V1 = ((P1 - P0) + (P2 - P1)) / 2;
		const FVector3f V2 = ((P2 - P1) + (P3 - P2)) / 2;
		HermiteLocationInterpolate(OutLocation, P1, P2, V1, V2, Alpha);
	}

	void RotationInterpolateCubic(FQuat4f& OutRotation, const FQuat4f R0, const FQuat4f R1, const FQuat4f R2, const FQuat4f R3, const float Alpha)
	{
		const FVector3f R1R0Diff = (R1 * R0.Inverse()).GetShortestArcWith(FQuat4f::Identity).ToRotationVector();
		const FVector3f R2R1Diff = (R2 * R1.Inverse()).GetShortestArcWith(FQuat4f::Identity).ToRotationVector();
		const FVector3f R3R2Diff = (R3 * R2.Inverse()).GetShortestArcWith(FQuat4f::Identity).ToRotationVector();
		const FVector3f V1 = (R1R0Diff + R2R1Diff) / 2;
		const FVector3f V2 = (R2R1Diff + R3R2Diff) / 2;
		HermiteRotationInterpolate(OutRotation, R1, R2, V1, V2, Alpha);
	}

	void ScaleInterpolateCubic(FVector3f& OutScale, const FVector3f S0, const FVector3f S1, const FVector3f S2, const FVector3f S3, const float Alpha)
	{
		const FVector3f S1SubS0 = VectorLogSafe(VectorDivMax(S1, S0, UE_SMALL_NUMBER));
		const FVector3f S2SubS1 = VectorLogSafe(VectorDivMax(S2, S1, UE_SMALL_NUMBER));
		const FVector3f S3SubS2 = VectorLogSafe(VectorDivMax(S3, S2, UE_SMALL_NUMBER));
		const FVector3f V1 = (S1SubS0 + S2SubS1) / 2;
		const FVector3f V2 = (S2SubS1 + S3SubS2) / 2;
		HermiteScaleInterpolate(OutScale, S1, S2, V1, V2, Alpha);
	}

	void DirectionInterpolateCubic(FVector3f& OutDirection, const FVector3f D0, const FVector3f D1, const FVector3f D2, const FVector3f D3, const float Alpha)
	{
		LocationInterpolateCubic(OutDirection, D0, D1, D2, D3, Alpha);
		OutDirection = OutDirection.GetSafeNormal(UE_SMALL_NUMBER, FVector3f::ForwardVector);
	}

	void TransformInterpolateCubic(FTransform3f& OutTransform, const FTransform3f T0, const FTransform3f T1, const FTransform3f T2, const FTransform3f T3, const float Alpha)
	{
		FVector3f OutLocation;
		FQuat4f OutRotation;
		FVector3f OutScale;
		LocationInterpolateCubic(OutLocation, T0.GetLocation(), T1.GetLocation(), T2.GetLocation(), T3.GetLocation(), Alpha);
		RotationInterpolateCubic(OutRotation, T0.GetRotation(), T1.GetRotation(), T2.GetRotation(), T3.GetRotation(), Alpha);
		ScaleInterpolateCubic(OutScale, T0.GetScale3D(), T1.GetScale3D(), T2.GetScale3D(), T3.GetScale3D(), Alpha);
		OutTransform = FTransform3f(OutRotation, OutLocation, OutScale);
	}

	void ArrayInterpolateCubic(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<1, const float> P0,
		const TLearningArrayView<1, const float> P1,
		const TLearningArrayView<1, const float> P2,
		const TLearningArrayView<1, const float> P3,
		const float Alpha)
	{
		const int64 ValueNum = OutValues.Num();

		for (int64 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
		{
			const float V1 = ((P1[ValueIdx] - P0[ValueIdx]) + (P2[ValueIdx] - P1[ValueIdx])) / 2;
			const float V2 = ((P2[ValueIdx] - P1[ValueIdx]) + (P3[ValueIdx] - P2[ValueIdx])) / 2;
			HermiteValueInterpolate(OutValues[ValueIdx], P1[ValueIdx], P2[ValueIdx], V1, V2, Alpha);
		}
	}

	void ValueInterpolateCubicStart(float& OutValue, float& OutValueVelocity, const float P1, const float P2, const float P3, const float Alpha, const float FrameTime)
	{
		const float V1 = (P2 - P1);
		const float V2 = ((P2 - P1) + (P3 - P2)) / 2;
		HermiteValueInterpolate(OutValue, OutValueVelocity, P1, P2, V1, V2, Alpha, FrameTime);
	}

	void AngleInterpolateCubicStart(float& OutAngle, float& OutAngularVelocity, const float A1, const float A2, const float A3, const float Alpha, const float FrameTime)
	{
		const float A2A1Diff = FMath::FindDeltaAngleRadians(A1, A2);
		const float A3A2Diff = FMath::FindDeltaAngleRadians(A2, A3);
		const float V1 = A2A1Diff;
		const float V2 = (A2A1Diff + A3A2Diff) / 2;
		HermiteAngleInterpolate(OutAngle, OutAngularVelocity, A1, A2, V1, V2, Alpha, FrameTime);
	}

	void LocationInterpolateCubicStart(FVector& OutLocation, FVector3f& OutLinearVelocity, const FVector P1, const FVector P2, const FVector P3, const float Alpha, const float FrameTime)
	{
		const FVector3f V1 = (FVector3f)(P2 - P1);
		const FVector3f V2 = (FVector3f)((P2 - P1) + (P3 - P2)) / 2;
		HermiteLocationInterpolate(OutLocation, OutLinearVelocity, P1, P2, V1, V2, Alpha, FrameTime);
	}

	void LocationInterpolateCubicStart(FVector3f& OutLocation, FVector3f& OutLinearVelocity, const FVector3f P1, const FVector3f P2, const FVector3f P3, const float Alpha, const float FrameTime)
	{
		const FVector3f V1 = (P2 - P1);
		const FVector3f V2 = ((P2 - P1) + (P3 - P2)) / 2;
		HermiteLocationInterpolate(OutLocation, OutLinearVelocity, P1, P2, V1, V2, Alpha, FrameTime);
	}

	void RotationInterpolateCubicStart(FQuat4f& OutRotation, FVector3f& OutAngularVelocity, const FQuat4f R1, const FQuat4f R2, const FQuat4f R3, const float Alpha, const float FrameTime)
	{
		const FVector3f R2R1Diff = (R2 * R1.Inverse()).GetShortestArcWith(FQuat4f::Identity).ToRotationVector();
		const FVector3f R3R2Diff = (R3 * R2.Inverse()).GetShortestArcWith(FQuat4f::Identity).ToRotationVector();
		const FVector3f V1 = R2R1Diff;
		const FVector3f V2 = (R2R1Diff + R3R2Diff) / 2;
		HermiteRotationInterpolate(OutRotation, OutAngularVelocity, R1, R2, V1, V2, Alpha, FrameTime);
	}

	void ScaleInterpolateCubicStart(FVector3f& OutScale, FVector3f& OutScalarVelocity, const FVector3f S1, const FVector3f S2, const FVector3f S3, const float Alpha, const float FrameTime)
	{
		const FVector3f S2SubS1 = VectorLogSafe(VectorDivMax(S2, S1, UE_SMALL_NUMBER));
		const FVector3f S3SubS2 = VectorLogSafe(VectorDivMax(S3, S2, UE_SMALL_NUMBER));
		const FVector3f V1 = S2SubS1;
		const FVector3f V2 = (S2SubS1 + S3SubS2) / 2;
		HermiteScaleInterpolate(OutScale, OutScalarVelocity, S1, S2, V1, V2, Alpha, FrameTime);
	}

	void ArrayInterpolateCubicStart(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<1, float> OutValueVelocities,
		const TLearningArrayView<1, const float> P1,
		const TLearningArrayView<1, const float> P2,
		const TLearningArrayView<1, const float> P3,
		const float Alpha,
		const float FrameTime)
	{
		const int64 ValueNum = OutValues.Num();

		for (int64 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
		{
			const float V1 = (P2[ValueIdx] - P1[ValueIdx]);
			const float V2 = ((P2[ValueIdx] - P1[ValueIdx]) + (P3[ValueIdx] - P2[ValueIdx])) / 2;
			HermiteValueInterpolate(OutValues[ValueIdx], OutValueVelocities[ValueIdx], P1[ValueIdx], P2[ValueIdx], V1, V2, Alpha, FrameTime);
		}
	}

	void ValueInterpolateCubicStart(float& OutValue, const float P1, const float P2, const float P3, const float Alpha)
	{
		const float V1 = (P2 - P1);
		const float V2 = ((P2 - P1) + (P3 - P2)) / 2;
		HermiteValueInterpolate(OutValue, P1, P2, V1, V2, Alpha);
	}

	void AngleInterpolateCubicStart(float& OutAngle, const float A1, const float A2, const float A3, const float Alpha)
	{
		const float A2A1Diff = FMath::FindDeltaAngleRadians(A1, A2);
		const float A3A2Diff = FMath::FindDeltaAngleRadians(A2, A3);
		const float V1 = A2A1Diff;
		const float V2 = (A2A1Diff + A3A2Diff) / 2;
		HermiteAngleInterpolate(OutAngle, A1, A2, V1, V2, Alpha);
	}

	void LocationInterpolateCubicStart(FVector& OutLocation, const FVector P1, const FVector P2, const FVector P3, const float Alpha)
	{
		const FVector3f V1 = (FVector3f)(P2 - P1);
		const FVector3f V2 = (FVector3f)((P2 - P1) + (P3 - P2)) / 2;
		HermiteLocationInterpolate(OutLocation, P1, P2, V1, V2, Alpha);
	}

	void LocationInterpolateCubicStart(FVector3f& OutLocation, const FVector3f P1, const FVector3f P2, const FVector3f P3, const float Alpha)
	{
		const FVector3f V1 = (P2 - P1);
		const FVector3f V2 = ((P2 - P1) + (P3 - P2)) / 2;
		HermiteLocationInterpolate(OutLocation, P1, P2, V1, V2, Alpha);
	}

	void RotationInterpolateCubicStart(FQuat4f& OutRotation, const FQuat4f R1, const FQuat4f R2, const FQuat4f R3, const float Alpha)
	{
		const FVector3f R2R1Diff = (R2 * R1.Inverse()).GetShortestArcWith(FQuat4f::Identity).ToRotationVector();
		const FVector3f R3R2Diff = (R3 * R2.Inverse()).GetShortestArcWith(FQuat4f::Identity).ToRotationVector();
		const FVector3f V1 = R2R1Diff;
		const FVector3f V2 = (R2R1Diff + R3R2Diff) / 2;
		HermiteRotationInterpolate(OutRotation, R1, R2, V1, V2, Alpha);
	}

	void ScaleInterpolateCubicStart(FVector3f& OutScale, const FVector3f S1, const FVector3f S2, const FVector3f S3, const float Alpha)
	{
		const FVector3f S2SubS1 = VectorLogSafe(VectorDivMax(S2, S1, UE_SMALL_NUMBER));
		const FVector3f S3SubS2 = VectorLogSafe(VectorDivMax(S3, S2, UE_SMALL_NUMBER));
		const FVector3f V1 = S2SubS1;
		const FVector3f V2 = (S2SubS1 + S3SubS2) / 2;
		HermiteScaleInterpolate(OutScale, S1, S2, V1, V2, Alpha);
	}

	void DirectionInterpolateCubicStart(FVector3f& OutDirection, const FVector3f D1, const FVector3f D2, const FVector3f D3, const float Alpha)
	{
		LocationInterpolateCubicStart(OutDirection, D1, D2, D3, Alpha);
		OutDirection = OutDirection.GetSafeNormal(UE_SMALL_NUMBER, FVector3f::ForwardVector);
	}

	void TransformInterpolateCubicStart(FTransform3f& OutTransform, const FTransform3f T1, const FTransform3f T2, const FTransform3f T3, const float Alpha)
	{
		FVector3f OutLocation;
		FQuat4f OutRotation;
		FVector3f OutScale;
		LocationInterpolateCubicStart(OutLocation, T1.GetLocation(), T2.GetLocation(), T3.GetLocation(), Alpha);
		RotationInterpolateCubicStart(OutRotation, T1.GetRotation(), T2.GetRotation(), T3.GetRotation(), Alpha);
		ScaleInterpolateCubicStart(OutScale, T1.GetScale3D(), T2.GetScale3D(), T3.GetScale3D(), Alpha);
		OutTransform = FTransform3f(OutRotation, OutLocation, OutScale);
	}

	void ArrayInterpolateCubicStart(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<1, const float> P1,
		const TLearningArrayView<1, const float> P2,
		const TLearningArrayView<1, const float> P3,
		const float Alpha)
	{
		const int64 ValueNum = OutValues.Num();

		for (int64 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
		{
			const float V1 = (P2[ValueIdx] - P1[ValueIdx]);
			const float V2 = ((P2[ValueIdx] - P1[ValueIdx]) + (P3[ValueIdx] - P2[ValueIdx])) / 2;
			HermiteValueInterpolate(OutValues[ValueIdx], P1[ValueIdx], P2[ValueIdx], V1, V2, Alpha);
		}
	}

	void ValueInterpolateCubicEnd(float& OutValue, float& OutValueVelocity, const float P0, const float P1, const float P2, const float Alpha, const float FrameTime)
	{
		const float V1 = ((P1 - P0) + (P2 - P1)) / 2;
		const float V2 = (P2 - P1);
		HermiteValueInterpolate(OutValue, OutValueVelocity, P1, P2, V1, V2, Alpha, FrameTime);
	}

	void AngleInterpolateCubicEnd(float& OutAngle, float& OutAngularVelocity, const float A0, const float A1, const float A2, const float Alpha, const float FrameTime)
	{
		const float A1A0Diff = FMath::FindDeltaAngleRadians(A0, A1);
		const float A2A1Diff = FMath::FindDeltaAngleRadians(A1, A2);
		const float V1 = (A1A0Diff + A2A1Diff) / 2;
		const float V2 = A2A1Diff;
		HermiteAngleInterpolate(OutAngle, OutAngularVelocity, A1, A2, V1, V2, Alpha, FrameTime);
	}

	void LocationInterpolateCubicEnd(FVector& OutLocation, FVector3f& OutLinearVelocity, const FVector P0, const FVector P1, const FVector P2, const float Alpha, const float FrameTime)
	{
		const FVector3f V1 = (FVector3f)((P1 - P0) + (P2 - P1)) / 2;
		const FVector3f V2 = (FVector3f)(P2 - P1);
		HermiteLocationInterpolate(OutLocation, OutLinearVelocity, P1, P2, V1, V2, Alpha, FrameTime);
	}

	void LocationInterpolateCubicEnd(FVector3f& OutLocation, FVector3f& OutLinearVelocity, const FVector3f P0, const FVector3f P1, const FVector3f P2, const float Alpha, const float FrameTime)
	{
		const FVector3f V1 = ((P1 - P0) + (P2 - P1)) / 2;
		const FVector3f V2 = (P2 - P1);
		HermiteLocationInterpolate(OutLocation, OutLinearVelocity, P1, P2, V1, V2, Alpha, FrameTime);
	}

	void RotationInterpolateCubicEnd(FQuat4f& OutRotation, FVector3f& OutAngularVelocity, const FQuat4f R0, const FQuat4f R1, const FQuat4f R2, const float Alpha, const float FrameTime)
	{
		const FVector3f R1R0Diff = (R1 * R0.Inverse()).GetShortestArcWith(FQuat4f::Identity).ToRotationVector();
		const FVector3f R2R1Diff = (R2 * R1.Inverse()).GetShortestArcWith(FQuat4f::Identity).ToRotationVector();
		const FVector3f V1 = (R1R0Diff + R2R1Diff) / 2;
		const FVector3f V2 = R2R1Diff;
		HermiteRotationInterpolate(OutRotation, OutAngularVelocity, R1, R2, V1, V2, Alpha, FrameTime);
	}

	void ScaleInterpolateCubicEnd(FVector3f& OutScale, FVector3f& OutScalarVelocity, const FVector3f S0, const FVector3f S1, const FVector3f S2, const float Alpha, const float FrameTime)
	{
		const FVector3f S1SubS0 = VectorLogSafe(VectorDivMax(S1, S0, UE_SMALL_NUMBER));
		const FVector3f S2SubS1 = VectorLogSafe(VectorDivMax(S2, S1, UE_SMALL_NUMBER));
		const FVector3f V1 = (S1SubS0 + S2SubS1) / 2;
		const FVector3f V2 = S2SubS1;
		HermiteScaleInterpolate(OutScale, OutScalarVelocity, S1, S2, V1, V2, Alpha, FrameTime);
	}

	void ArrayInterpolateCubicEnd(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<1, float> OutValueVelocities,
		const TLearningArrayView<1, const float> P0,
		const TLearningArrayView<1, const float> P1,
		const TLearningArrayView<1, const float> P2,
		const float Alpha,
		const float FrameTime)
	{
		const int64 ValueNum = OutValues.Num();

		for (int64 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
		{
			const float V1 = ((P1[ValueIdx] - P0[ValueIdx]) + (P2[ValueIdx] - P1[ValueIdx])) / 2;
			const float V2 = (P2[ValueIdx] - P1[ValueIdx]);
			HermiteValueInterpolate(OutValues[ValueIdx], OutValueVelocities[ValueIdx], P1[ValueIdx], P2[ValueIdx], V1, V2, Alpha, FrameTime);
		}
	}

	void ValueInterpolateCubicEnd(float& OutValue, const float P0, const float P1, const float P2, const float Alpha)
	{
		const float V1 = ((P1 - P0) + (P2 - P1)) / 2;
		const float V2 = (P2 - P1);
		HermiteValueInterpolate(OutValue, P1, P2, V1, V2, Alpha);
	}

	void AngleInterpolateCubicEnd(float& OutAngle, const float A0, const float A1, const float A2, const float Alpha)
	{
		const float A1A0Diff = FMath::FindDeltaAngleRadians(A0, A1);
		const float A2A1Diff = FMath::FindDeltaAngleRadians(A1, A2);
		const float V1 = (A1A0Diff + A2A1Diff) / 2;
		const float V2 = A2A1Diff;
		HermiteAngleInterpolate(OutAngle, A1, A2, V1, V2, Alpha);
	}

	void LocationInterpolateCubicEnd(FVector& OutLocation, const FVector P0, const FVector P1, const FVector P2, const float Alpha)
	{
		const FVector3f V1 = (FVector3f)((P1 - P0) + (P2 - P1)) / 2;
		const FVector3f V2 = (FVector3f)(P2 - P1);
		HermiteLocationInterpolate(OutLocation, P1, P2, V1, V2, Alpha);
	}

	void LocationInterpolateCubicEnd(FVector3f& OutLocation, const FVector3f P0, const FVector3f P1, const FVector3f P2, const float Alpha)
	{
		const FVector3f V1 = ((P1 - P0) + (P2 - P1)) / 2;
		const FVector3f V2 = (P2 - P1);
		HermiteLocationInterpolate(OutLocation, P1, P2, V1, V2, Alpha);
	}

	void RotationInterpolateCubicEnd(FQuat4f& OutRotation, const FQuat4f R0, const FQuat4f R1, const FQuat4f R2, const float Alpha)
	{
		const FVector3f R1R0Diff = (R1 * R0.Inverse()).GetShortestArcWith(FQuat4f::Identity).ToRotationVector();
		const FVector3f R2R1Diff = (R2 * R1.Inverse()).GetShortestArcWith(FQuat4f::Identity).ToRotationVector();
		const FVector3f V1 = (R1R0Diff + R2R1Diff) / 2;
		const FVector3f V2 = R2R1Diff;
		HermiteRotationInterpolate(OutRotation, R1, R2, V1, V2, Alpha);
	}

	void ScaleInterpolateCubicEnd(FVector3f& OutScale, const FVector3f S0, const FVector3f S1, const FVector3f S2, const float Alpha)
	{
		const FVector3f S1SubS0 = VectorLogSafe(VectorDivMax(S1, S0, UE_SMALL_NUMBER));
		const FVector3f S2SubS1 = VectorLogSafe(VectorDivMax(S2, S1, UE_SMALL_NUMBER));
		const FVector3f V1 = (S1SubS0 + S2SubS1) / 2;
		const FVector3f V2 = S2SubS1;
		HermiteScaleInterpolate(OutScale, S1, S2, V1, V2, Alpha);
	}

	void DirectionInterpolateCubicEnd(FVector3f& OutDirection, const FVector3f D0, const FVector3f D1, const FVector3f D2, const float Alpha)
	{
		LocationInterpolateCubicEnd(OutDirection, D0, D1, D2, Alpha);
		OutDirection = OutDirection.GetSafeNormal(UE_SMALL_NUMBER, FVector3f::ForwardVector);
	}

	void TransformInterpolateCubicEnd(FTransform3f& OutTransform, const FTransform3f T0, const FTransform3f T1, const FTransform3f T2, const float Alpha)
	{
		FVector3f OutLocation;
		FQuat4f OutRotation;
		FVector3f OutScale;
		LocationInterpolateCubicEnd(OutLocation, T0.GetLocation(), T1.GetLocation(), T2.GetLocation(), Alpha);
		RotationInterpolateCubicEnd(OutRotation, T0.GetRotation(), T1.GetRotation(), T2.GetRotation(), Alpha);
		ScaleInterpolateCubicEnd(OutScale, T0.GetScale3D(), T1.GetScale3D(), T2.GetScale3D(), Alpha);
		OutTransform = FTransform3f(OutRotation, OutLocation, OutScale);
	}

	void ArrayInterpolateCubicEnd(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<1, const float> P0,
		const TLearningArrayView<1, const float> P1,
		const TLearningArrayView<1, const float> P2,
		const float Alpha)
	{
		const int64 ValueNum = OutValues.Num();

		for (int64 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
		{
			const float V1 = ((P1[ValueIdx] - P0[ValueIdx]) + (P2[ValueIdx] - P1[ValueIdx])) / 2;
			const float V2 = (P2[ValueIdx] - P1[ValueIdx]);
			HermiteValueInterpolate(OutValues[ValueIdx], P1[ValueIdx], P2[ValueIdx], V1, V2, Alpha);
		}
	}

	// Cubic Monotone Interpolation

	float ComputeMonotoneVelocity(const float D0, const float D1)
	{
		return FMath::Sign(D0) != FMath::Sign(D1) ? 0.0f :
			FMath::Clamp(
				(D0 + D1) / 2.0f,
				FMath::Max(-3.0f * FMath::Abs(D0), -3.0f * FMath::Abs(D1)),
				FMath::Min(+3.0f * FMath::Abs(D0), +3.0f * FMath::Abs(D1)));
	}

	FVector3f ComputeMonotoneVelocity(const FVector3f D0, const FVector3f D1)
	{
		return FVector3f(
			ComputeMonotoneVelocity(D0.X, D1.X),
			ComputeMonotoneVelocity(D0.Y, D1.Y),
			ComputeMonotoneVelocity(D0.Z, D1.Z));
	}

	void ValueInterpolateCubicMono(float& OutValue, float& OutValueVelocity, const float P0, const float P1, const float P2, const float P3, const float Alpha, const float FrameTime)
	{
		const float V1 = ComputeMonotoneVelocity(P1 - P0, P2 - P1);
		const float V2 = ComputeMonotoneVelocity(P2 - P1, P3 - P2);
		HermiteValueInterpolate(OutValue, OutValueVelocity, P1, P2, V1, V2, Alpha, FrameTime);
	}

	void AngleInterpolateCubicMono(float& OutAngle, float& OutAngularVelocity, const float A0, const float A1, const float A2, const float A3, const float Alpha, const float FrameTime)
	{
		const float A1A0Diff = FMath::FindDeltaAngleRadians(A0, A1);
		const float A2A1Diff = FMath::FindDeltaAngleRadians(A1, A2);
		const float A3A2Diff = FMath::FindDeltaAngleRadians(A2, A3);
		const float V1 = ComputeMonotoneVelocity(A1A0Diff, A2A1Diff);
		const float V2 = ComputeMonotoneVelocity(A2A1Diff, A3A2Diff);
		HermiteAngleInterpolate(OutAngle, OutAngularVelocity, A1, A2, V1, V2, Alpha, FrameTime);
	}

	void LocationInterpolateCubicMono(FVector& OutLocation, FVector3f& OutLinearVelocity, const FVector P0, const FVector P1, const FVector P2, const FVector P3, const float Alpha, const float FrameTime)
	{
		const FVector3f V1 = ComputeMonotoneVelocity((FVector3f)(P1 - P0), (FVector3f)(P2 - P1));
		const FVector3f V2 = ComputeMonotoneVelocity((FVector3f)(P2 - P1), (FVector3f)(P3 - P2));
		HermiteLocationInterpolate(OutLocation, OutLinearVelocity, P1, P2, V1, V2, Alpha, FrameTime);
	}

	void LocationInterpolateCubicMono(FVector3f& OutLocation, FVector3f& OutLinearVelocity, const FVector3f P0, const FVector3f P1, const FVector3f P2, const FVector3f P3, const float Alpha, const float FrameTime)
	{
		const FVector3f V1 = ComputeMonotoneVelocity(P1 - P0, P2 - P1);
		const FVector3f V2 = ComputeMonotoneVelocity(P2 - P1, P3 - P2);
		HermiteLocationInterpolate(OutLocation, OutLinearVelocity, P1, P2, V1, V2, Alpha, FrameTime);
	}

	void RotationInterpolateCubicMono(FQuat4f& OutRotation, FVector3f& OutAngularVelocity, const FQuat4f R0, const FQuat4f R1, const FQuat4f R2, const FQuat4f R3, const float Alpha, const float FrameTime)
	{
		const FVector3f R1R0Diff = (R1 * R0.Inverse()).GetShortestArcWith(FQuat4f::Identity).ToRotationVector();
		const FVector3f R2R1Diff = (R2 * R1.Inverse()).GetShortestArcWith(FQuat4f::Identity).ToRotationVector();
		const FVector3f R3R2Diff = (R3 * R2.Inverse()).GetShortestArcWith(FQuat4f::Identity).ToRotationVector();
		const FVector3f V1 = ComputeMonotoneVelocity(R1R0Diff, R2R1Diff);
		const FVector3f V2 = ComputeMonotoneVelocity(R2R1Diff, R3R2Diff);
		HermiteRotationInterpolate(OutRotation, OutAngularVelocity, R1, R2, V1, V2, Alpha, FrameTime);
	}

	void ScaleInterpolateCubicMono(FVector3f& OutScale, FVector3f& OutScalarVelocity, const FVector3f S0, const FVector3f S1, const FVector3f S2, const FVector3f S3, const float Alpha, const float FrameTime)
	{
		const FVector3f S1SubS0 = VectorLogSafe(VectorDivMax(S1, S0, UE_SMALL_NUMBER));
		const FVector3f S2SubS1 = VectorLogSafe(VectorDivMax(S2, S1, UE_SMALL_NUMBER));
		const FVector3f S3SubS2 = VectorLogSafe(VectorDivMax(S3, S2, UE_SMALL_NUMBER));
		const FVector3f V1 = ComputeMonotoneVelocity(S1SubS0, S2SubS1);
		const FVector3f V2 = ComputeMonotoneVelocity(S2SubS1, S3SubS2);
		HermiteScaleInterpolate(OutScale, OutScalarVelocity, S1, S2, V1, V2, Alpha, FrameTime);
	}

	void ArrayInterpolateCubicMono(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<1, float> OutValueVelocities,
		const TLearningArrayView<1, const float> P0,
		const TLearningArrayView<1, const float> P1,
		const TLearningArrayView<1, const float> P2,
		const TLearningArrayView<1, const float> P3,
		const float Alpha,
		const float FrameTime)
	{
		const int64 ValueNum = OutValues.Num();

		for (int64 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
		{
			const float V1 = ComputeMonotoneVelocity(P1[ValueIdx] - P0[ValueIdx], P2[ValueIdx] - P1[ValueIdx]);
			const float V2 = ComputeMonotoneVelocity(P2[ValueIdx] - P1[ValueIdx], P3[ValueIdx] - P2[ValueIdx]);
			HermiteValueInterpolate(OutValues[ValueIdx], OutValueVelocities[ValueIdx], P1[ValueIdx], P2[ValueIdx], V1, V2, Alpha, FrameTime);
		}
	}

	void ValueInterpolateCubicMono(float& OutValue, const float P0, const float P1, const float P2, const float P3, const float Alpha)
	{
		const float V1 = ComputeMonotoneVelocity(P1 - P0, P2 - P1);
		const float V2 = ComputeMonotoneVelocity(P2 - P1, P3 - P2);
		HermiteValueInterpolate(OutValue, P1, P2, V1, V2, Alpha);
	}

	void AngleInterpolateCubicMono(float& OutAngle, const float A0, const float A1, const float A2, const float A3, const float Alpha)
	{
		const float A1A0Diff = FMath::FindDeltaAngleRadians(A0, A1);
		const float A2A1Diff = FMath::FindDeltaAngleRadians(A1, A2);
		const float A3A2Diff = FMath::FindDeltaAngleRadians(A2, A3);
		const float V1 = ComputeMonotoneVelocity(A1A0Diff, A2A1Diff);
		const float V2 = ComputeMonotoneVelocity(A2A1Diff, A3A2Diff);
		HermiteAngleInterpolate(OutAngle, A1, A2, V1, V2, Alpha);
	}

	void LocationInterpolateCubicMono(FVector& OutLocation, const FVector P0, const FVector P1, const FVector P2, const FVector P3, const float Alpha)
	{
		const FVector3f V1 = ComputeMonotoneVelocity((FVector3f)(P1 - P0), (FVector3f)(P2 - P1));
		const FVector3f V2 = ComputeMonotoneVelocity((FVector3f)(P2 - P1), (FVector3f)(P3 - P2));
		HermiteLocationInterpolate(OutLocation, P1, P2, V1, V2, Alpha);
	}

	void LocationInterpolateCubicMono(FVector3f& OutLocation, const FVector3f P0, const FVector3f P1, const FVector3f P2, const FVector3f P3, const float Alpha)
	{
		const FVector3f V1 = ComputeMonotoneVelocity(P1 - P0, P2 - P1);
		const FVector3f V2 = ComputeMonotoneVelocity(P2 - P1, P3 - P2);
		HermiteLocationInterpolate(OutLocation, P1, P2, V1, V2, Alpha);
	}

	void RotationInterpolateCubicMono(FQuat4f& OutRotation, const FQuat4f R0, const FQuat4f R1, const FQuat4f R2, const FQuat4f R3, const float Alpha)
	{
		const FVector3f R1R0Diff = (R1 * R0.Inverse()).GetShortestArcWith(FQuat4f::Identity).ToRotationVector();
		const FVector3f R2R1Diff = (R2 * R1.Inverse()).GetShortestArcWith(FQuat4f::Identity).ToRotationVector();
		const FVector3f R3R2Diff = (R3 * R2.Inverse()).GetShortestArcWith(FQuat4f::Identity).ToRotationVector();
		const FVector3f V1 = ComputeMonotoneVelocity(R1R0Diff, R2R1Diff);
		const FVector3f V2 = ComputeMonotoneVelocity(R2R1Diff, R3R2Diff);
		HermiteRotationInterpolate(OutRotation, R1, R2, V1, V2, Alpha);
	}

	void ScaleInterpolateCubicMono(FVector3f& OutScale, const FVector3f S0, const FVector3f S1, const FVector3f S2, const FVector3f S3, const float Alpha)
	{
		const FVector3f S1SubS0 = VectorLogSafe(VectorDivMax(S1, S0, UE_SMALL_NUMBER));
		const FVector3f S2SubS1 = VectorLogSafe(VectorDivMax(S2, S1, UE_SMALL_NUMBER));
		const FVector3f S3SubS2 = VectorLogSafe(VectorDivMax(S3, S2, UE_SMALL_NUMBER));
		const FVector3f V1 = ComputeMonotoneVelocity(S1SubS0, S2SubS1);
		const FVector3f V2 = ComputeMonotoneVelocity(S2SubS1, S3SubS2);
		HermiteScaleInterpolate(OutScale, S1, S2, V1, V2, Alpha);
	}

	void DirectionInterpolateCubicMono(FVector3f& OutDirection, const FVector3f D0, const FVector3f D1, const FVector3f D2, const FVector3f D3, const float Alpha)
	{
		LocationInterpolateCubicMono(OutDirection, D0, D1, D2, D3, Alpha);
		OutDirection = OutDirection.GetSafeNormal(UE_SMALL_NUMBER, FVector3f::ForwardVector);
	}

	void TransformInterpolateCubicMono(FTransform3f& OutTransform, const FTransform3f T0, const FTransform3f T1, const FTransform3f T2, const FTransform3f T3, const float Alpha)
	{
		FVector3f OutLocation;
		FQuat4f OutRotation;
		FVector3f OutScale;
		LocationInterpolateCubicMono(OutLocation, T0.GetLocation(), T1.GetLocation(), T2.GetLocation(), T3.GetLocation(), Alpha);
		RotationInterpolateCubicMono(OutRotation, T0.GetRotation(), T1.GetRotation(), T2.GetRotation(), T3.GetRotation(), Alpha);
		ScaleInterpolateCubicMono(OutScale, T0.GetScale3D(), T1.GetScale3D(), T2.GetScale3D(), T3.GetScale3D(), Alpha);
		OutTransform = FTransform3f(OutRotation, OutLocation, OutScale);
	}

	void ArrayInterpolateCubicMono(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<1, const float> P0,
		const TLearningArrayView<1, const float> P1,
		const TLearningArrayView<1, const float> P2,
		const TLearningArrayView<1, const float> P3,
		const float Alpha)
	{
		const int64 ValueNum = OutValues.Num();

		for (int64 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
		{
			const float V1 = ComputeMonotoneVelocity(P1[ValueIdx] - P0[ValueIdx], P2[ValueIdx] - P1[ValueIdx]);
			const float V2 = ComputeMonotoneVelocity(P2[ValueIdx] - P1[ValueIdx], P3[ValueIdx] - P2[ValueIdx]);
			HermiteValueInterpolate(OutValues[ValueIdx], P1[ValueIdx], P2[ValueIdx], V1, V2, Alpha);
		}
	}

	void ValueInterpolateCubicMonoStart(float& OutValue, float& OutValueVelocity, const float P1, const float P2, const float P3, const float Alpha, const float FrameTime)
	{
		const float V1 = (P2 - P1);
		const float V2 = ComputeMonotoneVelocity(P2 - P1, P3 - P2);
		HermiteValueInterpolate(OutValue, OutValueVelocity, P1, P2, V1, V2, Alpha, FrameTime);
	}

	void AngleInterpolateCubicMonoStart(float& OutAngle, float& OutAngularVelocity, const float A1, const float A2, const float A3, const float Alpha, const float FrameTime)
	{
		const float A2A1Diff = FMath::FindDeltaAngleRadians(A1, A2);
		const float A3A2Diff = FMath::FindDeltaAngleRadians(A2, A3);
		const float V1 = A2A1Diff;
		const float V2 = ComputeMonotoneVelocity(A2A1Diff, A3A2Diff);
		HermiteAngleInterpolate(OutAngle, OutAngularVelocity, A1, A2, V1, V2, Alpha, FrameTime);
	}

	void LocationInterpolateCubicMonoStart(FVector& OutLocation, FVector3f& OutLinearVelocity, const FVector P1, const FVector P2, const FVector P3, const float Alpha, const float FrameTime)
	{
		const FVector3f V1 = (FVector3f)(P2 - P1);
		const FVector3f V2 = ComputeMonotoneVelocity((FVector3f)(P2 - P1), (FVector3f)(P3 - P2));
		HermiteLocationInterpolate(OutLocation, OutLinearVelocity, P1, P2, V1, V2, Alpha, FrameTime);
	}

	void LocationInterpolateCubicMonoStart(FVector3f& OutLocation, FVector3f& OutLinearVelocity, const FVector3f P1, const FVector3f P2, const FVector3f P3, const float Alpha, const float FrameTime)
	{
		const FVector3f V1 = (P2 - P1);
		const FVector3f V2 = ComputeMonotoneVelocity(P2 - P1, P3 - P2);
		HermiteLocationInterpolate(OutLocation, OutLinearVelocity, P1, P2, V1, V2, Alpha, FrameTime);
	}

	void RotationInterpolateCubicMonoStart(FQuat4f& OutRotation, FVector3f& OutAngularVelocity, const FQuat4f R1, const FQuat4f R2, const FQuat4f R3, const float Alpha, const float FrameTime)
	{
		const FVector3f R2R1Diff = (R2 * R1.Inverse()).GetShortestArcWith(FQuat4f::Identity).ToRotationVector();
		const FVector3f R3R2Diff = (R3 * R2.Inverse()).GetShortestArcWith(FQuat4f::Identity).ToRotationVector();
		const FVector3f V1 = R2R1Diff;
		const FVector3f V2 = ComputeMonotoneVelocity(R2R1Diff, R3R2Diff);
		HermiteRotationInterpolate(OutRotation, OutAngularVelocity, R1, R2, V1, V2, Alpha, FrameTime);
	}

	void ScaleInterpolateCubicMonoStart(FVector3f& OutScale, FVector3f& OutScalarVelocity, const FVector3f S1, const FVector3f S2, const FVector3f S3, const float Alpha, const float FrameTime)
	{
		const FVector3f S2SubS1 = VectorLogSafe(VectorDivMax(S2, S1, UE_SMALL_NUMBER));
		const FVector3f S3SubS2 = VectorLogSafe(VectorDivMax(S3, S2, UE_SMALL_NUMBER));
		const FVector3f V1 = S2SubS1;
		const FVector3f V2 = ComputeMonotoneVelocity(S2SubS1, S3SubS2);
		HermiteScaleInterpolate(OutScale, OutScalarVelocity, S1, S2, V1, V2, Alpha, FrameTime);
	}

	void ArrayInterpolateCubicMonoStart(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<1, float> OutValueVelocities,
		const TLearningArrayView<1, const float> P1,
		const TLearningArrayView<1, const float> P2,
		const TLearningArrayView<1, const float> P3,
		const float Alpha,
		const float FrameTime)
	{
		const int64 ValueNum = OutValues.Num();

		for (int64 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
		{
			const float V1 = (P2[ValueIdx] - P1[ValueIdx]);
			const float V2 = ComputeMonotoneVelocity(P2[ValueIdx] - P1[ValueIdx], P3[ValueIdx] - P2[ValueIdx]);
			HermiteValueInterpolate(OutValues[ValueIdx], OutValueVelocities[ValueIdx], P1[ValueIdx], P2[ValueIdx], V1, V2, Alpha, FrameTime);
		}
	}

	void ValueInterpolateCubicMonoStart(float& OutValue, const float P1, const float P2, const float P3, const float Alpha)
	{
		const float V1 = (P2 - P1);
		const float V2 = ComputeMonotoneVelocity(P2 - P1, P3 - P2);
		HermiteValueInterpolate(OutValue, P1, P2, V1, V2, Alpha);
	}

	void AngleInterpolateCubicMonoStart(float& OutAngle, const float A1, const float A2, const float A3, const float Alpha)
	{
		const float A2A1Diff = FMath::FindDeltaAngleRadians(A1, A2);
		const float A3A2Diff = FMath::FindDeltaAngleRadians(A2, A3);
		const float V1 = A2A1Diff;
		const float V2 = ComputeMonotoneVelocity(A2A1Diff, A3A2Diff);
		HermiteAngleInterpolate(OutAngle, A1, A2, V1, V2, Alpha);
	}

	void LocationInterpolateCubicMonoStart(FVector& OutLocation, const FVector P1, const FVector P2, const FVector P3, const float Alpha)
	{
		const FVector3f V1 = (FVector3f)(P2 - P1);
		const FVector3f V2 = ComputeMonotoneVelocity((FVector3f)(P2 - P1), (FVector3f)(P3 - P2));
		HermiteLocationInterpolate(OutLocation, P1, P2, V1, V2, Alpha);
	}

	void LocationInterpolateCubicMonoStart(FVector3f& OutLocation, const FVector3f P1, const FVector3f P2, const FVector3f P3, const float Alpha)
	{
		const FVector3f V1 = (P2 - P1);
		const FVector3f V2 = ComputeMonotoneVelocity(P2 - P1, P3 - P2);
		HermiteLocationInterpolate(OutLocation, P1, P2, V1, V2, Alpha);
	}

	void RotationInterpolateCubicMonoStart(FQuat4f& OutRotation, const FQuat4f R1, const FQuat4f R2, const FQuat4f R3, const float Alpha)
	{
		const FVector3f R2R1Diff = (R2 * R1.Inverse()).GetShortestArcWith(FQuat4f::Identity).ToRotationVector();
		const FVector3f R3R2Diff = (R3 * R2.Inverse()).GetShortestArcWith(FQuat4f::Identity).ToRotationVector();
		const FVector3f V1 = R2R1Diff;
		const FVector3f V2 = ComputeMonotoneVelocity(R2R1Diff, R3R2Diff);
		HermiteRotationInterpolate(OutRotation, R1, R2, V1, V2, Alpha);
	}

	void ScaleInterpolateCubicMonoStart(FVector3f& OutScale, const FVector3f S1, const FVector3f S2, const FVector3f S3, const float Alpha)
	{
		const FVector3f S2SubS1 = VectorLogSafe(VectorDivMax(S2, S1, UE_SMALL_NUMBER));
		const FVector3f S3SubS2 = VectorLogSafe(VectorDivMax(S3, S2, UE_SMALL_NUMBER));
		const FVector3f V1 = S2SubS1;
		const FVector3f V2 = ComputeMonotoneVelocity(S2SubS1, S3SubS2);
		HermiteScaleInterpolate(OutScale, S1, S2, V1, V2, Alpha);
	}

	void DirectionInterpolateCubicMonoStart(FVector3f& OutDirection, const FVector3f D1, const FVector3f D2, const FVector3f D3, const float Alpha)
	{
		LocationInterpolateCubicMonoStart(OutDirection, D1, D2, D3, Alpha);
		OutDirection = OutDirection.GetSafeNormal(UE_SMALL_NUMBER, FVector3f::ForwardVector);
	}

	void TransformInterpolateCubicMonoStart(FTransform3f& OutTransform, const FTransform3f T1, const FTransform3f T2, const FTransform3f T3, const float Alpha)
	{
		FVector3f OutLocation;
		FQuat4f OutRotation;
		FVector3f OutScale;
		LocationInterpolateCubicMonoStart(OutLocation, T1.GetLocation(), T2.GetLocation(), T3.GetLocation(), Alpha);
		RotationInterpolateCubicMonoStart(OutRotation, T1.GetRotation(), T2.GetRotation(), T3.GetRotation(), Alpha);
		ScaleInterpolateCubicMonoStart(OutScale, T1.GetScale3D(), T2.GetScale3D(), T3.GetScale3D(), Alpha);
		OutTransform = FTransform3f(OutRotation, OutLocation, OutScale);
	}

	void ArrayInterpolateCubicMonoStart(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<1, const float> P1,
		const TLearningArrayView<1, const float> P2,
		const TLearningArrayView<1, const float> P3,
		const float Alpha)
	{
		const int64 ValueNum = OutValues.Num();

		for (int64 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
		{
			const float V1 = (P2[ValueIdx] - P1[ValueIdx]);
			const float V2 = ComputeMonotoneVelocity(P2[ValueIdx] - P1[ValueIdx], P3[ValueIdx] - P2[ValueIdx]);
			HermiteValueInterpolate(OutValues[ValueIdx], P1[ValueIdx], P2[ValueIdx], V1, V2, Alpha);
		}
	}

	void ValueInterpolateCubicMonoEnd(float& OutValue, float& OutValueVelocity, const float P0, const float P1, const float P2, const float Alpha, const float FrameTime)
	{
		const float V1 = ComputeMonotoneVelocity(P1 - P0, P2 - P1);
		const float V2 = (P2 - P1);
		HermiteValueInterpolate(OutValue, OutValueVelocity, P1, P2, V1, V2, Alpha, FrameTime);
	}

	void AngleInterpolateCubicMonoEnd(float& OutAngle, float& OutAngularVelocity, const float A0, const float A1, const float A2, const float Alpha, const float FrameTime)
	{
		const float A1A0Diff = FMath::FindDeltaAngleRadians(A0, A1);
		const float A2A1Diff = FMath::FindDeltaAngleRadians(A1, A2);
		const float V1 = ComputeMonotoneVelocity(A1A0Diff, A2A1Diff);
		const float V2 = A2A1Diff;
		HermiteAngleInterpolate(OutAngle, OutAngularVelocity, A1, A2, V1, V2, Alpha, FrameTime);
	}

	void LocationInterpolateCubicMonoEnd(FVector& OutLocation, FVector3f& OutLinearVelocity, const FVector P0, const FVector P1, const FVector P2, const float Alpha, const float FrameTime)
	{
		const FVector3f V1 = ComputeMonotoneVelocity((FVector3f)(P1 - P0), (FVector3f)(P2 - P1));
		const FVector3f V2 = (FVector3f)(P2 - P1);
		HermiteLocationInterpolate(OutLocation, OutLinearVelocity, P1, P2, V1, V2, Alpha, FrameTime);
	}

	void LocationInterpolateCubicMonoEnd(FVector3f& OutLocation, FVector3f& OutLinearVelocity, const FVector3f P0, const FVector3f P1, const FVector3f P2, const float Alpha, const float FrameTime)
	{
		const FVector3f V1 = ComputeMonotoneVelocity(P1 - P0, P2 - P1);
		const FVector3f V2 = (P2 - P1);
		HermiteLocationInterpolate(OutLocation, OutLinearVelocity, P1, P2, V1, V2, Alpha, FrameTime);
	}

	void RotationInterpolateCubicMonoEnd(FQuat4f& OutRotation, FVector3f& OutAngularVelocity, const FQuat4f R0, const FQuat4f R1, const FQuat4f R2, const float Alpha, const float FrameTime)
	{
		const FVector3f R1R0Diff = (R1 * R0.Inverse()).GetShortestArcWith(FQuat4f::Identity).ToRotationVector();
		const FVector3f R2R1Diff = (R2 * R1.Inverse()).GetShortestArcWith(FQuat4f::Identity).ToRotationVector();
		const FVector3f V1 = ComputeMonotoneVelocity(R1R0Diff, R2R1Diff);
		const FVector3f V2 = R2R1Diff;
		HermiteRotationInterpolate(OutRotation, OutAngularVelocity, R1, R2, V1, V2, Alpha, FrameTime);
	}

	void ScaleInterpolateCubicMonoEnd(FVector3f& OutScale, FVector3f& OutScalarVelocity, const FVector3f S0, const FVector3f S1, const FVector3f S2, const float Alpha, const float FrameTime)
	{
		const FVector3f S1SubS0 = VectorLogSafe(VectorDivMax(S1, S0, UE_SMALL_NUMBER));
		const FVector3f S2SubS1 = VectorLogSafe(VectorDivMax(S2, S1, UE_SMALL_NUMBER));
		const FVector3f V1 = ComputeMonotoneVelocity(S1SubS0, S2SubS1);
		const FVector3f V2 = S2SubS1;
		HermiteScaleInterpolate(OutScale, OutScalarVelocity, S1, S2, V1, V2, Alpha, FrameTime);
	}

	void ArrayInterpolateCubicMonoEnd(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<1, float> OutValueVelocities,
		const TLearningArrayView<1, const float> P0,
		const TLearningArrayView<1, const float> P1,
		const TLearningArrayView<1, const float> P2,
		const float Alpha,
		const float FrameTime)
	{
		const int64 ValueNum = OutValues.Num();

		for (int64 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
		{
			const float V1 = ComputeMonotoneVelocity(P1[ValueIdx] - P0[ValueIdx], P2[ValueIdx] - P1[ValueIdx]);
			const float V2 = (P2[ValueIdx] - P1[ValueIdx]);
			HermiteValueInterpolate(OutValues[ValueIdx], OutValueVelocities[ValueIdx], P1[ValueIdx], P2[ValueIdx], V1, V2, Alpha, FrameTime);
		}
	}

	void ValueInterpolateCubicMonoEnd(float& OutValue, const float P0, const float P1, const float P2, const float Alpha)
	{
		const float V1 = ComputeMonotoneVelocity(P1 - P0, P2 - P1);
		const float V2 = (P2 - P1);
		HermiteValueInterpolate(OutValue, P1, P2, V1, V2, Alpha);
	}

	void AngleInterpolateCubicMonoEnd(float& OutAngle, const float A0, const float A1, const float A2, const float Alpha)
	{
		const float A1A0Diff = FMath::FindDeltaAngleRadians(A0, A1);
		const float A2A1Diff = FMath::FindDeltaAngleRadians(A1, A2);
		const float V1 = ComputeMonotoneVelocity(A1A0Diff, A2A1Diff);
		const float V2 = A2A1Diff;
		HermiteAngleInterpolate(OutAngle, A1, A2, V1, V2, Alpha);
	}

	void LocationInterpolateCubicMonoEnd(FVector& OutLocation, const FVector P0, const FVector P1, const FVector P2, const float Alpha)
	{
		const FVector3f V1 = ComputeMonotoneVelocity((FVector3f)(P1 - P0), (FVector3f)(P2 - P1));
		const FVector3f V2 = (FVector3f)(P2 - P1);
		HermiteLocationInterpolate(OutLocation, P1, P2, V1, V2, Alpha);
	}

	void LocationInterpolateCubicMonoEnd(FVector3f& OutLocation, const FVector3f P0, const FVector3f P1, const FVector3f P2, const float Alpha)
	{
		const FVector3f V1 = ComputeMonotoneVelocity(P1 - P0, P2 - P1);
		const FVector3f V2 = (P2 - P1);
		HermiteLocationInterpolate(OutLocation, P1, P2, V1, V2, Alpha);
	}

	void RotationInterpolateCubicMonoEnd(FQuat4f& OutRotation, const FQuat4f R0, const FQuat4f R1, const FQuat4f R2, const float Alpha)
	{
		const FVector3f R1R0Diff = (R1 * R0.Inverse()).GetShortestArcWith(FQuat4f::Identity).ToRotationVector();
		const FVector3f R2R1Diff = (R2 * R1.Inverse()).GetShortestArcWith(FQuat4f::Identity).ToRotationVector();
		const FVector3f V1 = ComputeMonotoneVelocity(R1R0Diff, R2R1Diff);
		const FVector3f V2 = R2R1Diff;
		HermiteRotationInterpolate(OutRotation, R1, R2, V1, V2, Alpha);
	}

	void ScaleInterpolateCubicMonoEnd(FVector3f& OutScale, const FVector3f S0, const FVector3f S1, const FVector3f S2, const float Alpha)
	{
		const FVector3f S1SubS0 = VectorLogSafe(VectorDivMax(S1, S0, UE_SMALL_NUMBER));
		const FVector3f S2SubS1 = VectorLogSafe(VectorDivMax(S2, S1, UE_SMALL_NUMBER));
		const FVector3f V1 = ComputeMonotoneVelocity(S1SubS0, S2SubS1);
		const FVector3f V2 = S2SubS1;
		HermiteScaleInterpolate(OutScale, S1, S2, V1, V2, Alpha);
	}

	void DirectionInterpolateCubicMonoEnd(FVector3f& OutDirection, const FVector3f D0, const FVector3f D1, const FVector3f D2, const float Alpha)
	{
		LocationInterpolateCubicMonoEnd(OutDirection, D0, D1, D2, Alpha);
		OutDirection = OutDirection.GetSafeNormal(UE_SMALL_NUMBER, FVector3f::ForwardVector);
	}

	void TransformInterpolateCubicMonoEnd(FTransform3f& OutTransform, const FTransform3f T0, const FTransform3f T1, const FTransform3f T2, const float Alpha)
	{
		FVector3f OutLocation;
		FQuat4f OutRotation;
		FVector3f OutScale;
		LocationInterpolateCubicMonoEnd(OutLocation, T0.GetLocation(), T1.GetLocation(), T2.GetLocation(), Alpha);
		RotationInterpolateCubicMonoEnd(OutRotation, T0.GetRotation(), T1.GetRotation(), T2.GetRotation(), Alpha);
		ScaleInterpolateCubicMonoEnd(OutScale, T0.GetScale3D(), T1.GetScale3D(), T2.GetScale3D(), Alpha);
		OutTransform = FTransform3f(OutRotation, OutLocation, OutScale);
	}

	void ArrayInterpolateCubicMonoEnd(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<1, const float> P0,
		const TLearningArrayView<1, const float> P1,
		const TLearningArrayView<1, const float> P2,
		const float Alpha)
	{
		const int64 ValueNum = OutValues.Num();

		for (int64 ValueIdx = 0; ValueIdx < ValueNum; ValueIdx++)
		{
			const float V1 = ComputeMonotoneVelocity(P1[ValueIdx] - P0[ValueIdx], P2[ValueIdx] - P1[ValueIdx]);
			const float V2 = (P2[ValueIdx] - P1[ValueIdx]);
			HermiteValueInterpolate(OutValues[ValueIdx], P1[ValueIdx], P2[ValueIdx], V1, V2, Alpha);
		}
	}

	// Frame Sample Indices and Alpha

	int64 ComputeNearestSampleFrame(
		const float FrameTime,
		const int64 FrameNum)
	{
		const float ClampedFrameTime = FMath::Clamp(FrameTime, 0.0f, FMath::Max(FrameNum - 1, 0));
		return FMath::Clamp(FMath::RoundToInt(ClampedFrameTime), 0, FMath::Max(FrameNum - 1, 0));
	}

	void ComputeLinearSampleFramesAndAlpha(
		int64& OutSampleFrame0,
		int64& OutSampleFrame1,
		float& OutSampleAlpha,
		const float FrameTime,
		const int64 FrameNum)
	{
		const float ClampedFrameTime = FMath::Clamp(FrameTime, 0.0f, FMath::Max(FrameNum - 1, 0));
		const int64 SampleFrame = FMath::Clamp(FMath::FloorToInt(ClampedFrameTime), 0, FMath::Max(FrameNum - 1, 0));
		OutSampleFrame0 = FMath::Clamp(SampleFrame + 0, 0, FMath::Max(FrameNum - 1, 0));
		OutSampleFrame1 = FMath::Clamp(SampleFrame + 1, 0, FMath::Max(FrameNum - 1, 0));
		OutSampleAlpha = ClampedFrameTime - (float)SampleFrame;
	}

	void ComputeCubicSampleFramesAndAlpha(
		int64& OutSampleFrame0,
		int64& OutSampleFrame1,
		int64& OutSampleFrame2,
		int64& OutSampleFrame3,
		float& OutSampleAlpha,
		const float FrameTime,
		const int64 FrameNum)
	{
		const float ClampedFrameTime = FMath::Clamp(FrameTime, 0.0f, FMath::Max(FrameNum - 1, 0));
		const int64 SampleFrame = FMath::Clamp(FMath::FloorToInt(ClampedFrameTime), 0, FMath::Max(FrameNum - 2, 0));
		OutSampleFrame0 = FMath::Clamp(SampleFrame - 1, 0, FMath::Max(FrameNum - 1, 0));
		OutSampleFrame1 = FMath::Clamp(SampleFrame + 0, 0, FMath::Max(FrameNum - 1, 0));
		OutSampleFrame2 = FMath::Clamp(SampleFrame + 1, 0, FMath::Max(FrameNum - 1, 0));
		OutSampleFrame3 = FMath::Clamp(SampleFrame + 2, 0, FMath::Max(FrameNum - 1, 0));
		OutSampleAlpha = ClampedFrameTime - (float)SampleFrame;
	}

	int64 ComputeNearestSampleFrame(
		const int64 FrameTime,
		const int64 FrameNum)
	{
		return FMath::Clamp(FrameTime, 0, FMath::Max(FrameNum - 1, 0));
	}

	void ComputeLinearSampleFramesAndAlpha(
		int64& OutSampleFrame0,
		int64& OutSampleFrame1,
		float& OutSampleAlpha,
		const int64 FrameTime,
		const int64 FrameNum)
	{
		const int64 ClampedFrameTime = FMath::Clamp(FrameTime, 0, FMath::Max(FrameNum - 1, 0));
		const int64 SampleFrame = FMath::Clamp(ClampedFrameTime, 0, FMath::Max(FrameNum - 1, 0));
		OutSampleFrame0 = FMath::Clamp(SampleFrame + 0, 0, FMath::Max(FrameNum - 1, 0));
		OutSampleFrame1 = FMath::Clamp(SampleFrame + 1, 0, FMath::Max(FrameNum - 1, 0));
		OutSampleAlpha = ClampedFrameTime - SampleFrame;
	}

	void ComputeCubicSampleFramesAndAlpha(
		int64& OutSampleFrame0,
		int64& OutSampleFrame1,
		int64& OutSampleFrame2,
		int64& OutSampleFrame3,
		float& OutSampleAlpha,
		const int64 FrameTime,
		const int64 FrameNum)
	{
		const int64 ClampedFrameTime = FMath::Clamp(FrameTime, 0, FMath::Max(FrameNum - 1, 0));
		const int64 SampleFrame = FMath::Clamp(ClampedFrameTime, 0, FMath::Max(FrameNum - 2, 0));
		OutSampleFrame0 = FMath::Clamp(SampleFrame - 1, 0, FMath::Max(FrameNum - 1, 0));
		OutSampleFrame1 = FMath::Clamp(SampleFrame + 0, 0, FMath::Max(FrameNum - 1, 0));
		OutSampleFrame2 = FMath::Clamp(SampleFrame + 1, 0, FMath::Max(FrameNum - 1, 0));
		OutSampleFrame3 = FMath::Clamp(SampleFrame + 2, 0, FMath::Max(FrameNum - 1, 0));
		OutSampleAlpha = ClampedFrameTime - SampleFrame;
	}

	// Sampling

	void ValueSampleNearest(
		float& OutValue,
		const TLearningArrayView<1, const float> InValues,
		const int32 SampleFrame)
	{
		OutValue = InValues[SampleFrame];
	}

	void AngleSampleNearest(
		float& OutAngle,
		const TLearningArrayView<1, const float> InAngles,
		const int32 SampleFrame)
	{
		OutAngle = InAngles[SampleFrame];
	}

	void LocationSampleNearest(
		FVector& OutLocation,
		const TLearningArrayView<1, const FVector> InLocations,
		const int32 SampleFrame)
	{
		OutLocation = InLocations[SampleFrame];
	}

	void LocationSampleNearest(
		FVector& OutLocation,
		FVector3f& OutLinearVelocity,
		const TLearningArrayView<1, const FVector> InLocations,
		const int32 SampleFrame)
	{
		OutLocation = InLocations[SampleFrame];
		OutLinearVelocity = FVector3f::ZeroVector;
	}

	void RotationSampleNearest(
		FQuat4f& OutRotation,
		const TLearningArrayView<1, const FQuat4f> InRotations,
		const int32 SampleFrame)
	{
		OutRotation = InRotations[SampleFrame];
	}

	void RotationSampleNearest(
		FQuat4f& OutRotation,
		FVector3f& OutAngularVelocity,
		const TLearningArrayView<1, const FQuat4f> InRotations,
		const int32 SampleFrame)
	{
		OutRotation = InRotations[SampleFrame];
		OutAngularVelocity = FVector3f::ZeroVector;
	}

	void TransformSampleNearest(
		FVector& OutLocation,
		FQuat4f& OutRotation,
		FVector3f& OutScale,
		const TLearningArrayView<1, const FVector> InLocations,
		const TLearningArrayView<1, const FQuat4f> InRotations,
		const TLearningArrayView<1, const FVector3f> InScales,
		const int32 SampleFrame)
	{
		OutLocation = InLocations[SampleFrame];
		OutRotation = InRotations[SampleFrame];
		OutScale = InScales[SampleFrame];
	}

	void TransformSampleNearest(
		FVector& OutLocation,
		FQuat4f& OutRotation,
		FVector3f& OutScale,
		FVector3f& OutLinearVelocity,
		FVector3f& OutAngularVelocity,
		FVector3f& OutScalarVelocity,
		const TLearningArrayView<1, const FVector> InLocations,
		const TLearningArrayView<1, const FQuat4f> InRotations,
		const TLearningArrayView<1, const FVector3f> InScales,
		const int32 SampleFrame)
	{
		OutLocation = InLocations[SampleFrame];
		OutRotation = InRotations[SampleFrame];
		OutScale = InScales[SampleFrame];
		OutLinearVelocity = FVector3f::ZeroVector;
		OutAngularVelocity = FVector3f::ZeroVector;
		OutScalarVelocity = FVector3f::ZeroVector;
	}

	void TransformSampleNearest(
		const TLearningArrayView<1, FVector3f> OutLocations,
		const TLearningArrayView<1, FQuat4f> OutRotations,
		const TLearningArrayView<1, FVector3f> OutScales,
		const TLearningArrayView<1, FVector3f> OutLinearVelocities,
		const TLearningArrayView<1, FVector3f> OutAngularVelocities,
		const TLearningArrayView<1, FVector3f> OutScalarVelocities,
		const TLearningArrayView<2, const FVector3f> InLocations,
		const TLearningArrayView<2, const FQuat4f> InRotations,
		const TLearningArrayView<2, const FVector3f> InScales,
		const int32 SampleFrame)
	{
		Learning::Array::Copy(OutLocations, InLocations[SampleFrame]);
		Learning::Array::Copy(OutRotations, InRotations[SampleFrame]);
		Learning::Array::Copy(OutScales, InScales[SampleFrame]);
		Learning::Array::Zero(OutLinearVelocities);
		Learning::Array::Zero(OutAngularVelocities);
		Learning::Array::Zero(OutScalarVelocities);
	}

	void ArraySampleNearest(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<2, const float> InValues,
		const int32 SampleFrame)
	{
		Learning::Array::Copy(OutValues, InValues[SampleFrame]);
	}

	void ArraySampleNearest(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<1, float> OutVelocities,
		const TLearningArrayView<2, const float> InValues,
		const int32 SampleFrame)
	{
		Learning::Array::Copy(OutValues, InValues[SampleFrame]);
		Learning::Array::Zero(OutVelocities);
	}


	void ValueSampleLinear(
		float& OutValue,
		const TLearningArrayView<1, const float> InValues,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const float SampleAlpha)
	{
		Math::ValueInterpolateLinear(OutValue, InValues[SampleFrame0], InValues[SampleFrame1], SampleAlpha);
	}

	void AngleSampleLinear(
		float& OutAngle,
		const TLearningArrayView<1, const float> InAngles,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const float SampleAlpha)
	{
		Math::AngleInterpolateLinear(OutAngle, InAngles[SampleFrame0], InAngles[SampleFrame1], SampleAlpha);
	}

	void LocationSampleLinear(
		FVector& OutLocation,
		const TLearningArrayView<1, const FVector> InLocations,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const float SampleAlpha)
	{
		Math::LocationInterpolateLinear(OutLocation, InLocations[SampleFrame0], InLocations[SampleFrame1], SampleAlpha);
	}

	void LocationSampleLinear(
		FVector& OutLocation,
		FVector3f& OutLinearVelocity,
		const TLearningArrayView<1, const FVector> InLocations,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const float SampleAlpha,
		const float FrameDeltaTime)
	{
		Math::LocationInterpolateLinear(OutLocation, OutLinearVelocity, InLocations[SampleFrame0], InLocations[SampleFrame1], SampleAlpha, FrameDeltaTime);
	}

	void RotationSampleLinear(
		FQuat4f& OutRotation,
		const TLearningArrayView<1, const FQuat4f> InRotations,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const float SampleAlpha)
	{
		Math::RotationInterpolateLinear(OutRotation, InRotations[SampleFrame0], InRotations[SampleFrame1], SampleAlpha);
	}

	void RotationSampleLinear(
		FQuat4f& OutRotation,
		FVector3f& OutAngularVelocity,
		const TLearningArrayView<1, const FQuat4f> InRotations,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const float SampleAlpha,
		const float FrameDeltaTime)
	{
		Math::RotationInterpolateLinear(OutRotation, OutAngularVelocity, InRotations[SampleFrame0], InRotations[SampleFrame1], SampleAlpha, FrameDeltaTime);
	}

	void TransformSampleLinear(
		FVector& OutLocation,
		FQuat4f& OutRotation,
		FVector3f& OutScale,
		const TLearningArrayView<1, const FVector> InLocations,
		const TLearningArrayView<1, const FQuat4f> InRotations,
		const TLearningArrayView<1, const FVector3f> InScales,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const float SampleAlpha)
	{
		Math::LocationInterpolateLinear(OutLocation, InLocations[SampleFrame0], InLocations[SampleFrame1], SampleAlpha);
		Math::RotationInterpolateLinear(OutRotation, InRotations[SampleFrame0], InRotations[SampleFrame1], SampleAlpha);
		Math::ScaleInterpolateLinear(OutScale, InScales[SampleFrame0], InScales[SampleFrame1], SampleAlpha);
	}

	void TransformSampleLinear(
		FVector& OutLocation,
		FQuat4f& OutRotation,
		FVector3f& OutScale,
		FVector3f& OutLinearVelocity,
		FVector3f& OutAngularVelocity,
		FVector3f& OutScalarVelocity,
		const TLearningArrayView<1, const FVector> InLocations,
		const TLearningArrayView<1, const FQuat4f> InRotations,
		const TLearningArrayView<1, const FVector3f> InScales,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const float SampleAlpha,
		const float FrameDeltaTime)
	{
		Math::LocationInterpolateLinear(OutLocation, OutLinearVelocity, InLocations[SampleFrame0], InLocations[SampleFrame1], SampleAlpha, FrameDeltaTime);
		Math::RotationInterpolateLinear(OutRotation, OutAngularVelocity, InRotations[SampleFrame0], InRotations[SampleFrame1], SampleAlpha, FrameDeltaTime);
		Math::ScaleInterpolateLinear(OutScale, OutScalarVelocity, InScales[SampleFrame0], InScales[SampleFrame1], SampleAlpha, FrameDeltaTime);
	}

	void TransformSampleLinear(
		const TLearningArrayView<1, FVector3f> OutLocations,
		const TLearningArrayView<1, FQuat4f> OutRotations,
		const TLearningArrayView<1, FVector3f> OutScales,
		const TLearningArrayView<1, FVector3f> OutLinearVelocities,
		const TLearningArrayView<1, FVector3f> OutAngularVelocities,
		const TLearningArrayView<1, FVector3f> OutScalarVelocities,
		const TLearningArrayView<2, const FVector3f> InLocations,
		const TLearningArrayView<2, const FQuat4f> InRotations,
		const TLearningArrayView<2, const FVector3f> InScales,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const float SampleAlpha,
		const float FrameDeltaTime)
	{
		const int32 BoneNum = OutLocations.Num();

		for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
		{
			Math::LocationInterpolateLinear(OutLocations[BoneIdx], OutLinearVelocities[BoneIdx], InLocations[SampleFrame0][BoneIdx], InLocations[SampleFrame1][BoneIdx], SampleAlpha, FrameDeltaTime);
			Math::RotationInterpolateLinear(OutRotations[BoneIdx], OutAngularVelocities[BoneIdx], InRotations[SampleFrame0][BoneIdx], InRotations[SampleFrame1][BoneIdx], SampleAlpha, FrameDeltaTime);
			Math::ScaleInterpolateLinear(OutScales[BoneIdx], OutScalarVelocities[BoneIdx], InScales[SampleFrame0][BoneIdx], InScales[SampleFrame1][BoneIdx], SampleAlpha, FrameDeltaTime);
		}
	}

	void ArraySampleLinear(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<2, const float> InValues,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const float SampleAlpha)
	{
		ArrayInterpolateLinear(OutValues, InValues[SampleFrame0], InValues[SampleFrame1], SampleAlpha);
	}

	void ArraySampleLinear(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<1, float> OutVelocities,
		const TLearningArrayView<2, const float> InValues,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const float SampleAlpha,
		const float FrameDeltaTime)
	{
		ArrayInterpolateLinear(OutValues, OutVelocities, InValues[SampleFrame0], InValues[SampleFrame1], SampleAlpha, FrameDeltaTime);
	}

	void ValueSampleCubic(
		float& OutValue,
		const TLearningArrayView<1, const float> InValues,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha)
	{
		if (SampleFrame0 == SampleFrame1 && SampleFrame2 == SampleFrame3)
		{
			Math::ValueInterpolateLinear(OutValue, InValues[SampleFrame1], InValues[SampleFrame2], SampleAlpha);
		}
		else if (SampleFrame0 == SampleFrame1)
		{
			Math::ValueInterpolateCubicStart(OutValue, InValues[SampleFrame1], InValues[SampleFrame2], InValues[SampleFrame3], SampleAlpha);
		}
		else if (SampleFrame2 == SampleFrame3)
		{
			Math::ValueInterpolateCubicEnd(OutValue, InValues[SampleFrame0], InValues[SampleFrame1], InValues[SampleFrame2], SampleAlpha);
		}
		else
		{
			Math::ValueInterpolateCubic(OutValue, InValues[SampleFrame0], InValues[SampleFrame1], InValues[SampleFrame2], InValues[SampleFrame3], SampleAlpha);
		}
	}

	void AngleSampleCubic(
		float& OutAngle,
		const TLearningArrayView<1, const float> InAngles,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha)
	{
		if (SampleFrame0 == SampleFrame1 && SampleFrame2 == SampleFrame3)
		{
			Math::AngleInterpolateLinear(OutAngle, InAngles[SampleFrame1], InAngles[SampleFrame2], SampleAlpha);
		}
		else if (SampleFrame0 == SampleFrame1)
		{
			Math::AngleInterpolateCubicStart(OutAngle, InAngles[SampleFrame1], InAngles[SampleFrame2], InAngles[SampleFrame3], SampleAlpha);
		}
		else if (SampleFrame2 == SampleFrame3)
		{
			Math::AngleInterpolateCubicEnd(OutAngle, InAngles[SampleFrame0], InAngles[SampleFrame1], InAngles[SampleFrame2], SampleAlpha);
		}
		else
		{
			Math::AngleInterpolateCubic(OutAngle, InAngles[SampleFrame0], InAngles[SampleFrame1], InAngles[SampleFrame2], InAngles[SampleFrame3], SampleAlpha);
		}
	}

	void LocationSampleCubic(
		FVector& OutLocation,
		const TLearningArrayView<1, const FVector> InLocations,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha)
	{
		if (SampleFrame0 == SampleFrame1 && SampleFrame2 == SampleFrame3)
		{
			Math::LocationInterpolateLinear(OutLocation, InLocations[SampleFrame1], InLocations[SampleFrame2], SampleAlpha);
		}
		else if (SampleFrame0 == SampleFrame1)
		{
			Math::LocationInterpolateCubicStart(OutLocation, InLocations[SampleFrame1], InLocations[SampleFrame2], InLocations[SampleFrame3], SampleAlpha);
		}
		else if (SampleFrame2 == SampleFrame3)
		{
			Math::LocationInterpolateCubicEnd(OutLocation, InLocations[SampleFrame0], InLocations[SampleFrame1], InLocations[SampleFrame2], SampleAlpha);
		}
		else
		{
			Math::LocationInterpolateCubic(OutLocation, InLocations[SampleFrame0], InLocations[SampleFrame1], InLocations[SampleFrame2], InLocations[SampleFrame3], SampleAlpha);
		}
	}

	void LocationSampleCubic(
		FVector& OutLocation,
		FVector3f& OutLinearVelocity,
		const TLearningArrayView<1, const FVector> InLocations,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha,
		const float FrameDeltaTime)
	{
		if (SampleFrame0 == SampleFrame1 && SampleFrame2 == SampleFrame3)
		{
			Math::LocationInterpolateLinear(OutLocation, OutLinearVelocity, InLocations[SampleFrame1], InLocations[SampleFrame2], SampleAlpha, FrameDeltaTime);
		}
		else if (SampleFrame0 == SampleFrame1)
		{
			Math::LocationInterpolateCubicStart(OutLocation, OutLinearVelocity, InLocations[SampleFrame1], InLocations[SampleFrame2], InLocations[SampleFrame3], SampleAlpha, FrameDeltaTime);
		}
		else if (SampleFrame2 == SampleFrame3)
		{
			Math::LocationInterpolateCubicEnd(OutLocation, OutLinearVelocity, InLocations[SampleFrame0], InLocations[SampleFrame1], InLocations[SampleFrame2], SampleAlpha, FrameDeltaTime);
		}
		else
		{
			Math::LocationInterpolateCubic(OutLocation, OutLinearVelocity, InLocations[SampleFrame0], InLocations[SampleFrame1], InLocations[SampleFrame2], InLocations[SampleFrame3], SampleAlpha, FrameDeltaTime);
		}
	}

	void RotationSampleCubic(
		FQuat4f& OutRotation,
		const TLearningArrayView<1, const FQuat4f> InRotations,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha)
	{
		if (SampleFrame0 == SampleFrame1 && SampleFrame2 == SampleFrame3)
		{
			Math::RotationInterpolateLinear(OutRotation, InRotations[SampleFrame1], InRotations[SampleFrame2], SampleAlpha);
		}
		else if (SampleFrame0 == SampleFrame1)
		{
			Math::RotationInterpolateCubicStart(OutRotation, InRotations[SampleFrame1], InRotations[SampleFrame2], InRotations[SampleFrame3], SampleAlpha);
		}
		else if (SampleFrame2 == SampleFrame3)
		{
			Math::RotationInterpolateCubicEnd(OutRotation, InRotations[SampleFrame0], InRotations[SampleFrame1], InRotations[SampleFrame2], SampleAlpha);
		}
		else
		{
			Math::RotationInterpolateCubic(OutRotation, InRotations[SampleFrame0], InRotations[SampleFrame1], InRotations[SampleFrame2], InRotations[SampleFrame3], SampleAlpha);
		}
	}

	void RotationSampleCubic(
		FQuat4f& OutRotation,
		FVector3f& OutAngularVelocity,
		const TLearningArrayView<1, const FQuat4f> InRotations,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha,
		const float FrameDeltaTime)
	{
		if (SampleFrame0 == SampleFrame1 && SampleFrame2 == SampleFrame3)
		{
			Math::RotationInterpolateLinear(OutRotation, OutAngularVelocity, InRotations[SampleFrame1], InRotations[SampleFrame2], SampleAlpha, FrameDeltaTime);
		}
		else if (SampleFrame0 == SampleFrame1)
		{
			Math::RotationInterpolateCubicStart(OutRotation, OutAngularVelocity, InRotations[SampleFrame1], InRotations[SampleFrame2], InRotations[SampleFrame3], SampleAlpha, FrameDeltaTime);
		}
		else if (SampleFrame2 == SampleFrame3)
		{
			Math::RotationInterpolateCubicEnd(OutRotation, OutAngularVelocity, InRotations[SampleFrame0], InRotations[SampleFrame1], InRotations[SampleFrame2], SampleAlpha, FrameDeltaTime);
		}
		else
		{
			Math::RotationInterpolateCubic(OutRotation, OutAngularVelocity, InRotations[SampleFrame0], InRotations[SampleFrame1], InRotations[SampleFrame2], InRotations[SampleFrame3], SampleAlpha, FrameDeltaTime);
		}
	}

	void TransformSampleCubic(
		FVector& OutLocation,
		FQuat4f& OutRotation,
		FVector3f& OutScale,
		const TLearningArrayView<1, const FVector> InLocations,
		const TLearningArrayView<1, const FQuat4f> InRotations,
		const TLearningArrayView<1, const FVector3f> InScales,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha)
	{
		if (SampleFrame0 == SampleFrame1 && SampleFrame2 == SampleFrame3)
		{
			Math::LocationInterpolateLinear(OutLocation, InLocations[SampleFrame1], InLocations[SampleFrame2], SampleAlpha);
			Math::RotationInterpolateLinear(OutRotation, InRotations[SampleFrame1], InRotations[SampleFrame2], SampleAlpha);
			Math::ScaleInterpolateLinear(OutScale, InScales[SampleFrame1], InScales[SampleFrame2], SampleAlpha);
		}
		else if (SampleFrame0 == SampleFrame1)
		{
			Math::LocationInterpolateCubicStart(OutLocation, InLocations[SampleFrame1], InLocations[SampleFrame2], InLocations[SampleFrame3], SampleAlpha);
			Math::RotationInterpolateCubicStart(OutRotation, InRotations[SampleFrame1], InRotations[SampleFrame2], InRotations[SampleFrame3], SampleAlpha);
			Math::ScaleInterpolateCubicStart(OutScale, InScales[SampleFrame1], InScales[SampleFrame2], InScales[SampleFrame3], SampleAlpha);
		}
		else if (SampleFrame2 == SampleFrame3)
		{
			Math::LocationInterpolateCubicEnd(OutLocation, InLocations[SampleFrame0], InLocations[SampleFrame1], InLocations[SampleFrame2], SampleAlpha);
			Math::RotationInterpolateCubicEnd(OutRotation, InRotations[SampleFrame0], InRotations[SampleFrame1], InRotations[SampleFrame2], SampleAlpha);
			Math::ScaleInterpolateCubicEnd(OutScale, InScales[SampleFrame0], InScales[SampleFrame1], InScales[SampleFrame2], SampleAlpha);
		}
		else
		{
			Math::LocationInterpolateCubic(OutLocation, InLocations[SampleFrame0], InLocations[SampleFrame1], InLocations[SampleFrame2], InLocations[SampleFrame3], SampleAlpha);
			Math::RotationInterpolateCubic(OutRotation, InRotations[SampleFrame0], InRotations[SampleFrame1], InRotations[SampleFrame2], InRotations[SampleFrame3], SampleAlpha);
			Math::ScaleInterpolateCubic(OutScale, InScales[SampleFrame0], InScales[SampleFrame1], InScales[SampleFrame2], InScales[SampleFrame3], SampleAlpha);
		}
	}

	void TransformSampleCubic(
		FVector& OutLocation,
		FQuat4f& OutRotation,
		FVector3f& OutScale,
		FVector3f& OutLinearVelocity,
		FVector3f& OutAngularVelocity,
		FVector3f& OutScalarVelocity,
		const TLearningArrayView<1, const FVector> InLocations,
		const TLearningArrayView<1, const FQuat4f> InRotations,
		const TLearningArrayView<1, const FVector3f> InScales,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha,
		const float FrameDeltaTime)
	{
		if (SampleFrame0 == SampleFrame1 && SampleFrame2 == SampleFrame3)
		{
			Math::LocationInterpolateLinear(OutLocation, OutLinearVelocity, InLocations[SampleFrame1], InLocations[SampleFrame2], SampleAlpha, FrameDeltaTime);
			Math::RotationInterpolateLinear(OutRotation, OutAngularVelocity, InRotations[SampleFrame1], InRotations[SampleFrame2], SampleAlpha, FrameDeltaTime);
			Math::ScaleInterpolateLinear(OutScale, OutScalarVelocity, InScales[SampleFrame1], InScales[SampleFrame2], SampleAlpha, FrameDeltaTime);
		}
		else if (SampleFrame0 == SampleFrame1)
		{
			Math::LocationInterpolateCubicStart(OutLocation, OutLinearVelocity, InLocations[SampleFrame1], InLocations[SampleFrame2], InLocations[SampleFrame3], SampleAlpha, FrameDeltaTime);
			Math::RotationInterpolateCubicStart(OutRotation, OutAngularVelocity, InRotations[SampleFrame1], InRotations[SampleFrame2], InRotations[SampleFrame3], SampleAlpha, FrameDeltaTime);
			Math::ScaleInterpolateCubicStart(OutScale, OutScalarVelocity, InScales[SampleFrame1], InScales[SampleFrame2], InScales[SampleFrame3], SampleAlpha, FrameDeltaTime);
		}
		else if (SampleFrame2 == SampleFrame3)
		{
			Math::LocationInterpolateCubicEnd(OutLocation, OutLinearVelocity, InLocations[SampleFrame0], InLocations[SampleFrame1], InLocations[SampleFrame2], SampleAlpha, FrameDeltaTime);
			Math::RotationInterpolateCubicEnd(OutRotation, OutAngularVelocity, InRotations[SampleFrame0], InRotations[SampleFrame1], InRotations[SampleFrame2], SampleAlpha, FrameDeltaTime);
			Math::ScaleInterpolateCubicEnd(OutScale, OutScalarVelocity, InScales[SampleFrame0], InScales[SampleFrame1], InScales[SampleFrame2], SampleAlpha, FrameDeltaTime);
		}
		else
		{
			Math::LocationInterpolateCubic(OutLocation, OutLinearVelocity, InLocations[SampleFrame0], InLocations[SampleFrame1], InLocations[SampleFrame2], InLocations[SampleFrame3], SampleAlpha, FrameDeltaTime);
			Math::RotationInterpolateCubic(OutRotation, OutAngularVelocity, InRotations[SampleFrame0], InRotations[SampleFrame1], InRotations[SampleFrame2], InRotations[SampleFrame3], SampleAlpha, FrameDeltaTime);
			Math::ScaleInterpolateCubic(OutScale, OutScalarVelocity, InScales[SampleFrame0], InScales[SampleFrame1], InScales[SampleFrame2], InScales[SampleFrame3], SampleAlpha, FrameDeltaTime);
		}
	}

	void TransformSampleCubic(
		const TLearningArrayView<1, FVector3f> OutLocations,
		const TLearningArrayView<1, FQuat4f> OutRotations,
		const TLearningArrayView<1, FVector3f> OutScales,
		const TLearningArrayView<1, FVector3f> OutLinearVelocities,
		const TLearningArrayView<1, FVector3f> OutAngularVelocities,
		const TLearningArrayView<1, FVector3f> OutScalarVelocities,
		const TLearningArrayView<2, const FVector3f> InLocations,
		const TLearningArrayView<2, const FQuat4f> InRotations,
		const TLearningArrayView<2, const FVector3f> InScales,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha,
		const float FrameDeltaTime)
	{
		const int32 BoneNum = OutLocations.Num();

		if (SampleFrame0 == SampleFrame1 && SampleFrame2 == SampleFrame3)
		{
			for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
			{
				Math::LocationInterpolateLinear(OutLocations[BoneIdx], OutLinearVelocities[BoneIdx], InLocations[SampleFrame1][BoneIdx], InLocations[SampleFrame2][BoneIdx], SampleAlpha, FrameDeltaTime);
				Math::RotationInterpolateLinear(OutRotations[BoneIdx], OutAngularVelocities[BoneIdx], InRotations[SampleFrame1][BoneIdx], InRotations[SampleFrame2][BoneIdx], SampleAlpha, FrameDeltaTime);
				Math::ScaleInterpolateLinear(OutScales[BoneIdx], OutScalarVelocities[BoneIdx], InScales[SampleFrame1][BoneIdx], InScales[SampleFrame2][BoneIdx], SampleAlpha, FrameDeltaTime);
			}
		}
		else if (SampleFrame0 == SampleFrame1)
		{
			for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
			{
				Math::LocationInterpolateCubicStart(OutLocations[BoneIdx], OutLinearVelocities[BoneIdx], InLocations[SampleFrame1][BoneIdx], InLocations[SampleFrame2][BoneIdx], InLocations[SampleFrame3][BoneIdx], SampleAlpha, FrameDeltaTime);
				Math::RotationInterpolateCubicStart(OutRotations[BoneIdx], OutAngularVelocities[BoneIdx], InRotations[SampleFrame1][BoneIdx], InRotations[SampleFrame2][BoneIdx], InRotations[SampleFrame3][BoneIdx], SampleAlpha, FrameDeltaTime);
				Math::ScaleInterpolateCubicStart(OutScales[BoneIdx], OutScalarVelocities[BoneIdx], InScales[SampleFrame1][BoneIdx], InScales[SampleFrame2][BoneIdx], InScales[SampleFrame3][BoneIdx], SampleAlpha, FrameDeltaTime);
			}
		}
		else if (SampleFrame2 == SampleFrame3)
		{
			for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
			{
				Math::LocationInterpolateCubicEnd(OutLocations[BoneIdx], OutLinearVelocities[BoneIdx], InLocations[SampleFrame0][BoneIdx], InLocations[SampleFrame1][BoneIdx], InLocations[SampleFrame2][BoneIdx], SampleAlpha, FrameDeltaTime);
				Math::RotationInterpolateCubicEnd(OutRotations[BoneIdx], OutAngularVelocities[BoneIdx], InRotations[SampleFrame0][BoneIdx], InRotations[SampleFrame1][BoneIdx], InRotations[SampleFrame2][BoneIdx], SampleAlpha, FrameDeltaTime);
				Math::ScaleInterpolateCubicEnd(OutScales[BoneIdx], OutScalarVelocities[BoneIdx], InScales[SampleFrame0][BoneIdx], InScales[SampleFrame1][BoneIdx], InScales[SampleFrame2][BoneIdx], SampleAlpha, FrameDeltaTime);
			}
		}
		else
		{
			for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
			{
				Math::LocationInterpolateCubic(OutLocations[BoneIdx], OutLinearVelocities[BoneIdx], InLocations[SampleFrame0][BoneIdx], InLocations[SampleFrame1][BoneIdx], InLocations[SampleFrame2][BoneIdx], InLocations[SampleFrame3][BoneIdx], SampleAlpha, FrameDeltaTime);
				Math::RotationInterpolateCubic(OutRotations[BoneIdx], OutAngularVelocities[BoneIdx], InRotations[SampleFrame0][BoneIdx], InRotations[SampleFrame1][BoneIdx], InRotations[SampleFrame2][BoneIdx], InRotations[SampleFrame3][BoneIdx], SampleAlpha, FrameDeltaTime);
				Math::ScaleInterpolateCubic(OutScales[BoneIdx], OutScalarVelocities[BoneIdx], InScales[SampleFrame0][BoneIdx], InScales[SampleFrame1][BoneIdx], InScales[SampleFrame2][BoneIdx], InScales[SampleFrame3][BoneIdx], SampleAlpha, FrameDeltaTime);
			}
		}
	}

	void ArraySampleCubic(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<2, const float> InValues,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha)
	{
		if (SampleFrame0 == SampleFrame1 && SampleFrame2 == SampleFrame3)
		{
			ArrayInterpolateLinear(OutValues, InValues[SampleFrame1], InValues[SampleFrame2], SampleAlpha);
		}
		else if (SampleFrame0 == SampleFrame1)
		{
			ArrayInterpolateCubicStart(OutValues, InValues[SampleFrame1], InValues[SampleFrame2], InValues[SampleFrame3], SampleAlpha);
		}
		else if (SampleFrame2 == SampleFrame3)
		{
			ArrayInterpolateCubicEnd(OutValues, InValues[SampleFrame0], InValues[SampleFrame1], InValues[SampleFrame2], SampleAlpha);
		}
		else
		{
			ArrayInterpolateCubic(OutValues, InValues[SampleFrame0], InValues[SampleFrame1], InValues[SampleFrame2], InValues[SampleFrame3], SampleAlpha);
		}
	}

	void ArraySampleCubic(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<1, float> OutVelocities,
		const TLearningArrayView<2, const float> InValues,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha,
		const float FrameDeltaTime)
	{
		if (SampleFrame0 == SampleFrame1 && SampleFrame2 == SampleFrame3)
		{
			ArrayInterpolateLinear(OutValues, OutVelocities, InValues[SampleFrame1], InValues[SampleFrame2], SampleAlpha, FrameDeltaTime);
		}
		else if (SampleFrame0 == SampleFrame1)
		{
			ArrayInterpolateCubicStart(OutValues, OutVelocities, InValues[SampleFrame1], InValues[SampleFrame2], InValues[SampleFrame3], SampleAlpha, FrameDeltaTime);
		}
		else if (SampleFrame2 == SampleFrame3)
		{
			ArrayInterpolateCubicEnd(OutValues, OutVelocities, InValues[SampleFrame0], InValues[SampleFrame1], InValues[SampleFrame2], SampleAlpha, FrameDeltaTime);
		}
		else
		{
			ArrayInterpolateCubic(OutValues, OutVelocities, InValues[SampleFrame0], InValues[SampleFrame1], InValues[SampleFrame2], InValues[SampleFrame3], SampleAlpha, FrameDeltaTime);
		}
	}

	void ValueSampleCubicMono(
		float& OutValue,
		const TLearningArrayView<1, const float> InValues,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha)
	{
		if (SampleFrame0 == SampleFrame1 && SampleFrame2 == SampleFrame3)
		{
			Math::ValueInterpolateLinear(OutValue, InValues[SampleFrame1], InValues[SampleFrame2], SampleAlpha);
		}
		else if (SampleFrame0 == SampleFrame1)
		{
			Math::ValueInterpolateCubicMonoStart(OutValue, InValues[SampleFrame1], InValues[SampleFrame2], InValues[SampleFrame3], SampleAlpha);
		}
		else if (SampleFrame2 == SampleFrame3)
		{
			Math::ValueInterpolateCubicMonoEnd(OutValue, InValues[SampleFrame0], InValues[SampleFrame1], InValues[SampleFrame2], SampleAlpha);
		}
		else
		{
			Math::ValueInterpolateCubicMono(OutValue, InValues[SampleFrame0], InValues[SampleFrame1], InValues[SampleFrame2], InValues[SampleFrame3], SampleAlpha);
		}
	}

	void AngleSampleCubicMono(
		float& OutAngle,
		const TLearningArrayView<1, const float> InAngles,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha)
	{
		if (SampleFrame0 == SampleFrame1 && SampleFrame2 == SampleFrame3)
		{
			Math::AngleInterpolateLinear(OutAngle, InAngles[SampleFrame1], InAngles[SampleFrame2], SampleAlpha);
		}
		else if (SampleFrame0 == SampleFrame1)
		{
			Math::AngleInterpolateCubicMonoStart(OutAngle, InAngles[SampleFrame1], InAngles[SampleFrame2], InAngles[SampleFrame3], SampleAlpha);
		}
		else if (SampleFrame2 == SampleFrame3)
		{
			Math::AngleInterpolateCubicMonoEnd(OutAngle, InAngles[SampleFrame0], InAngles[SampleFrame1], InAngles[SampleFrame2], SampleAlpha);
		}
		else
		{
			Math::AngleInterpolateCubicMono(OutAngle, InAngles[SampleFrame0], InAngles[SampleFrame1], InAngles[SampleFrame2], InAngles[SampleFrame3], SampleAlpha);
		}
	}

	void LocationSampleCubicMono(
		FVector& OutLocation,
		const TLearningArrayView<1, const FVector> InLocations,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha)
	{
		if (SampleFrame0 == SampleFrame1 && SampleFrame2 == SampleFrame3)
		{
			Math::LocationInterpolateLinear(OutLocation, InLocations[SampleFrame1], InLocations[SampleFrame2], SampleAlpha);
		}
		else if (SampleFrame0 == SampleFrame1)
		{
			Math::LocationInterpolateCubicMonoStart(OutLocation, InLocations[SampleFrame1], InLocations[SampleFrame2], InLocations[SampleFrame3], SampleAlpha);
		}
		else if (SampleFrame2 == SampleFrame3)
		{
			Math::LocationInterpolateCubicMonoEnd(OutLocation, InLocations[SampleFrame0], InLocations[SampleFrame1], InLocations[SampleFrame2], SampleAlpha);
		}
		else
		{
			Math::LocationInterpolateCubicMono(OutLocation, InLocations[SampleFrame0], InLocations[SampleFrame1], InLocations[SampleFrame2], InLocations[SampleFrame3], SampleAlpha);
		}
	}

	void LocationSampleCubicMono(
		FVector& OutLocation,
		FVector3f& OutLinearVelocity,
		const TLearningArrayView<1, const FVector> InLocations,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha,
		const float FrameDeltaTime)
	{
		if (SampleFrame0 == SampleFrame1 && SampleFrame2 == SampleFrame3)
		{
			Math::LocationInterpolateLinear(OutLocation, OutLinearVelocity, InLocations[SampleFrame1], InLocations[SampleFrame2], SampleAlpha, FrameDeltaTime);
		}
		else if (SampleFrame0 == SampleFrame1)
		{
			Math::LocationInterpolateCubicMonoStart(OutLocation, OutLinearVelocity, InLocations[SampleFrame1], InLocations[SampleFrame2], InLocations[SampleFrame3], SampleAlpha, FrameDeltaTime);
		}
		else if (SampleFrame2 == SampleFrame3)
		{
			Math::LocationInterpolateCubicMonoEnd(OutLocation, OutLinearVelocity, InLocations[SampleFrame0], InLocations[SampleFrame1], InLocations[SampleFrame2], SampleAlpha, FrameDeltaTime);
		}
		else
		{
			Math::LocationInterpolateCubicMono(OutLocation, OutLinearVelocity, InLocations[SampleFrame0], InLocations[SampleFrame1], InLocations[SampleFrame2], InLocations[SampleFrame3], SampleAlpha, FrameDeltaTime);
		}
	}

	void RotationSampleCubicMono(
		FQuat4f& OutRotation,
		const TLearningArrayView<1, const FQuat4f> InRotations,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha)
	{
		if (SampleFrame0 == SampleFrame1 && SampleFrame2 == SampleFrame3)
		{
			Math::RotationInterpolateLinear(OutRotation, InRotations[SampleFrame1], InRotations[SampleFrame2], SampleAlpha);
		}
		else if (SampleFrame0 == SampleFrame1)
		{
			Math::RotationInterpolateCubicMonoStart(OutRotation, InRotations[SampleFrame1], InRotations[SampleFrame2], InRotations[SampleFrame3], SampleAlpha);
		}
		else if (SampleFrame2 == SampleFrame3)
		{
			Math::RotationInterpolateCubicMonoEnd(OutRotation, InRotations[SampleFrame0], InRotations[SampleFrame1], InRotations[SampleFrame2], SampleAlpha);
		}
		else
		{
			Math::RotationInterpolateCubicMono(OutRotation, InRotations[SampleFrame0], InRotations[SampleFrame1], InRotations[SampleFrame2], InRotations[SampleFrame3], SampleAlpha);
		}
	}

	void RotationSampleCubicMono(
		FQuat4f& OutRotation,
		FVector3f& OutAngularVelocity,
		const TLearningArrayView<1, const FQuat4f> InRotations,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha,
		const float FrameDeltaTime)
	{
		if (SampleFrame0 == SampleFrame1 && SampleFrame2 == SampleFrame3)
		{
			Math::RotationInterpolateLinear(OutRotation, OutAngularVelocity, InRotations[SampleFrame1], InRotations[SampleFrame2], SampleAlpha, FrameDeltaTime);
		}
		else if (SampleFrame0 == SampleFrame1)
		{
			Math::RotationInterpolateCubicMonoStart(OutRotation, OutAngularVelocity, InRotations[SampleFrame1], InRotations[SampleFrame2], InRotations[SampleFrame3], SampleAlpha, FrameDeltaTime);
		}
		else if (SampleFrame2 == SampleFrame3)
		{
			Math::RotationInterpolateCubicMonoEnd(OutRotation, OutAngularVelocity, InRotations[SampleFrame0], InRotations[SampleFrame1], InRotations[SampleFrame2], SampleAlpha, FrameDeltaTime);
		}
		else
		{
			Math::RotationInterpolateCubicMono(OutRotation, OutAngularVelocity, InRotations[SampleFrame0], InRotations[SampleFrame1], InRotations[SampleFrame2], InRotations[SampleFrame3], SampleAlpha, FrameDeltaTime);
		}
	}

	void TransformSampleCubicMono(
		FVector& OutLocation,
		FQuat4f& OutRotation,
		FVector3f& OutScale,
		const TLearningArrayView<1, const FVector> InLocations,
		const TLearningArrayView<1, const FQuat4f> InRotations,
		const TLearningArrayView<1, const FVector3f> InScales,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha)
	{
		if (SampleFrame0 == SampleFrame1 && SampleFrame2 == SampleFrame3)
		{
			Math::LocationInterpolateLinear(OutLocation, InLocations[SampleFrame1], InLocations[SampleFrame2], SampleAlpha);
			Math::RotationInterpolateLinear(OutRotation, InRotations[SampleFrame1], InRotations[SampleFrame2], SampleAlpha);
			Math::ScaleInterpolateLinear(OutScale, InScales[SampleFrame1], InScales[SampleFrame2], SampleAlpha);
		}
		else if (SampleFrame0 == SampleFrame1)
		{
			Math::LocationInterpolateCubicMonoStart(OutLocation, InLocations[SampleFrame1], InLocations[SampleFrame2], InLocations[SampleFrame3], SampleAlpha);
			Math::RotationInterpolateCubicMonoStart(OutRotation, InRotations[SampleFrame1], InRotations[SampleFrame2], InRotations[SampleFrame3], SampleAlpha);
			Math::ScaleInterpolateCubicMonoStart(OutScale, InScales[SampleFrame1], InScales[SampleFrame2], InScales[SampleFrame3], SampleAlpha);
		}
		else if (SampleFrame2 == SampleFrame3)
		{
			Math::LocationInterpolateCubicMonoEnd(OutLocation, InLocations[SampleFrame0], InLocations[SampleFrame1], InLocations[SampleFrame2], SampleAlpha);
			Math::RotationInterpolateCubicMonoEnd(OutRotation, InRotations[SampleFrame0], InRotations[SampleFrame1], InRotations[SampleFrame2], SampleAlpha);
			Math::ScaleInterpolateCubicMonoEnd(OutScale, InScales[SampleFrame0], InScales[SampleFrame1], InScales[SampleFrame2], SampleAlpha);
		}
		else
		{
			Math::LocationInterpolateCubicMono(OutLocation, InLocations[SampleFrame0], InLocations[SampleFrame1], InLocations[SampleFrame2], InLocations[SampleFrame3], SampleAlpha);
			Math::RotationInterpolateCubicMono(OutRotation, InRotations[SampleFrame0], InRotations[SampleFrame1], InRotations[SampleFrame2], InRotations[SampleFrame3], SampleAlpha);
			Math::ScaleInterpolateCubicMono(OutScale, InScales[SampleFrame0], InScales[SampleFrame1], InScales[SampleFrame2], InScales[SampleFrame3], SampleAlpha);
		}
	}

	void TransformSampleCubicMono(
		FVector& OutLocation,
		FQuat4f& OutRotation,
		FVector3f& OutScale,
		FVector3f& OutLinearVelocity,
		FVector3f& OutAngularVelocity,
		FVector3f& OutScalarVelocity,
		const TLearningArrayView<1, const FVector> InLocations,
		const TLearningArrayView<1, const FQuat4f> InRotations,
		const TLearningArrayView<1, const FVector3f> InScales,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha,
		const float FrameDeltaTime)
	{
		if (SampleFrame0 == SampleFrame1 && SampleFrame2 == SampleFrame3)
		{
			Math::LocationInterpolateLinear(OutLocation, OutLinearVelocity, InLocations[SampleFrame1], InLocations[SampleFrame2], SampleAlpha, FrameDeltaTime);
			Math::RotationInterpolateLinear(OutRotation, OutAngularVelocity, InRotations[SampleFrame1], InRotations[SampleFrame2], SampleAlpha, FrameDeltaTime);
			Math::ScaleInterpolateLinear(OutScale, OutScalarVelocity, InScales[SampleFrame1], InScales[SampleFrame2], SampleAlpha, FrameDeltaTime);
		}
		else if (SampleFrame0 == SampleFrame1)
		{
			Math::LocationInterpolateCubicMonoStart(OutLocation, OutLinearVelocity, InLocations[SampleFrame1], InLocations[SampleFrame2], InLocations[SampleFrame3], SampleAlpha, FrameDeltaTime);
			Math::RotationInterpolateCubicMonoStart(OutRotation, OutAngularVelocity, InRotations[SampleFrame1], InRotations[SampleFrame2], InRotations[SampleFrame3], SampleAlpha, FrameDeltaTime);
			Math::ScaleInterpolateCubicMonoStart(OutScale, OutScalarVelocity, InScales[SampleFrame1], InScales[SampleFrame2], InScales[SampleFrame3], SampleAlpha, FrameDeltaTime);
		}
		else if (SampleFrame2 == SampleFrame3)
		{
			Math::LocationInterpolateCubicMonoEnd(OutLocation, OutLinearVelocity, InLocations[SampleFrame0], InLocations[SampleFrame1], InLocations[SampleFrame2], SampleAlpha, FrameDeltaTime);
			Math::RotationInterpolateCubicMonoEnd(OutRotation, OutAngularVelocity, InRotations[SampleFrame0], InRotations[SampleFrame1], InRotations[SampleFrame2], SampleAlpha, FrameDeltaTime);
			Math::ScaleInterpolateCubicMonoEnd(OutScale, OutScalarVelocity, InScales[SampleFrame0], InScales[SampleFrame1], InScales[SampleFrame2], SampleAlpha, FrameDeltaTime);
		}
		else
		{
			Math::LocationInterpolateCubicMono(OutLocation, OutLinearVelocity, InLocations[SampleFrame0], InLocations[SampleFrame1], InLocations[SampleFrame2], InLocations[SampleFrame3], SampleAlpha, FrameDeltaTime);
			Math::RotationInterpolateCubicMono(OutRotation, OutAngularVelocity, InRotations[SampleFrame0], InRotations[SampleFrame1], InRotations[SampleFrame2], InRotations[SampleFrame3], SampleAlpha, FrameDeltaTime);
			Math::ScaleInterpolateCubicMono(OutScale, OutScalarVelocity, InScales[SampleFrame0], InScales[SampleFrame1], InScales[SampleFrame2], InScales[SampleFrame3], SampleAlpha, FrameDeltaTime);
		}
	}

	void TransformSampleCubicMono(
		const TLearningArrayView<1, FVector3f> OutLocations,
		const TLearningArrayView<1, FQuat4f> OutRotations,
		const TLearningArrayView<1, FVector3f> OutScales,
		const TLearningArrayView<1, FVector3f> OutLinearVelocities,
		const TLearningArrayView<1, FVector3f> OutAngularVelocities,
		const TLearningArrayView<1, FVector3f> OutScalarVelocities,
		const TLearningArrayView<2, const FVector3f> InLocations,
		const TLearningArrayView<2, const FQuat4f> InRotations,
		const TLearningArrayView<2, const FVector3f> InScales,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha,
		const float FrameDeltaTime)
	{
		const int32 BoneNum = OutLocations.Num();

		if (SampleFrame0 == SampleFrame1 && SampleFrame2 == SampleFrame3)
		{
			for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
			{
				Math::LocationInterpolateLinear(OutLocations[BoneIdx], OutLinearVelocities[BoneIdx], InLocations[SampleFrame1][BoneIdx], InLocations[SampleFrame2][BoneIdx], SampleAlpha, FrameDeltaTime);
				Math::RotationInterpolateLinear(OutRotations[BoneIdx], OutAngularVelocities[BoneIdx], InRotations[SampleFrame1][BoneIdx], InRotations[SampleFrame2][BoneIdx], SampleAlpha, FrameDeltaTime);
				Math::ScaleInterpolateLinear(OutScales[BoneIdx], OutScalarVelocities[BoneIdx], InScales[SampleFrame1][BoneIdx], InScales[SampleFrame2][BoneIdx], SampleAlpha, FrameDeltaTime);
			}
		}
		else if (SampleFrame0 == SampleFrame1)
		{
			for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
			{
				Math::LocationInterpolateCubicMonoStart(OutLocations[BoneIdx], OutLinearVelocities[BoneIdx], InLocations[SampleFrame1][BoneIdx], InLocations[SampleFrame2][BoneIdx], InLocations[SampleFrame3][BoneIdx], SampleAlpha, FrameDeltaTime);
				Math::RotationInterpolateCubicMonoStart(OutRotations[BoneIdx], OutAngularVelocities[BoneIdx], InRotations[SampleFrame1][BoneIdx], InRotations[SampleFrame2][BoneIdx], InRotations[SampleFrame3][BoneIdx], SampleAlpha, FrameDeltaTime);
				Math::ScaleInterpolateCubicMonoStart(OutScales[BoneIdx], OutScalarVelocities[BoneIdx], InScales[SampleFrame1][BoneIdx], InScales[SampleFrame2][BoneIdx], InScales[SampleFrame3][BoneIdx], SampleAlpha, FrameDeltaTime);
			}
		}
		else if (SampleFrame2 == SampleFrame3)
		{
			for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
			{
				Math::LocationInterpolateCubicMonoEnd(OutLocations[BoneIdx], OutLinearVelocities[BoneIdx], InLocations[SampleFrame0][BoneIdx], InLocations[SampleFrame1][BoneIdx], InLocations[SampleFrame2][BoneIdx], SampleAlpha, FrameDeltaTime);
				Math::RotationInterpolateCubicMonoEnd(OutRotations[BoneIdx], OutAngularVelocities[BoneIdx], InRotations[SampleFrame0][BoneIdx], InRotations[SampleFrame1][BoneIdx], InRotations[SampleFrame2][BoneIdx], SampleAlpha, FrameDeltaTime);
				Math::ScaleInterpolateCubicMonoEnd(OutScales[BoneIdx], OutScalarVelocities[BoneIdx], InScales[SampleFrame0][BoneIdx], InScales[SampleFrame1][BoneIdx], InScales[SampleFrame2][BoneIdx], SampleAlpha, FrameDeltaTime);
			}
		}
		else
		{
			for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
			{
				Math::LocationInterpolateCubicMono(OutLocations[BoneIdx], OutLinearVelocities[BoneIdx], InLocations[SampleFrame0][BoneIdx], InLocations[SampleFrame1][BoneIdx], InLocations[SampleFrame2][BoneIdx], InLocations[SampleFrame3][BoneIdx], SampleAlpha, FrameDeltaTime);
				Math::RotationInterpolateCubicMono(OutRotations[BoneIdx], OutAngularVelocities[BoneIdx], InRotations[SampleFrame0][BoneIdx], InRotations[SampleFrame1][BoneIdx], InRotations[SampleFrame2][BoneIdx], InRotations[SampleFrame3][BoneIdx], SampleAlpha, FrameDeltaTime);
				Math::ScaleInterpolateCubicMono(OutScales[BoneIdx], OutScalarVelocities[BoneIdx], InScales[SampleFrame0][BoneIdx], InScales[SampleFrame1][BoneIdx], InScales[SampleFrame2][BoneIdx], InScales[SampleFrame3][BoneIdx], SampleAlpha, FrameDeltaTime);
			}
		}
	}

	void ArraySampleCubicMono(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<2, const float> InValues,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha)
	{
		if (SampleFrame0 == SampleFrame1 && SampleFrame2 == SampleFrame3)
		{
			ArrayInterpolateLinear(OutValues, InValues[SampleFrame1], InValues[SampleFrame2], SampleAlpha);
		}
		else if (SampleFrame0 == SampleFrame1)
		{
			ArrayInterpolateCubicMonoStart(OutValues, InValues[SampleFrame1], InValues[SampleFrame2], InValues[SampleFrame3], SampleAlpha);
		}
		else if (SampleFrame2 == SampleFrame3)
		{
			ArrayInterpolateCubicMonoEnd(OutValues, InValues[SampleFrame0], InValues[SampleFrame1], InValues[SampleFrame2], SampleAlpha);
		}
		else
		{
			ArrayInterpolateCubicMono(OutValues, InValues[SampleFrame0], InValues[SampleFrame1], InValues[SampleFrame2], InValues[SampleFrame3], SampleAlpha);
		}
	}

	void ArraySampleCubicMono(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<1, float> OutVelocities,
		const TLearningArrayView<2, const float> InValues,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha,
		const float FrameDeltaTime)
	{
		if (SampleFrame0 == SampleFrame1 && SampleFrame2 == SampleFrame3)
		{
			ArrayInterpolateLinear(OutValues, OutVelocities, InValues[SampleFrame1], InValues[SampleFrame2], SampleAlpha, FrameDeltaTime);
		}
		else if (SampleFrame0 == SampleFrame1)
		{
			ArrayInterpolateCubicMonoStart(OutValues, OutVelocities, InValues[SampleFrame1], InValues[SampleFrame2], InValues[SampleFrame3], SampleAlpha, FrameDeltaTime);
		}
		else if (SampleFrame2 == SampleFrame3)
		{
			ArrayInterpolateCubicMonoEnd(OutValues, OutVelocities, InValues[SampleFrame0], InValues[SampleFrame1], InValues[SampleFrame2], SampleAlpha, FrameDeltaTime);
		}
		else
		{
			ArrayInterpolateCubicMono(OutValues, OutVelocities, InValues[SampleFrame0], InValues[SampleFrame1], InValues[SampleFrame2], InValues[SampleFrame3], SampleAlpha, FrameDeltaTime);
		}
	}

	void NearestSample(
		float& OutSampledValue,
		const TLearningArrayView<1, const float> InValues,
		const float Time,
		const float FrameDeltaTime)
	{
		const int64 SampleFrame = ComputeNearestSampleFrame(Time / FMath::Max(FrameDeltaTime, UE_SMALL_NUMBER), InValues.Num<0>());
		OutSampledValue = InValues[SampleFrame];
	}

	void NearestSampleArray(
		const TLearningArrayView<1, float> OutSampledVector,
		const TLearningArrayView<2, const float> InVectors,
		const float Time,
		const float FrameDeltaTime)
	{
		const int64 SampleFrame = ComputeNearestSampleFrame(Time / FMath::Max(FrameDeltaTime, UE_SMALL_NUMBER), InVectors.Num<0>());
		Learning::Array::Copy(OutSampledVector, InVectors[SampleFrame]);
	}

	void LinearSample(
		float& OutSampledValue,
		const TLearningArrayView<1, const float> InValues,
		const float Time,
		const float FrameDeltaTime)
	{
		const int64 FrameNum = InValues.Num<0>();

		int64 SampleFrame0, SampleFrame1;
		float SampleAlpha;
		ComputeLinearSampleFramesAndAlpha(
			SampleFrame0,
			SampleFrame1,
			SampleAlpha,
			Time / FMath::Max(FrameDeltaTime, UE_SMALL_NUMBER),
			FrameNum);

		ValueInterpolateLinear(
			OutSampledValue,
			InValues[SampleFrame0],
			InValues[SampleFrame1],
			SampleAlpha);
	}

	void LinearSampleArray(
		const TLearningArrayView<1, float> OutSampledVector,
		const TLearningArrayView<2, const float> InVectors,
		const float Time,
		const float FrameDeltaTime)
	{
		const int64 FrameNum = InVectors.Num<0>();

		int64 SampleFrame0, SampleFrame1;
		float SampleAlpha;
		ComputeLinearSampleFramesAndAlpha(
			SampleFrame0,
			SampleFrame1,
			SampleAlpha,
			Time / FMath::Max(FrameDeltaTime, UE_SMALL_NUMBER),
			FrameNum);

		ArrayInterpolateLinear(
			OutSampledVector,
			InVectors[SampleFrame0],
			InVectors[SampleFrame1],
			SampleAlpha);
	}

	void CubicSample(
		float& OutSampledValue,
		const TLearningArrayView<1, const float> InValues,
		const float Time,
		const float FrameDeltaTime)
	{
		const int64 FrameNum = InValues.Num<0>();

		int64 SampleFrame0, SampleFrame1, SampleFrame2, SampleFrame3;
		float SampleAlpha;
		ComputeCubicSampleFramesAndAlpha(
			SampleFrame0,
			SampleFrame1,
			SampleFrame2,
			SampleFrame3,
			SampleAlpha,
			Time / FMath::Max(FrameDeltaTime, UE_SMALL_NUMBER),
			FrameNum);

		if (SampleFrame0 == SampleFrame1 && SampleFrame2 == SampleFrame3)
		{
			ValueInterpolateLinear(
				OutSampledValue,
				InValues[SampleFrame1],
				InValues[SampleFrame2],
				SampleAlpha);
		}
		else if (SampleFrame0 == SampleFrame1)
		{
			ValueInterpolateCubicStart(
				OutSampledValue,
				InValues[SampleFrame1],
				InValues[SampleFrame2],
				InValues[SampleFrame3],
				SampleAlpha);
		}
		else if (SampleFrame2 == SampleFrame3)
		{
			ValueInterpolateCubicEnd(
				OutSampledValue,
				InValues[SampleFrame0],
				InValues[SampleFrame1],
				InValues[SampleFrame2],
				SampleAlpha);
		}
		else
		{
			ValueInterpolateCubic(
				OutSampledValue,
				InValues[SampleFrame0],
				InValues[SampleFrame1],
				InValues[SampleFrame2],
				InValues[SampleFrame3],
				SampleAlpha);
		}
	}

	void CubicSampleArray(
		const TLearningArrayView<1, float> OutSampledVector,
		const TLearningArrayView<2, const float> InVectors,
		const float Time,
		const float FrameDeltaTime)
	{
		const int64 FrameNum = InVectors.Num<0>();

		int64 SampleFrame0, SampleFrame1, SampleFrame2, SampleFrame3;
		float SampleAlpha;
		ComputeCubicSampleFramesAndAlpha(
			SampleFrame0,
			SampleFrame1,
			SampleFrame2,
			SampleFrame3,
			SampleAlpha,
			Time / FMath::Max(FrameDeltaTime, UE_SMALL_NUMBER),
			FrameNum);

		if (SampleFrame0 == SampleFrame1 && SampleFrame2 == SampleFrame3)
		{
			ArrayInterpolateLinear(
				OutSampledVector,
				InVectors[SampleFrame1],
				InVectors[SampleFrame2],
				SampleAlpha);
		}
		else if (SampleFrame0 == SampleFrame1)
		{
			ArrayInterpolateCubicStart(
				OutSampledVector,
				InVectors[SampleFrame1],
				InVectors[SampleFrame2],
				InVectors[SampleFrame3],
				SampleAlpha);
		}
		else if (SampleFrame2 == SampleFrame3)
		{
			ArrayInterpolateCubicEnd(
				OutSampledVector,
				InVectors[SampleFrame0],
				InVectors[SampleFrame1],
				InVectors[SampleFrame2],
				SampleAlpha);
		}
		else
		{
			ArrayInterpolateCubic(
				OutSampledVector,
				InVectors[SampleFrame0],
				InVectors[SampleFrame1],
				InVectors[SampleFrame2],
				InVectors[SampleFrame3],
				SampleAlpha);
		}
	}

	void CubicMonoSample(
		float& OutSampledValue,
		const TLearningArrayView<1, const float> InValues,
		const float Time,
		const float FrameDeltaTime)
	{
		const int64 FrameNum = InValues.Num<0>();

		int64 SampleFrame0, SampleFrame1, SampleFrame2, SampleFrame3;
		float SampleAlpha;
		ComputeCubicSampleFramesAndAlpha(
			SampleFrame0,
			SampleFrame1,
			SampleFrame2,
			SampleFrame3,
			SampleAlpha,
			Time / FMath::Max(FrameDeltaTime, UE_SMALL_NUMBER),
			FrameNum);

		ValueSampleCubicMono(
			OutSampledValue, 
			InValues, 
			SampleFrame0, 
			SampleFrame1, 
			SampleFrame2, 
			SampleFrame3, 
			SampleAlpha);
	}

	void CubicMonoSampleLocation(
		FVector& OutSampledLocation,
		FVector3f& OutSampledVelocity,
		const TLearningArrayView<1, const FVector> InLocations,
		const float Time,
		const float FrameDeltaTime)
	{
		const int64 FrameNum = InLocations.Num<0>();

		int64 SampleFrame0, SampleFrame1, SampleFrame2, SampleFrame3;
		float SampleAlpha;
		ComputeCubicSampleFramesAndAlpha(
			SampleFrame0,
			SampleFrame1,
			SampleFrame2,
			SampleFrame3,
			SampleAlpha,
			Time / FMath::Max(FrameDeltaTime, UE_SMALL_NUMBER),
			FrameNum);

		LocationSampleCubicMono(
			OutSampledLocation,
			OutSampledVelocity,
			InLocations,
			SampleFrame0,
			SampleFrame1,
			SampleFrame2,
			SampleFrame3,
			SampleAlpha,
			FrameDeltaTime);
	}

	void CubicMonoSampleLocation(
		FVector& OutSampledValue,
		const TLearningArrayView<1, const FVector> InValues,
		const float Time,
		const float FrameDeltaTime)
	{
		const int64 FrameNum = InValues.Num<0>();

		int64 SampleFrame0, SampleFrame1, SampleFrame2, SampleFrame3;
		float SampleAlpha;
		ComputeCubicSampleFramesAndAlpha(
			SampleFrame0,
			SampleFrame1,
			SampleFrame2,
			SampleFrame3,
			SampleAlpha,
			Time / FMath::Max(FrameDeltaTime, UE_SMALL_NUMBER),
			FrameNum);

		LocationSampleCubicMono(
			OutSampledValue,
			InValues,
			SampleFrame0,
			SampleFrame1,
			SampleFrame2,
			SampleFrame3,
			SampleAlpha);
	}

	void CubicMonoSampleArray(
		const TLearningArrayView<1, float> OutSampledVector,
		const TLearningArrayView<2, const float> InVectors,
		const float Time,
		const float FrameDeltaTime)
	{
		const int64 FrameNum = InVectors.Num<0>();

		int64 SampleFrame0, SampleFrame1, SampleFrame2, SampleFrame3;
		float SampleAlpha;
		ComputeCubicSampleFramesAndAlpha(
			SampleFrame0,
			SampleFrame1,
			SampleFrame2,
			SampleFrame3,
			SampleAlpha,
			Time / FMath::Max(FrameDeltaTime, UE_SMALL_NUMBER),
			FrameNum);

		ArraySampleCubicMono(
			OutSampledVector,
			InVectors,
			SampleFrame0,
			SampleFrame1,
			SampleFrame2,
			SampleFrame3,
			SampleAlpha);
	}

	// Mean/Std/Scale

	float Sum(const TLearningArrayView<1, const float> Data)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::AnimDatabase::Math::Sum);

		const int64 Num = Data.Num();

		float Total = 0.0f;
		for (int64 Idx = 0; Idx < Num; Idx++)
		{
			Total += Data[Idx];
		}
		return Total;
	}

	void ComputeMean(float& OutMean, const TLearningArrayView<1, const float> Data)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::AnimDatabase::Math::ComputeMean);

		const int64 Num = Data.Num();

		OutMean = 0.0f;
		for (int64 Idx = 0; Idx < Num; Idx++)
		{
			OutMean += (Data[Idx] - OutMean) / (Idx + 1);
		}
	}

	void ComputeMeanStd(float& OutMean, float& OutStd, const TLearningArrayView<1, const float> Data)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::AnimDatabase::Math::ComputeMeanStd);

		const int64 Num = Data.Num();

		OutMean = 0.0f;
		OutStd = 0.0f;
		for (int64 Idx = 0; Idx < Num; Idx++)
		{
			OutStd += (((float)Idx / Num) / (Idx + 1)) * FMath::Square(Data[Idx] - OutMean);
			OutMean += (Data[Idx] - OutMean) / (Idx + 1);
		}

		OutStd = FMath::Sqrt(OutStd);
	}

	void ComputeMean(
		const TLearningArrayView<1, float> OutMean,
		const TLearningArrayView<2, const float> Data)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::AnimDatabase::Math::ComputeMean);

		const int64 SampleNum = Data.Num<0>();
		const int64 DimNum = Data.Num<1>();

		check(OutMean.Num() == DimNum);

		if (UE_ANIMDATABASE_ISPC && SampleNum * DimNum * sizeof(float) < MAX_int32)
		{
#if UE_ANIMDATABASE_ISPC
			ispc::AnimDatabaseComputeMean(
				OutMean.GetData(),
				Data.GetData(),
				SampleNum,
				DimNum);
#endif
		}
		else
		{
			Learning::Array::Zero(OutMean);

			for (int64 SampleIdx = 0; SampleIdx < SampleNum; SampleIdx++)
			{
				for (int64 DimIdx = 0; DimIdx < DimNum; DimIdx++)
				{
					OutMean[DimIdx] += (Data[SampleIdx][DimIdx] - OutMean[DimIdx]) / (SampleIdx + 1);
				}
			}
		}
	}

	void ComputeMeanStd(
		const TLearningArrayView<1, float> OutMean,
		const TLearningArrayView<1, float> OutStd,
		const TLearningArrayView<2, const float> Data)
	{
		ComputeMeanStd(OutMean, OutStd, Data, 0, Data.Num<1>());
	}

	void ComputeMeanStd(
		const TLearningArrayView<1, float> OutMean,
		const TLearningArrayView<1, float> OutStd,
		const TLearningArrayView<2, const float> Data,
		const int32 ColSliceStart,
		const int32 ColSliceNum)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::AnimDatabase::Math::ComputeMeanStd);

		const int64 SampleNum = Data.Num<0>();
		const int64 DimNum = Data.Num<1>();

		check(OutMean.Num() == DimNum);
		check(OutStd.Num() == DimNum);
		check(ColSliceStart >= 0 && ColSliceNum >= 0 && ColSliceStart + ColSliceNum <= DimNum);

		if (UE_ANIMDATABASE_ISPC && SampleNum * DimNum * sizeof(float) < MAX_int32)
		{
#if UE_ANIMDATABASE_ISPC
			ispc::AnimDatabaseComputeMeanStd(
				OutMean.GetData(),
				OutStd.GetData(),
				Data.GetData(),
				SampleNum,
				DimNum,
				ColSliceStart,
				ColSliceNum);
#endif
		}
		else
		{
			Learning::Array::Zero(OutMean.Slice(ColSliceStart, ColSliceNum));
			Learning::Array::Zero(OutStd.Slice(ColSliceStart, ColSliceNum));

			for (int64 SampleIdx = 0; SampleIdx < SampleNum; SampleIdx++)
			{
				for (int64 DimIdx = ColSliceStart; DimIdx < ColSliceStart + ColSliceNum; DimIdx++)
				{
					OutStd[DimIdx] += (((float)SampleIdx / SampleNum) / (SampleIdx + 1)) * FMath::Square(Data[SampleIdx][DimIdx] - OutMean[DimIdx]);
					OutMean[DimIdx] += (Data[SampleIdx][DimIdx] - OutMean[DimIdx]) / (SampleIdx + 1);
				}
			}

			for (int64 DimIdx = ColSliceStart; DimIdx < ColSliceStart + ColSliceNum; DimIdx++)
			{
				OutStd[DimIdx] = FMath::Sqrt(OutStd[DimIdx]);
			}
		}
	}

	void ComputeMeanStdMasked(
		const TLearningArrayView<1, float> OutMean,
		const TLearningArrayView<1, float> OutStd,
		const TLearningArrayView<2, const float> Data,
		const TLearningArrayView<1, const bool> Mask,
		const int32 ColSliceStart,
		const int32 ColSliceNum)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::AnimDatabase::Math::ComputeMeanStdMasked);

		const int64 SampleNum = Data.Num<0>();
		const int64 DimNum = Data.Num<1>();

		check(OutMean.Num() == DimNum);
		check(OutStd.Num() == DimNum);
		check(ColSliceStart >= 0 && ColSliceNum >= 0 && ColSliceStart + ColSliceNum <= DimNum);
		check(Mask.Num() == SampleNum);

		Learning::Array::Zero(OutMean.Slice(ColSliceStart, ColSliceNum));
		Learning::Array::Zero(OutStd.Slice(ColSliceStart, ColSliceNum));

		int64 MaskedNum = 0;
		for (int64 SampleIdx = 0; SampleIdx < SampleNum; SampleIdx++)
		{
			MaskedNum += Mask[SampleIdx] ? 1 : 0;
		}

		if (MaskedNum == 0) { return; }

		int64 MaskedIdx = 0;
		for (int64 SampleIdx = 0; SampleIdx < SampleNum; SampleIdx++)
		{
			if (Mask[SampleIdx])
			{
				for (int64 DimIdx = ColSliceStart; DimIdx < ColSliceStart + ColSliceNum; DimIdx++)
				{
					OutStd[DimIdx] += (((float)MaskedIdx / MaskedNum) / (MaskedIdx + 1)) * FMath::Square(Data[SampleIdx][DimIdx] - OutMean[DimIdx]);
					OutMean[DimIdx] += (Data[SampleIdx][DimIdx] - OutMean[DimIdx]) / (MaskedIdx + 1);
				}
				MaskedIdx++;
			}
		}

		check(MaskedIdx == MaskedNum);

		for (int64 DimIdx = ColSliceStart; DimIdx < ColSliceStart + ColSliceNum; DimIdx++)
		{
			OutStd[DimIdx] = FMath::Sqrt(OutStd[DimIdx]);
		}
	}

	void ComputeMeanStd(
		FVector3f& OutMean,
		FVector3f& OutStd,
		const TLearningArrayView<1, const FVector3f> Data)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::AnimDatabase::Math::ComputeMeanStd);

		const int64 Num = Data.Num();

		if (UE_ANIMDATABASE_ISPC && Data.NumBytes() < MAX_int32)
		{
#if UE_ANIMDATABASE_ISPC
			ispc::AnimDatabaseComputeMeanStd(
				(float*)&OutMean,
				(float*)&OutStd,
				(const float*)Data.GetData(),
				Num,
				3,
				0, 
				3);
#endif
		}
		else
		{
			OutMean = FVector3f::ZeroVector;
			OutStd = FVector3f::ZeroVector;
			for (int64 Idx = 0; Idx < Num; Idx++)
			{
				OutStd += (((float)Idx / Num) / (Idx + 1)) * FMath::Square(Data[Idx] - OutMean);
				OutMean += (Data[Idx] - OutMean) / (Idx + 1);
			}

			OutStd = VectorSqrt(OutStd);
		}
	}

	void ComputeLocalMeanStd(
		FVector3f& OutMean,
		FVector3f& OutStd,
		const TLearningArrayView<1, const FVector3f> Data,
		const TLearningArrayView<1, const FQuat4f> Reference)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::AnimDatabase::Math::ComputeLocalMeanStd);

		const int64 Num = Data.Num();

		OutMean = FVector3f::ZeroVector;
		OutStd = FVector3f::ZeroVector;
		for (int64 Idx = 0; Idx < Num; Idx++)
		{
			OutStd += (((float)Idx / Num) / (Idx + 1)) * FMath::Square(Reference[Idx].UnrotateVector(Data[Idx]) - OutMean);
			OutMean += (Reference[Idx].UnrotateVector(Data[Idx]) - OutMean) / (Idx + 1);
		}

		OutStd = VectorSqrt(OutStd);
	}

	void ComputeMeanStdOfLog(
		FVector3f& OutMeanLog,
		FVector3f& OutStdLog,
		const TLearningArrayView<1, const FVector3f> Data)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::AnimDatabase::Math::ComputeMeanStdOfLog);

		const int64 Num = Data.Num();

		if (UE_ANIMDATABASE_ISPC && Data.NumBytes() < MAX_int32)
		{
#if UE_ANIMDATABASE_ISPC
			ispc::AnimDatabaseComputeMeanStdOfLog(
				(float*)&OutMeanLog,
				(float*)&OutStdLog,
				(const float*)Data.GetData(),
				Num,
				3);
#endif
		}
		else
		{
			OutMeanLog = FVector3f::ZeroVector;
			OutStdLog = FVector3f::ZeroVector;
			for (int64 Idx = 0; Idx < Num; Idx++)
			{
				OutStdLog += (((float)Idx / Num) / (Idx + 1)) * FMath::Square(VectorLogSafe(Data[Idx]) - OutMeanLog);
				OutMeanLog += (VectorLogSafe(Data[Idx]) - OutMeanLog) / (Idx + 1);
			}

			OutMeanLog = VectorExp(OutMeanLog);
			OutStdLog = VectorSqrt(OutStdLog);
		}
	}

	void ComputeMeanStd(
		const TLearningArrayView<1, FVector3f> OutMeans,
		const TLearningArrayView<1, FVector3f> OutStds,
		const TLearningArrayView<2, const FVector3f> Vectors)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::AnimDatabase::Math::ComputeMeanStd);

		const int64 RowNum = Vectors.Num<0>();
		const int64 ColNum = Vectors.Num<1>();
		check(RowNum > 0);

		if (UE_ANIMDATABASE_ISPC && Vectors.NumBytes() < MAX_int32)
		{
#if UE_ANIMDATABASE_ISPC
			ispc::AnimDatabaseComputeMeanStd(
				(float*)OutMeans.GetData(),
				(float*)OutStds.GetData(),
				(const float*)Vectors.GetData(),
				RowNum,
				ColNum * 3,
				0,
				ColNum * 3);
#endif
		}
		else
		{
			Learning::Array::Zero(OutMeans);
			Learning::Array::Zero(OutStds);

			for (int64 RowIdx = 0; RowIdx < RowNum; RowIdx++)
			{
				for (int64 ColIdx = 0; ColIdx < ColNum; ColIdx++)
				{
					OutStds[ColIdx] += (((float)RowIdx / RowNum) / (RowIdx + 1)) * FMath::Square(Vectors[RowIdx][ColIdx] - OutMeans[ColIdx]);
				}

				for (int64 ColIdx = 0; ColIdx < ColNum; ColIdx++)
				{
					OutMeans[ColIdx] += (Vectors[RowIdx][ColIdx] - OutMeans[ColIdx]) / (RowIdx + 1);
				}
			}

			for (int64 ColIdx = 0; ColIdx < ColNum; ColIdx++)
			{
				OutStds[ColIdx] = VectorSqrt(OutStds[ColIdx]);
			}
		}
	}

	void ComputeMeanStdOfLog(
		const TLearningArrayView<1, FVector3f> OutMeanLogs,
		const TLearningArrayView<1, FVector3f> OutStdLogs,
		const TLearningArrayView<2, const FVector3f> Vectors)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::AnimDatabase::Math::ComputeMeanStdOfLog);

		const int64 RowNum = Vectors.Num<0>();
		const int64 ColNum = Vectors.Num<1>();
		check(RowNum > 0);

		if (UE_ANIMDATABASE_ISPC && Vectors.NumBytes() < MAX_int32)
		{
#if UE_ANIMDATABASE_ISPC
			ispc::AnimDatabaseComputeMeanStdOfLog(
				(float*)OutMeanLogs.GetData(),
				(float*)OutStdLogs.GetData(),
				(const float*)Vectors.GetData(),
				RowNum,
				ColNum * 3);
#endif
		}
		else
		{
			Learning::Array::Zero(OutMeanLogs);
			Learning::Array::Zero(OutStdLogs);

			for (int64 RowIdx = 0; RowIdx < RowNum; RowIdx++)
			{
				for (int64 ColIdx = 0; ColIdx < ColNum; ColIdx++)
				{
					OutStdLogs[ColIdx] += (((float)RowIdx / RowNum) / (RowIdx + 1)) * FMath::Square(VectorLogSafe(Vectors[RowIdx][ColIdx]) - OutMeanLogs[ColIdx]);
				}

				for (int64 ColIdx = 0; ColIdx < ColNum; ColIdx++)
				{
					OutMeanLogs[ColIdx] += (VectorLogSafe(Vectors[RowIdx][ColIdx]) - OutMeanLogs[ColIdx]) / (RowIdx + 1);
				}
			}

			for (int64 ColIdx = 0; ColIdx < ColNum; ColIdx++)
			{
				OutMeanLogs[ColIdx] = VectorExp(OutMeanLogs[ColIdx]);
				OutStdLogs[ColIdx] = VectorSqrt(OutStdLogs[ColIdx]);
			}
		}
	}

	FVector4f DominantEigenVector(
		const FMatrix44f& A,
		const FVector4f V0)
	{
		// Initial Guess at Eigen Vector
		FVector4f V = V0;

		for (int32 Iteration = 0; Iteration < 20; Iteration++)
		{
			// Power Iteration
			const FVector4f Av = A.TransformFVector4(V);

			// Next Guess at Eigen Vector
			V = VectorNormalize(Av);
		}

		return V;
	}

	void ComputeMeanStd(
		const TLearningArrayView<1, FQuat4f> OutMeans,
		const TLearningArrayView<1, FMatrix44f> OutAccum,
		const TLearningArrayView<1, FVector3f> OutStds,
		const TLearningArrayView<2, const FQuat4f> Rotations)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::AnimDatabase::Math::ComputeMeanStd);

		const int64 RowNum = Rotations.Num<0>();
		const int64 ColNum = Rotations.Num<1>();
		check(RowNum > 0);

		Learning::Array::Zero(OutAccum);
		for (int64 RowIdx = 0; RowIdx < RowNum; RowIdx++)
		{
			for (int64 ColIdx = 0; ColIdx < ColNum; ColIdx++)
			{
				const FQuat4f Q = Rotations[RowIdx][ColIdx];
				
				// make Q^T * Q matrix and accumulate using online mean algorithm :

				OutAccum[ColIdx].M[0][0] += ((Q.X * Q.X) - OutAccum[ColIdx].M[0][0]) / (RowIdx + 1);
				OutAccum[ColIdx].M[0][1] += ((Q.X * Q.Y) - OutAccum[ColIdx].M[0][1]) / (RowIdx + 1);
				OutAccum[ColIdx].M[0][2] += ((Q.X * Q.Z) - OutAccum[ColIdx].M[0][2]) / (RowIdx + 1);
				OutAccum[ColIdx].M[0][3] += ((Q.X * Q.W) - OutAccum[ColIdx].M[0][3]) / (RowIdx + 1);

				OutAccum[ColIdx].M[1][0] += ((Q.Y * Q.X) - OutAccum[ColIdx].M[1][0]) / (RowIdx + 1);
				OutAccum[ColIdx].M[1][1] += ((Q.Y * Q.Y) - OutAccum[ColIdx].M[1][1]) / (RowIdx + 1);
				OutAccum[ColIdx].M[1][2] += ((Q.Y * Q.Z) - OutAccum[ColIdx].M[1][2]) / (RowIdx + 1);
				OutAccum[ColIdx].M[1][3] += ((Q.Y * Q.W) - OutAccum[ColIdx].M[1][3]) / (RowIdx + 1);

				OutAccum[ColIdx].M[2][0] += ((Q.Z * Q.X) - OutAccum[ColIdx].M[2][0]) / (RowIdx + 1);
				OutAccum[ColIdx].M[2][1] += ((Q.Z * Q.Y) - OutAccum[ColIdx].M[2][1]) / (RowIdx + 1);
				OutAccum[ColIdx].M[2][2] += ((Q.Z * Q.Z) - OutAccum[ColIdx].M[2][2]) / (RowIdx + 1);
				OutAccum[ColIdx].M[2][3] += ((Q.Z * Q.W) - OutAccum[ColIdx].M[2][3]) / (RowIdx + 1);

				OutAccum[ColIdx].M[3][0] += ((Q.W * Q.X) - OutAccum[ColIdx].M[3][0]) / (RowIdx + 1);
				OutAccum[ColIdx].M[3][1] += ((Q.W * Q.Y) - OutAccum[ColIdx].M[3][1]) / (RowIdx + 1);
				OutAccum[ColIdx].M[3][2] += ((Q.W * Q.Z) - OutAccum[ColIdx].M[3][2]) / (RowIdx + 1);
				OutAccum[ColIdx].M[3][3] += ((Q.W * Q.W) - OutAccum[ColIdx].M[3][3]) / (RowIdx + 1);
			}
		}

		for (int64 ColIdx = 0; ColIdx < ColNum; ColIdx++)
		{
			const FVector4f AverageQuat = DominantEigenVector(OutAccum[ColIdx], FVector4f(0.0f, 0.0f, 0.0f, 1.0f) );
			OutMeans[ColIdx] = FQuat4f(AverageQuat.X, AverageQuat.Y, AverageQuat.Z, AverageQuat.W);
		}

		Learning::Array::Zero(OutStds);
		for (int64 RowIdx = 0; RowIdx < RowNum; RowIdx++)
		{
			for (int64 ColIdx = 0; ColIdx < ColNum; ColIdx++)
			{
				const FQuat4f Diff = (Rotations[RowIdx][ColIdx] * OutMeans[ColIdx].Inverse()).GetShortestArcWith(FQuat4f::Identity);
				OutStds[ColIdx] += FMath::Square(Diff.ToRotationVector()) / RowNum;
			}
		}

		for (int64 ColIdx = 0; ColIdx < ColNum; ColIdx++)
		{
			OutStds[ColIdx] = VectorSqrt(OutStds[ColIdx]);
		}
	}

	void ComputeMeanStdWithReference(
		FQuat4f& OutMean,
		FVector3f& OutStd,
		const TLearningArrayView<1, const FQuat4f> Rotations,
		const FQuat4f Reference)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::AnimDatabase::Math::ComputeMeanStdWithReference);

		const int64 Num = Rotations.Num<0>();
		check(Num > 0);

		if (UE_ANIMDATABASE_ISPC && Rotations.NumBytes() < MAX_int32)
		{
#if UE_ANIMDATABASE_ISPC
			ispc::AnimDatabaseComputeMeanStdWithReference1D(
				(float*)&OutMean,
				(float*)&OutStd,
				(const float*)Rotations.GetData(),
				(const float*)&Reference,
				Num);
#endif
		}
		else
		{
			FMatrix44f Accum;
			FMemory::Memzero(Accum);

			for (int64 Idx = 0; Idx < Num; Idx++)
			{
				const FQuat4f Q = Rotations[Idx];
				
				// make Q^T * Q matrix and accumulate using online mean algorithm :

				Accum.M[0][0] += ((Q.X * Q.X) - Accum.M[0][0]) / (Idx + 1);
				Accum.M[0][1] += ((Q.X * Q.Y) - Accum.M[0][1]) / (Idx + 1);
				Accum.M[0][2] += ((Q.X * Q.Z) - Accum.M[0][2]) / (Idx + 1);
				Accum.M[0][3] += ((Q.X * Q.W) - Accum.M[0][3]) / (Idx + 1);

				Accum.M[1][0] += ((Q.Y * Q.X) - Accum.M[1][0]) / (Idx + 1);
				Accum.M[1][1] += ((Q.Y * Q.Y) - Accum.M[1][1]) / (Idx + 1);
				Accum.M[1][2] += ((Q.Y * Q.Z) - Accum.M[1][2]) / (Idx + 1);
				Accum.M[1][3] += ((Q.Y * Q.W) - Accum.M[1][3]) / (Idx + 1);

				Accum.M[2][0] += ((Q.Z * Q.X) - Accum.M[2][0]) / (Idx + 1);
				Accum.M[2][1] += ((Q.Z * Q.Y) - Accum.M[2][1]) / (Idx + 1);
				Accum.M[2][2] += ((Q.Z * Q.Z) - Accum.M[2][2]) / (Idx + 1);
				Accum.M[2][3] += ((Q.Z * Q.W) - Accum.M[2][3]) / (Idx + 1);

				Accum.M[3][0] += ((Q.W * Q.X) - Accum.M[3][0]) / (Idx + 1);
				Accum.M[3][1] += ((Q.W * Q.Y) - Accum.M[3][1]) / (Idx + 1);
				Accum.M[3][2] += ((Q.W * Q.Z) - Accum.M[3][2]) / (Idx + 1);
				Accum.M[3][3] += ((Q.W * Q.W) - Accum.M[3][3]) / (Idx + 1);
			}

			const FQuat4f R = Reference;
			const FVector4f AverageQuat = DominantEigenVector(Accum, FVector4f(R.X, R.Y, R.Z, R.W) );
			OutMean = FQuat4f(AverageQuat.X, AverageQuat.Y, AverageQuat.Z, AverageQuat.W);

			OutStd = FVector3f::ZeroVector;
			for (int64 Idx = 0; Idx < Num; Idx++)
			{
				const FQuat4f Diff = (Rotations[Idx] * OutMean.Inverse()).GetShortestArcWith(FQuat4f::Identity);
				OutStd += FMath::Square(Diff.ToRotationVector()) / Num;
			}

			OutStd = VectorSqrt(OutStd);
		}
	}

	void ComputeMeanStdWithReference(
		const TLearningArrayView<1, FQuat4f> OutMeans,
		const TLearningArrayView<1, FMatrix44f> OutAccum,
		const TLearningArrayView<1, FVector3f> OutStds,
		const TLearningArrayView<2, const FQuat4f> Rotations,
		const TLearningArrayView<1, const FQuat4f> Reference)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::AnimDatabase::Math::ComputeMeanStdWithReference);

		const int64 RowNum = Rotations.Num<0>();
		const int64 ColNum = Rotations.Num<1>();
		check(RowNum > 0);

		if (UE_ANIMDATABASE_ISPC && Rotations.NumBytes() < MAX_int32)
		{
#if UE_ANIMDATABASE_ISPC
			ispc::AnimDatabaseComputeMeanStdWithReference(
				(float*)OutMeans.GetData(),
				(float*)OutAccum.GetData(),
				(float*)OutStds.GetData(),
				(const float*)Rotations.GetData(),
				(const float*)Reference.GetData(),
				RowNum,
				ColNum);
#endif
		}
		else
		{
			Learning::Array::Zero(OutAccum);
			for (int64 RowIdx = 0; RowIdx < RowNum; RowIdx++)
			{
				for (int64 ColIdx = 0; ColIdx < ColNum; ColIdx++)
				{
					const FQuat4f Q = Rotations[RowIdx][ColIdx];

					OutAccum[ColIdx].M[0][0] += ((Q.X * Q.X) - OutAccum[ColIdx].M[0][0]) / (RowIdx + 1);
					OutAccum[ColIdx].M[0][1] += ((Q.X * Q.Y) - OutAccum[ColIdx].M[0][1]) / (RowIdx + 1);
					OutAccum[ColIdx].M[0][2] += ((Q.X * Q.Z) - OutAccum[ColIdx].M[0][2]) / (RowIdx + 1);
					OutAccum[ColIdx].M[0][3] += ((Q.X * Q.W) - OutAccum[ColIdx].M[0][3]) / (RowIdx + 1);

					OutAccum[ColIdx].M[1][0] += ((Q.Y * Q.X) - OutAccum[ColIdx].M[1][0]) / (RowIdx + 1);
					OutAccum[ColIdx].M[1][1] += ((Q.Y * Q.Y) - OutAccum[ColIdx].M[1][1]) / (RowIdx + 1);
					OutAccum[ColIdx].M[1][2] += ((Q.Y * Q.Z) - OutAccum[ColIdx].M[1][2]) / (RowIdx + 1);
					OutAccum[ColIdx].M[1][3] += ((Q.Y * Q.W) - OutAccum[ColIdx].M[1][3]) / (RowIdx + 1);

					OutAccum[ColIdx].M[2][0] += ((Q.Z * Q.X) - OutAccum[ColIdx].M[2][0]) / (RowIdx + 1);
					OutAccum[ColIdx].M[2][1] += ((Q.Z * Q.Y) - OutAccum[ColIdx].M[2][1]) / (RowIdx + 1);
					OutAccum[ColIdx].M[2][2] += ((Q.Z * Q.Z) - OutAccum[ColIdx].M[2][2]) / (RowIdx + 1);
					OutAccum[ColIdx].M[2][3] += ((Q.Z * Q.W) - OutAccum[ColIdx].M[2][3]) / (RowIdx + 1);

					OutAccum[ColIdx].M[3][0] += ((Q.W * Q.X) - OutAccum[ColIdx].M[3][0]) / (RowIdx + 1);
					OutAccum[ColIdx].M[3][1] += ((Q.W * Q.Y) - OutAccum[ColIdx].M[3][1]) / (RowIdx + 1);
					OutAccum[ColIdx].M[3][2] += ((Q.W * Q.Z) - OutAccum[ColIdx].M[3][2]) / (RowIdx + 1);
					OutAccum[ColIdx].M[3][3] += ((Q.W * Q.W) - OutAccum[ColIdx].M[3][3]) / (RowIdx + 1);
				}
			}

			for (int64 ColIdx = 0; ColIdx < ColNum; ColIdx++)
			{
				const FQuat4f R = Reference[ColIdx];
				const FVector4f AverageQuat = DominantEigenVector(OutAccum[ColIdx], FVector4f(R.X, R.Y, R.Z, R.W) );
				OutMeans[ColIdx] = FQuat4f(AverageQuat.X, AverageQuat.Y, AverageQuat.Z, AverageQuat.W);
			}

			Learning::Array::Zero(OutStds);
			for (int64 RowIdx = 0; RowIdx < RowNum; RowIdx++)
			{
				for (int64 ColIdx = 0; ColIdx < ColNum; ColIdx++)
				{
					const FQuat4f Diff = (Rotations[RowIdx][ColIdx] * OutMeans[ColIdx].Inverse()).GetShortestArcWith(FQuat4f::Identity);
					OutStds[ColIdx] += FMath::Square(Diff.ToRotationVector()) / RowNum;
				}
			}

			for (int64 ColIdx = 0; ColIdx < ColNum; ColIdx++)
			{
				OutStds[ColIdx] = VectorSqrt(OutStds[ColIdx]);
			}
		}
	}

	void ComputeMeanStdAroundReference(
		const TLearningArrayView<1, FQuat4f> OutMeans,
		const TLearningArrayView<1, FVector3f> OutStds,
		const TLearningArrayView<2, const FQuat4f> Rotations,
		const TLearningArrayView<1, const FQuat4f> Reference)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::AnimDatabase::Math::ComputeMeanStdAroundReference);

		const int64 RowNum = Rotations.Num<0>();
		const int64 ColNum = Rotations.Num<1>();
		check(RowNum > 0);

		Learning::Array::Zero(OutMeans);
		for (int64 RowIdx = 0; RowIdx < RowNum; RowIdx++)
		{
			for (int64 ColIdx = 0; ColIdx < ColNum; ColIdx++)
			{
				const FQuat4f Q = (Reference[ColIdx].Inverse() * Rotations[RowIdx][ColIdx]).GetShortestArcWith(FQuat4f::Identity);
				OutMeans[ColIdx] += (Q - OutMeans[ColIdx]) / (float)(RowIdx + 1);
			}
		}

		for (int64 ColIdx = 0; ColIdx < ColNum; ColIdx++)
		{
			OutMeans[ColIdx] = Reference[ColIdx] * OutMeans[ColIdx].GetNormalized();
		}

		Learning::Array::Zero(OutStds);
		for (int64 RowIdx = 0; RowIdx < RowNum; RowIdx++)
		{
			for (int64 ColIdx = 0; ColIdx < ColNum; ColIdx++)
			{
				const FQuat4f Diff = (Rotations[RowIdx][ColIdx] * OutMeans[ColIdx].Inverse()).GetShortestArcWith(FQuat4f::Identity);
				OutStds[ColIdx] += FMath::Square(Diff.ToRotationVector()) / RowNum;
			}
		}

		for (int64 ColIdx = 0; ColIdx < ColNum; ColIdx++)
		{
			OutStds[ColIdx] = VectorSqrt(OutStds[ColIdx]);
		}
	}

	void ComputeMinMax(
		float& OutMin,
		float& OutMax,
		const TLearningArrayView<1, const float> Data)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::AnimDatabase::Math::ComputeMinMax);

		const int64 SampleNum = Data.Num<0>();

		OutMin = +UE_MAX_FLT;
		OutMax = -UE_MAX_FLT;

		for (int64 SampleIdx = 0; SampleIdx < SampleNum; SampleIdx++)
		{
			OutMin = FMath::Min(OutMin, Data[SampleIdx]);
			OutMax = FMath::Max(OutMax, Data[SampleIdx]);
		}
	}

	void ComputeMinMax(
		const TLearningArrayView<1, float> OutMin,
		const TLearningArrayView<1, float> OutMax,
		const TLearningArrayView<2, const float> Data)
	{
		ComputeMinMax(OutMin, OutMax, Data, 0, Data.Num<1>());
	}

	void ComputeMinMax(
		const TLearningArrayView<1, float> OutMin,
		const TLearningArrayView<1, float> OutMax,
		const TLearningArrayView<2, const float> Data,
		const int32 ColSliceStart,
		const int32 ColSliceNum)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::AnimDatabase::Math::ComputeMinMax);

		const int64 SampleNum = Data.Num<0>();
		const int64 DimNum = Data.Num<1>();

		check(OutMin.Num() == DimNum);
		check(OutMax.Num() == DimNum);
		check(ColSliceStart >= 0 && ColSliceNum >= 0 && ColSliceStart + ColSliceNum <= DimNum);

		if (UE_ANIMDATABASE_ISPC && Data.NumBytes() < MAX_int32)
		{
#if UE_ANIMDATABASE_ISPC
			ispc::AnimDatabaseComputeMinMax(
				OutMin.GetData(),
				OutMax.GetData(),
				Data.GetData(),
				SampleNum,
				DimNum,
				ColSliceStart,
				ColSliceNum);
#endif
		}
		else
		{
			Learning::Array::Set(OutMin.Slice(ColSliceStart, ColSliceNum), +UE_MAX_FLT);
			Learning::Array::Set(OutMax.Slice(ColSliceStart, ColSliceNum), -UE_MAX_FLT);

			for (int64 SampleIdx = 0; SampleIdx < SampleNum; SampleIdx++)
			{
				for (int64 DimIdx = ColSliceStart; DimIdx < ColSliceStart + ColSliceNum; DimIdx++)
				{
					OutMin[DimIdx] = FMath::Min(OutMin[DimIdx], Data[SampleIdx][DimIdx]);
					OutMax[DimIdx] = FMath::Max(OutMax[DimIdx], Data[SampleIdx][DimIdx]);
				}
			}
		}
	}

	void ComputeMaskedMinMax(
		const TLearningArrayView<1, float> OutMin,
		const TLearningArrayView<1, float> OutMax,
		const TLearningArrayView<2, const float> Data,
		const TLearningArrayView<2, const bool> Mask)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::AnimDatabase::Math::ComputeMaskedMinMax);

		const int64 SampleNum = Data.Num<0>();
		const int64 DimNum = Data.Num<1>();

		check(OutMin.Num() == DimNum);
		check(OutMax.Num() == DimNum);
		check(Mask.Num<0>() == SampleNum);
		check(Mask.Num<1>() == DimNum);

		Learning::Array::Set(OutMin, +UE_MAX_FLT);
		Learning::Array::Set(OutMax, -UE_MAX_FLT);

		for (int64 SampleIdx = 0; SampleIdx < SampleNum; SampleIdx++)
		{
			for (int64 DimIdx = 0; DimIdx < DimNum; DimIdx++)
			{
				if (Mask[SampleIdx][DimIdx])
				{
					OutMin[DimIdx] = FMath::Min(OutMin[DimIdx], Data[SampleIdx][DimIdx]);
					OutMax[DimIdx] = FMath::Max(OutMax[DimIdx], Data[SampleIdx][DimIdx]);
				}
			}
		}
	}

	void ScaleInplace(
		const TLearningArrayView<2, float> InOutData,
		const TLearningArrayView<1, const float> Scale)
	{
		const int64 SampleNum = InOutData.Num<0>();
		const int64 DimNum = InOutData.Num<1>();

		check(Scale.Num() == DimNum);

		if (UE_ANIMDATABASE_ISPC && InOutData.NumBytes() < MAX_int32)
		{
#if UE_ANIMDATABASE_ISPC
			ispc::AnimDatabaseScaleInplace(
				InOutData.GetData(),
				Scale.GetData(),
				SampleNum,
				DimNum);
#endif
		}
		else
		{
			for (int64 SampleIdx = 0; SampleIdx < SampleNum; SampleIdx++)
			{
				for (int64 DimIdx = 0; DimIdx < DimNum; DimIdx++)
				{
					InOutData[SampleIdx][DimIdx] = InOutData[SampleIdx][DimIdx] * Scale[DimIdx];
				}
			}
		}
	}

	void NormalizeInplace(
		const TLearningArrayView<2, float> InOutData,
		const TLearningArrayView<1, const float> Mean,
		const TLearningArrayView<1, const float> Std,
		const float Eps)
	{
		const int64 SampleNum = InOutData.Num<0>();
		const int64 DimNum = InOutData.Num<1>();

		check(Mean.Num() == DimNum);
		check(Std.Num() == DimNum);

		if (UE_ANIMDATABASE_ISPC && InOutData.NumBytes() < MAX_int32)
		{
#if UE_ANIMDATABASE_ISPC
			ispc::AnimDatabaseNormalizeInplace(
				InOutData.GetData(),
				Mean.GetData(),
				Std.GetData(),
				SampleNum,
				DimNum,
				Eps);
#endif
		}
		else
		{
			for (int64 SampleIdx = 0; SampleIdx < SampleNum; SampleIdx++)
			{
				for (int64 DimIdx = 0; DimIdx < DimNum; DimIdx++)
				{
					InOutData[SampleIdx][DimIdx] = (InOutData[SampleIdx][DimIdx] - Mean[DimIdx]) / FMath::Max(Std[DimIdx], Eps);
				}
			}
		}
	}

	void NormalizeInplace(
		const TLearningArrayView<2, float> InOutData,
		const TLearningArrayView<1, const float> Mean,
		const float Std,
		const float Eps)
	{
		const int64 SampleNum = InOutData.Num<0>();
		const int64 DimNum = InOutData.Num<1>();

		check(Mean.Num() == DimNum);

		if (UE_ANIMDATABASE_ISPC && InOutData.NumBytes() < MAX_int32)
		{
#if UE_ANIMDATABASE_ISPC
			ispc::AnimDatabaseNormalizeSingleScaleInplace(
				InOutData.GetData(),
				Mean.GetData(),
				Std,
				SampleNum,
				DimNum,
				Eps);
#endif
		}
		else
		{
			for (int64 SampleIdx = 0; SampleIdx < SampleNum; SampleIdx++)
			{
				for (int64 DimIdx = 0; DimIdx < DimNum; DimIdx++)
				{
					InOutData[SampleIdx][DimIdx] = (InOutData[SampleIdx][DimIdx] - Mean[DimIdx]) / FMath::Max(Std, Eps);
				}
			}
		}
	}

	void DenormalizeInplace(
		const TLearningArrayView<2, float> InOutData,
		const TLearningArrayView<1, const float> Mean,
		const TLearningArrayView<1, const float> Std)
	{
		const int64 SampleNum = InOutData.Num<0>();
		const int64 DimNum = InOutData.Num<1>();

		check(Mean.Num() == DimNum);
		check(Std.Num() == DimNum);

		if (UE_ANIMDATABASE_ISPC && InOutData.NumBytes() < MAX_int32)
		{
#if UE_ANIMDATABASE_ISPC
			ispc::AnimDatabaseDenormalizeInplace(
				InOutData.GetData(),
				Mean.GetData(),
				Std.GetData(),
				SampleNum,
				DimNum);
#endif
		}
		else
		{
			for (int64 SampleIdx = 0; SampleIdx < SampleNum; SampleIdx++)
			{
				for (int64 DimIdx = 0; DimIdx < DimNum; DimIdx++)
				{
					InOutData[SampleIdx][DimIdx] = InOutData[SampleIdx][DimIdx] * Std[DimIdx] + Mean[DimIdx];
				}
			}
		}
	}

	void DenormalizeInplace(
		const TLearningArrayView<2, float> InOutData,
		const TLearningArrayView<1, const float> Mean,
		const float Std)
	{
		const int64 SampleNum = InOutData.Num<0>();
		const int64 DimNum = InOutData.Num<1>();

		check(Mean.Num() == DimNum);

		if (UE_ANIMDATABASE_ISPC && InOutData.NumBytes() < MAX_int32)
		{
#if UE_ANIMDATABASE_ISPC
			ispc::AnimDatabaseDenormalizeSingleScaleInplace(
				InOutData.GetData(),
				Mean.GetData(),
				Std,
				SampleNum,
				DimNum);
#endif
		}
		else
		{
			for (int64 SampleIdx = 0; SampleIdx < SampleNum; SampleIdx++)
			{
				for (int64 DimIdx = 0; DimIdx < DimNum; DimIdx++)
				{
					InOutData[SampleIdx][DimIdx] = InOutData[SampleIdx][DimIdx] * Std + Mean[DimIdx];
				}
			}
		}
	}

	void ClampInplace(
		const TLearningArrayView<2, float> InOutData,
		const TLearningArrayView<1, const float> Min,
		const TLearningArrayView<1, const float> Max)
	{
		const int64 SampleNum = InOutData.Num<0>();
		const int64 DimNum = InOutData.Num<1>();

		check(Min.Num() == DimNum);
		check(Max.Num() == DimNum);

		if (UE_ANIMDATABASE_ISPC && InOutData.NumBytes() < MAX_int32)
		{
#if UE_ANIMDATABASE_ISPC
			ispc::AnimDatabaseClampInplace(
				InOutData.GetData(),
				Min.GetData(),
				Max.GetData(),
				SampleNum,
				DimNum);
#endif
		}
		else
		{
			for (int64 SampleIdx = 0; SampleIdx < SampleNum; SampleIdx++)
			{
				for (int64 DimIdx = 0; DimIdx < DimNum; DimIdx++)
				{
					InOutData[SampleIdx][DimIdx] = FMath::Clamp(InOutData[SampleIdx][DimIdx], Min[DimIdx], Max[DimIdx]);
				}
			}
		}
	}

	void MaskedClampInplace(
		const TLearningArrayView<2, float> InOutData,
		const TLearningArrayView<2, const bool> Mask,
		const TLearningArrayView<1, const float> Min,
		const TLearningArrayView<1, const float> Max)
	{
		const int64 SampleNum = InOutData.Num<0>();
		const int64 DimNum = InOutData.Num<1>();

		check(Min.Num() == DimNum);
		check(Max.Num() == DimNum);
		check(Mask.Num<0>() == SampleNum);
		check(Mask.Num<1>() == DimNum);

		for (int64 SampleIdx = 0; SampleIdx < SampleNum; SampleIdx++)
		{
			for (int64 DimIdx = 0; DimIdx < DimNum; DimIdx++)
			{
				if (Mask[SampleIdx][DimIdx])
				{
					InOutData[SampleIdx][DimIdx] = FMath::Clamp(InOutData[SampleIdx][DimIdx], Min[DimIdx], Max[DimIdx]);
				}
			}
		}
	}

	void ClampNormalizedInplace(
		const TLearningArrayView<2, float> InOutData,
		const TLearningArrayView<1, const float> Mean,
		const float Std,
		const TLearningArrayView<1, const float> Min,
		const TLearningArrayView<1, const float> Max)
	{
		const int64 SampleNum = InOutData.Num<0>();
		const int64 DimNum = InOutData.Num<1>();
		check(DimNum == Mean.Num());
		check(DimNum == Min.Num());
		check(DimNum == Max.Num());

		if (UE_ANIMDATABASE_ISPC && InOutData.NumBytes() < MAX_int32)
		{
#if UE_ANIMDATABASE_ISPC
			ispc::AnimDatabaseClampNormalizedSingleScaleInplace(
				InOutData.GetData(),
				Mean.GetData(),
				Std,
				Min.GetData(),
				Max.GetData(),
				SampleNum,
				DimNum);
#endif
		}
		else
		{
			for (int64 SampleIdx = 0; SampleIdx < SampleNum; SampleIdx++)
			{
				for (int64 DimIdx = 0; DimIdx < DimNum; DimIdx++)
				{
					InOutData[SampleIdx][DimIdx] = FMath::Clamp(
						InOutData[SampleIdx][DimIdx],
						(Min[DimIdx] - Mean[DimIdx]) / FMath::Max(Std, UE_SMALL_NUMBER),
						(Max[DimIdx] - Mean[DimIdx]) / FMath::Max(Std, UE_SMALL_NUMBER));
				}
			}
		}
	}

	bool AnyBoneIndicesInvalid(const UE::Learning::FIndexSet Indices)
	{
		for (int32 Idx : Indices)
		{
			if (Idx == INDEX_NONE)
			{
				return true;
			}
		}
		return false;
	}

	bool BoneIndicesAreSortedAndUnique(const UE::Learning::FIndexSet Indices)
	{
		const int32 Num = Indices.Num();
		for (int32 Idx = 0; Idx < Num - 1; Idx++)
		{
			if (Indices[Idx] >= Indices[Idx + 1]) { return false; }
		}
		return true;
	}

	int32 BoneAscendantsInclusiveNum(
		const int32 BoneIdx,
		const TLearningArrayView<1, const int32> BoneParents)
	{
		check(BoneIdx != INDEX_NONE);

		int32 Idx = BoneIdx;

		int32 Total = 0;
		while (BoneParents[Idx] != INDEX_NONE)
		{
			Total++;
			Idx = BoneParents[Idx];
		}
		Total++;

		return Total;
	}

	void BoneAscendantsInclusive(
		const TLearningArrayView<1, int32> OutBoneAscendants,
		const int32 BoneIdx,
		const TLearningArrayView<1, const int32> BoneParents)
	{
		int32 Idx = BoneIdx;

		int32 Total = 0;
		OutBoneAscendants[Total] = Idx;
		while (BoneParents[Idx] != INDEX_NONE)
		{
			Total++;
			Idx = BoneParents[Idx];
			OutBoneAscendants[Total] = Idx;
		}
		Total++;

		check(Total == OutBoneAscendants.Num());

		for (Idx = 0; Idx < Total / 2; Idx++)
		{
			Swap(OutBoneAscendants[Idx], OutBoneAscendants[Total - 1 - Idx]);
		}

		check(BoneIndicesAreSortedAndUnique(OutBoneAscendants));
	}

	void BoneAscendantsInclusive(
		TArray<int32>& OutBoneAscendants,
		const int32 BoneIdx,
		const TLearningArrayView<1, const int32> BoneParents)
	{

		OutBoneAscendants.SetNumUninitialized(BoneAscendantsInclusiveNum(BoneIdx, BoneParents));
		BoneAscendantsInclusive(MakeArrayView(OutBoneAscendants), BoneIdx, BoneParents);
	}

	int32 BoneUnionNum(
		const UE::Learning::FIndexSet Lhs,
		const UE::Learning::FIndexSet Rhs)
	{
		check(BoneIndicesAreSortedAndUnique(Lhs));
		check(BoneIndicesAreSortedAndUnique(Rhs));

		const int32 LhsNum = Lhs.Num();
		const int32 RhsNum = Rhs.Num();

		int32 Total = 0;
		int32 LhsIdx = 0;
		int32 RhsIdx = 0;

		while (LhsIdx < LhsNum && RhsIdx < RhsNum)
		{
			if (Lhs[LhsIdx] == Rhs[RhsIdx])
			{
				Total++;
				LhsIdx++;
				RhsIdx++;
			}
			else if (Lhs[LhsIdx] < Rhs[RhsIdx])
			{
				Total++;
				LhsIdx++;
			}
			else if (Rhs[RhsIdx] < Lhs[LhsIdx])
			{
				Total++;
				RhsIdx++;
			}
		}

		Total += LhsNum - LhsIdx;
		Total += RhsNum - RhsIdx;

		return Total;
	}

	void BoneUnion(
		const TLearningArrayView<1, int32> OutBoneUnion,
		const UE::Learning::FIndexSet Lhs,
		const UE::Learning::FIndexSet Rhs)
	{
		check(BoneIndicesAreSortedAndUnique(Lhs));
		check(BoneIndicesAreSortedAndUnique(Rhs));

		const int32 LhsNum = Lhs.Num();
		const int32 RhsNum = Rhs.Num();

		int32 Total = 0;
		int32 LhsIdx = 0;
		int32 RhsIdx = 0;

		while (LhsIdx < LhsNum && RhsIdx < RhsNum)
		{
			if (Lhs[LhsIdx] == Rhs[RhsIdx])
			{
				OutBoneUnion[Total] = Lhs[LhsIdx];
				Total++;
				LhsIdx++;
				RhsIdx++;
			}
			else if (Lhs[LhsIdx] < Rhs[RhsIdx])
			{
				OutBoneUnion[Total] = Lhs[LhsIdx];
				Total++;
				LhsIdx++;
			}
			else if (Rhs[RhsIdx] < Lhs[LhsIdx])
			{
				OutBoneUnion[Total] = Rhs[RhsIdx];
				Total++;
				RhsIdx++;
			}
		}

		while (LhsIdx < LhsNum)
		{
			OutBoneUnion[Total] = Lhs[LhsIdx];
			Total++;
			LhsIdx++;
		}

		while (RhsIdx < RhsNum)
		{
			OutBoneUnion[Total] = Rhs[RhsIdx];
			Total++;
			RhsIdx++;
		}

		check(OutBoneUnion.Num() == Total);
		check(BoneIndicesAreSortedAndUnique(OutBoneUnion));
	}

	void BoneUnion(
		TArray<int32>& OutBoneUnion,
		const UE::Learning::FIndexSet Lhs,
		const UE::Learning::FIndexSet Rhs)
	{
		OutBoneUnion.SetNumUninitialized(BoneUnionNum(Lhs, Rhs));
		BoneUnion(MakeArrayView(OutBoneUnion), Lhs, Rhs);
	}

	void BoneFindIndicesOf(
		const TLearningArrayView<1, int32> OutIndicesOf,
		const UE::Learning::FIndexSet Items,
		const UE::Learning::FIndexSet Elements)
	{
		const int32 ItemNum = Items.Num();

		check(OutIndicesOf.Num() == ItemNum);
		for (int32 ItemIdx = 0; ItemIdx < ItemNum; ItemIdx++)
		{
			OutIndicesOf[ItemIdx] = Elements.Find(Items[ItemIdx]);
		}
	}

	void BoneChildrenMatrix(
		TLearningArrayView<2, bool> OutBoneChildren,
		const TLearningArrayView<1, const int32> BoneParents)
	{
		const int32 BoneNum = BoneParents.Num();
		check(BoneNum == OutBoneChildren.Num<0>());
		check(BoneNum == OutBoneChildren.Num<1>());

		for (int32 RowIdx = 0; RowIdx < BoneNum; RowIdx++)
		{
			for (int32 ColIdx = 0; ColIdx < BoneNum; ColIdx++)
			{
				OutBoneChildren[RowIdx][ColIdx] = BoneParents[ColIdx] == RowIdx;
			}
		}
	}

	void BoneDescedantsMatrix(
		TLearningArrayView<2, bool> OutBoneDescendants,
		const TLearningArrayView<1, const int32> BoneParents)
	{
		const int32 BoneNum = BoneParents.Num();
		check(BoneNum == OutBoneDescendants.Num<0>());
		check(BoneNum == OutBoneDescendants.Num<1>());

		for (int32 RowIdx = 0; RowIdx < BoneNum; RowIdx++)
		{
			for (int32 ColIdx = 0; ColIdx < BoneNum; ColIdx++)
			{
				OutBoneDescendants[RowIdx][ColIdx] = false;

				int32 ParIdx = BoneParents[ColIdx];
				while (ParIdx != -1)
				{
					if (ParIdx == RowIdx)
					{
						OutBoneDescendants[RowIdx][ColIdx] = true;
						break;
					}

					ParIdx = BoneParents[ParIdx];
				}
			}
		}

	}

}

#undef UE_ANIMDATABASE_ISPC