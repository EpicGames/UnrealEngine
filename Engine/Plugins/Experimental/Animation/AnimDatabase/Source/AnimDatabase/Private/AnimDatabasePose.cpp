// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimDatabasePose.h"

#include "AnimDatabaseMath.h"
#include "AnimDatabaseFrameAttribute.h"

#include "Async/ParallelFor.h"

#ifndef UE_ANIMDATABASE_ISPC
#define UE_ANIMDATABASE_ISPC INTEL_ISPC
//#define UE_ANIMDATABASE_ISPC 0
#endif

#if UE_ANIMDATABASE_ISPC
#include "AnimDatabase.ispc.generated.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimDatabasePose)

namespace UE::AnimDatabase
{
	//--------------------------------------------------

	int32 FPoseRootDataView::GetFrameNum() const { return RootLocations.Num<0>(); }

	FPoseRootDataView FPoseRootData::View()
	{
		FPoseRootDataView View;
		View.RootLocations = RootLocations;
		View.RootRotations = RootRotations;
		View.RootScales = RootScales;
		View.RootLinearVelocities = RootLinearVelocities;
		View.RootAngularVelocities = RootAngularVelocities;
		View.RootScalarVelocities = RootScalarVelocities;
		return View;
	}

	FPoseRootDataConstView FPoseRootData::ConstView() const
	{
		FPoseRootDataConstView View;
		View.RootLocations = RootLocations;
		View.RootRotations = RootRotations;
		View.RootScales = RootScales;
		View.RootLinearVelocities = RootLinearVelocities;
		View.RootAngularVelocities = RootAngularVelocities;
		View.RootScalarVelocities = RootScalarVelocities;
		return View;
	}

	FPoseRootDataView FPoseRootData::Slice(const int32 FrameStart, const int32 FrameNum)
	{
		return View().Slice(FrameStart, FrameNum);
	}

	FPoseRootDataConstView FPoseRootData::ConstSlice(const int32 FrameStart, const int32 FrameNum) const
	{
		return ConstView().Slice(FrameStart, FrameNum);
	}

	FPoseRootDataConstView FPoseRootDataView::ConstView() const
	{
		FPoseRootDataConstView View;
		View.RootLocations = RootLocations;
		View.RootRotations = RootRotations;
		View.RootScales = RootScales;
		View.RootLinearVelocities = RootLinearVelocities;
		View.RootAngularVelocities = RootAngularVelocities;
		View.RootScalarVelocities = RootScalarVelocities;
		return View;
	}

	FPoseRootDataView FPoseRootDataView::Slice(const int32 FrameStart, const int32 FrameNum) const
	{
		FPoseRootDataView View;
		View.RootLocations = RootLocations.Slice(FrameStart, FrameNum);
		View.RootRotations = RootRotations.Slice(FrameStart, FrameNum);
		View.RootScales = RootScales.Slice(FrameStart, FrameNum);
		View.RootLinearVelocities = RootLinearVelocities.Slice(FrameStart, FrameNum);
		View.RootAngularVelocities = RootAngularVelocities.Slice(FrameStart, FrameNum);
		View.RootScalarVelocities = RootScalarVelocities.Slice(FrameStart, FrameNum);
		return View;
	}

	int32 FPoseRootDataConstView::GetFrameNum() const { return RootLocations.Num<0>(); }

	FPoseRootDataConstView FPoseRootDataConstView::Slice(const int32 FrameStart, const int32 FrameNum) const
	{
		FPoseRootDataConstView View;
		View.RootLocations = RootLocations.Slice(FrameStart, FrameNum);
		View.RootRotations = RootRotations.Slice(FrameStart, FrameNum);
		View.RootScales = RootScales.Slice(FrameStart, FrameNum);
		View.RootLinearVelocities = RootLinearVelocities.Slice(FrameStart, FrameNum);
		View.RootAngularVelocities = RootAngularVelocities.Slice(FrameStart, FrameNum);
		View.RootScalarVelocities = RootScalarVelocities.Slice(FrameStart, FrameNum);
		return View;
	}

	int32 FPoseRootData::GetFrameNum() const { return RootLocations.Num<0>(); }

	void FPoseRootData::Resize(const int32 FrameNum)
	{
		RootLocations.SetNumUninitialized({ FrameNum });
		RootRotations.SetNumUninitialized({ FrameNum });
		RootScales.SetNumUninitialized({ FrameNum });
		RootLinearVelocities.SetNumUninitialized({ FrameNum });
		RootAngularVelocities.SetNumUninitialized({ FrameNum });
		RootScalarVelocities.SetNumUninitialized({ FrameNum });
	}

	bool FPoseRootData::IsEmpty() const
	{
		return RootLocations.IsEmpty();
	}

	void FPoseRootData::Empty()
	{
		RootLocations.Empty();
		RootRotations.Empty();
		RootScales.Empty();
		RootLinearVelocities.Empty();
		RootAngularVelocities.Empty();
		RootScalarVelocities.Empty();
	}

	//--------------------------------------------------

	int32 FPoseLocalBoneDataView::GetFrameNum() const { return BoneLocations.Num<0>(); }
	int32 FPoseLocalBoneDataView::GetBoneNum() const { return BoneLocations.Num<1>(); }

	FPoseLocalBoneDataView FPoseLocalBoneData::View()
	{
		FPoseLocalBoneDataView View;
		View.BoneLocations = BoneLocations;
		View.BoneRotations = BoneRotations;
		View.BoneScales = BoneScales;
		View.BoneLinearVelocities = BoneLinearVelocities;
		View.BoneAngularVelocities = BoneAngularVelocities;
		View.BoneScalarVelocities = BoneScalarVelocities;
		return View;
	}

	FPoseLocalBoneDataConstView FPoseLocalBoneData::ConstView() const
	{
		FPoseLocalBoneDataConstView View;
		View.BoneLocations = BoneLocations;
		View.BoneRotations = BoneRotations;
		View.BoneScales = BoneScales;
		View.BoneLinearVelocities = BoneLinearVelocities;
		View.BoneAngularVelocities = BoneAngularVelocities;
		View.BoneScalarVelocities = BoneScalarVelocities;
		return View;
	}

	FPoseLocalBoneDataView FPoseLocalBoneData::Slice(const int32 FrameStart, const int32 FrameNum)
	{
		return View().Slice(FrameStart, FrameNum);
	}

	FPoseLocalBoneDataConstView FPoseLocalBoneData::ConstSlice(const int32 FrameStart, const int32 FrameNum) const
	{
		return ConstView().Slice(FrameStart, FrameNum);
	}

	FPoseLocalBoneDataConstView FPoseLocalBoneDataView::ConstView() const
	{
		FPoseLocalBoneDataConstView View;
		View.BoneLocations = BoneLocations;
		View.BoneRotations = BoneRotations;
		View.BoneScales = BoneScales;
		View.BoneLinearVelocities = BoneLinearVelocities;
		View.BoneAngularVelocities = BoneAngularVelocities;
		View.BoneScalarVelocities = BoneScalarVelocities;
		return View;
	}

	FPoseLocalBoneDataView FPoseLocalBoneDataView::Slice(const int32 FrameStart, const int32 FrameNum) const
	{
		FPoseLocalBoneDataView View;
		View.BoneLocations = BoneLocations.Slice(FrameStart, FrameNum);
		View.BoneRotations = BoneRotations.Slice(FrameStart, FrameNum);
		View.BoneScales = BoneScales.Slice(FrameStart, FrameNum);
		View.BoneLinearVelocities = BoneLinearVelocities.Slice(FrameStart, FrameNum);
		View.BoneAngularVelocities = BoneAngularVelocities.Slice(FrameStart, FrameNum);
		View.BoneScalarVelocities = BoneScalarVelocities.Slice(FrameStart, FrameNum);
		return View;
	}

	int32 FPoseLocalBoneDataConstView::GetFrameNum() const { return BoneLocations.Num<0>(); }
	int32 FPoseLocalBoneDataConstView::GetBoneNum() const { return BoneLocations.Num<1>(); }

	FPoseLocalBoneDataConstView FPoseLocalBoneDataConstView::Slice(const int32 FrameStart, const int32 FrameNum) const
	{
		FPoseLocalBoneDataConstView View;
		View.BoneLocations = BoneLocations.Slice(FrameStart, FrameNum);
		View.BoneRotations = BoneRotations.Slice(FrameStart, FrameNum);
		View.BoneScales = BoneScales.Slice(FrameStart, FrameNum);
		View.BoneLinearVelocities = BoneLinearVelocities.Slice(FrameStart, FrameNum);
		View.BoneAngularVelocities = BoneAngularVelocities.Slice(FrameStart, FrameNum);
		View.BoneScalarVelocities = BoneScalarVelocities.Slice(FrameStart, FrameNum);
		return View;
	}

	int32 FPoseLocalBoneData::GetFrameNum() const { return BoneLocations.Num<0>(); }
	int32 FPoseLocalBoneData::GetBoneNum() const { return BoneLocations.Num<1>(); }

	void FPoseLocalBoneData::Resize(const int32 FrameNum, const int32 BoneNum)
	{
		BoneLocations.SetNumUninitialized({ FrameNum, BoneNum });
		BoneRotations.SetNumUninitialized({ FrameNum, BoneNum });
		BoneScales.SetNumUninitialized({ FrameNum, BoneNum });
		BoneLinearVelocities.SetNumUninitialized({ FrameNum, BoneNum });
		BoneAngularVelocities.SetNumUninitialized({ FrameNum, BoneNum });
		BoneScalarVelocities.SetNumUninitialized({ FrameNum, BoneNum });
	}

	bool FPoseLocalBoneData::IsEmpty() const
	{
		return BoneLocations.IsEmpty();
	}

	void FPoseLocalBoneData::Empty()
	{
		BoneLocations.Empty();
		BoneRotations.Empty();
		BoneScales.Empty();
		BoneLinearVelocities.Empty();
		BoneAngularVelocities.Empty();
		BoneScalarVelocities.Empty();
	}

	//--------------------------------------------------

	int32 FPoseGlobalBoneDataView::GetFrameNum() const { return BoneLocations.Num<0>(); }
	int32 FPoseGlobalBoneDataView::GetBoneNum() const { return BoneLocations.Num<1>(); }

	FPoseGlobalBoneDataView FPoseGlobalBoneData::View()
	{
		FPoseGlobalBoneDataView View;
		View.BoneLocations = BoneLocations;
		View.BoneRotations = BoneRotations;
		View.BoneScales = BoneScales;
		View.BoneLinearVelocities = BoneLinearVelocities;
		View.BoneAngularVelocities = BoneAngularVelocities;
		View.BoneScalarVelocities = BoneScalarVelocities;
		return View;
	}

	FPoseGlobalBoneDataConstView FPoseGlobalBoneData::ConstView() const
	{
		FPoseGlobalBoneDataConstView View;
		View.BoneLocations = BoneLocations;
		View.BoneRotations = BoneRotations;
		View.BoneScales = BoneScales;
		View.BoneLinearVelocities = BoneLinearVelocities;
		View.BoneAngularVelocities = BoneAngularVelocities;
		View.BoneScalarVelocities = BoneScalarVelocities;
		return View;
	}

	FPoseGlobalBoneDataView FPoseGlobalBoneData::Slice(const int32 FrameStart, const int32 FrameNum)
	{
		return View().Slice(FrameStart, FrameNum);
	}

	FPoseGlobalBoneDataConstView FPoseGlobalBoneData::ConstSlice(const int32 FrameStart, const int32 FrameNum) const
	{
		return ConstView().Slice(FrameStart, FrameNum);
	}

	FPoseGlobalBoneDataView FPoseGlobalBoneDataView::Slice(const int32 FrameStart, const int32 FrameNum) const
	{
		FPoseGlobalBoneDataView View;
		View.BoneLocations = BoneLocations.Slice(FrameStart, FrameNum);
		View.BoneRotations = BoneRotations.Slice(FrameStart, FrameNum);
		View.BoneScales = BoneScales.Slice(FrameStart, FrameNum);
		View.BoneLinearVelocities = BoneLinearVelocities.Slice(FrameStart, FrameNum);
		View.BoneAngularVelocities = BoneAngularVelocities.Slice(FrameStart, FrameNum);
		View.BoneScalarVelocities = BoneScalarVelocities.Slice(FrameStart, FrameNum);
		return View;
	}

	int32 FPoseGlobalBoneDataConstView::GetFrameNum() const { return BoneLocations.Num<0>(); }
	int32 FPoseGlobalBoneDataConstView::GetBoneNum() const { return BoneLocations.Num<1>(); }

	FPoseGlobalBoneDataConstView FPoseGlobalBoneDataConstView::Slice(const int32 FrameStart, const int32 FrameNum) const
	{
		FPoseGlobalBoneDataConstView View;
		View.BoneLocations = BoneLocations.Slice(FrameStart, FrameNum);
		View.BoneRotations = BoneRotations.Slice(FrameStart, FrameNum);
		View.BoneScales = BoneScales.Slice(FrameStart, FrameNum);
		View.BoneLinearVelocities = BoneLinearVelocities.Slice(FrameStart, FrameNum);
		View.BoneAngularVelocities = BoneAngularVelocities.Slice(FrameStart, FrameNum);
		View.BoneScalarVelocities = BoneScalarVelocities.Slice(FrameStart, FrameNum);
		return View;
	}

	int32 FPoseGlobalBoneData::GetFrameNum() const { return BoneLocations.Num<0>(); }
	int32 FPoseGlobalBoneData::GetBoneNum() const { return BoneLocations.Num<1>(); }

	void FPoseGlobalBoneData::Resize(const int32 FrameNum, const int32 BoneNum)
	{
		BoneLocations.SetNumUninitialized({ FrameNum, BoneNum });
		BoneRotations.SetNumUninitialized({ FrameNum, BoneNum });
		BoneScales.SetNumUninitialized({ FrameNum, BoneNum });
		BoneLinearVelocities.SetNumUninitialized({ FrameNum, BoneNum });
		BoneAngularVelocities.SetNumUninitialized({ FrameNum, BoneNum });
		BoneScalarVelocities.SetNumUninitialized({ FrameNum, BoneNum });
	}

	bool FPoseGlobalBoneData::IsEmpty() const
	{
		return BoneLocations.IsEmpty();
	}

	void FPoseGlobalBoneData::Empty()
	{
		BoneLocations.Empty();
		BoneRotations.Empty();
		BoneScales.Empty();
		BoneLinearVelocities.Empty();
		BoneAngularVelocities.Empty();
		BoneScalarVelocities.Empty();
	}

	//--------------------------------------------------

	int32 FPoseAttributeDataConstView::GetFrameNum() const { return AttributeActive.Num<0>(); }
	int32 FPoseAttributeDataConstView::GetAttributeNum() const { return AttributeActive.Num<1>(); }
	TLearningArrayView<1, const EAnimDatabaseAttributeType> FPoseAttributeDataConstView::GetAttributeTypes() const { return AttributeTypes; }
	TLearningArrayView<1, const FName> FPoseAttributeDataConstView::GetAttributeNames() const { return AttributeNames; }

	FPoseAttributeDataConstView FPoseAttributeDataConstView::Slice(const int32 FrameStart, const int32 FrameNum) const
	{
		FPoseAttributeDataConstView View;
		View.AttributeTypes = AttributeTypes;
		View.AttributeNames = AttributeNames;
		View.AttributeOffsets = AttributeOffsets;
		View.AttributeActive = AttributeActive.Slice(FrameStart, FrameNum);
		View.AttributeData = AttributeData.Slice(FrameStart, FrameNum);
		return View;
	}

	int32 FPoseAttributeDataView::GetFrameNum() const { return AttributeActive.Num<0>(); }
	int32 FPoseAttributeDataView::GetAttributeNum() const { return AttributeActive.Num<1>(); }
	TLearningArrayView<1, const EAnimDatabaseAttributeType> FPoseAttributeDataView::GetAttributeTypes() const { return AttributeTypes; }
	TLearningArrayView<1, const FName> FPoseAttributeDataView::GetAttributeNames() const { return AttributeNames; }

	FPoseAttributeDataConstView FPoseAttributeDataView::ConstView() const
	{
		FPoseAttributeDataConstView View;
		View.AttributeTypes = AttributeTypes;
		View.AttributeNames = AttributeNames;
		View.AttributeOffsets = AttributeOffsets;
		View.AttributeActive = AttributeActive;
		View.AttributeData = AttributeData;
		return View;
	}

	FPoseAttributeDataView FPoseAttributeDataView::Slice(const int32 FrameStart, const int32 FrameNum) const
	{
		FPoseAttributeDataView View;
		View.AttributeTypes = AttributeTypes;
		View.AttributeNames = AttributeNames;
		View.AttributeOffsets = AttributeOffsets;
		View.AttributeActive = AttributeActive.Slice(FrameStart, FrameNum);
		View.AttributeData = AttributeData.Slice(FrameStart, FrameNum);
		return View;
	}

	int32 FPoseAttributeData::GetFrameNum() const { return AttributeActive.Num<0>(); }
	int32 FPoseAttributeData::GetAttributeNum() const { return AttributeActive.Num<1>(); }
	TLearningArrayView<1, const EAnimDatabaseAttributeType> FPoseAttributeData::GetAttributeTypes() const { return AttributeTypes; }
	TLearningArrayView<1, const FName> FPoseAttributeData::GetAttributeNames() const { return AttributeNames; }

	void FPoseAttributeData::Resize(const int32 FrameNum, const TLearningArrayView<1, const EAnimDatabaseAttributeType> InAttributeTypes, const TLearningArrayView<1, const FName> InAttributeNames)
	{
		const int32 AttributeNum = InAttributeTypes.Num();
		check(InAttributeNames.Num() == AttributeNum);

		AttributeTypes.SetNumUninitialized({ AttributeNum });
		AttributeNames.SetNumUninitialized({ AttributeNum });
		Learning::Array::Copy(AttributeTypes, InAttributeTypes);
		Learning::Array::Copy(AttributeNames, InAttributeNames);
		AttributeOffsets.SetNumUninitialized({ AttributeNum });

		int32 TotalAttributeSize = 0;
		for (int32 AttributeIdx = 0; AttributeIdx < AttributeNum; AttributeIdx++)
		{
			AttributeOffsets[AttributeIdx] = TotalAttributeSize;
			TotalAttributeSize += UAnimDatabaseFrameAttributeLibrary::AttributeTypeSize(AttributeTypes[AttributeIdx]);
		}

		AttributeActive.SetNumUninitialized({ FrameNum, AttributeNum });
		AttributeData.SetNumUninitialized({ FrameNum, TotalAttributeSize });
	}

	bool FPoseAttributeData::IsEmpty() const
	{
		return AttributeData.IsEmpty();
	}

	void FPoseAttributeData::Empty()
	{
		AttributeTypes.Empty();
		AttributeNames.Empty();
		AttributeOffsets.Empty();
		AttributeActive.Empty();
		AttributeData.Empty();
	}

	FPoseAttributeDataView FPoseAttributeData::View()
	{
		FPoseAttributeDataView View;
		View.AttributeTypes = AttributeTypes;
		View.AttributeNames = AttributeNames;
		View.AttributeOffsets = AttributeOffsets;
		View.AttributeActive = AttributeActive;
		View.AttributeData = AttributeData;
		return View;
	}

	FPoseAttributeDataConstView FPoseAttributeData::ConstView() const
	{
		FPoseAttributeDataConstView View;
		View.AttributeTypes = AttributeTypes;
		View.AttributeNames = AttributeNames;
		View.AttributeOffsets = AttributeOffsets;
		View.AttributeActive = AttributeActive;
		View.AttributeData = AttributeData;
		return View;
	}

	FPoseAttributeDataView FPoseAttributeData::Slice(const int32 FrameStart, const int32 FrameNum)
	{
		FPoseAttributeDataView View;
		View.AttributeTypes = AttributeTypes;
		View.AttributeNames = AttributeNames;
		View.AttributeOffsets = AttributeOffsets;
		View.AttributeActive = AttributeActive.Slice(FrameStart, FrameNum);
		View.AttributeData = AttributeData.Slice(FrameStart, FrameNum);
		return View;
	}

	FPoseAttributeDataConstView FPoseAttributeData::ConstSlice(const int32 FrameStart, const int32 FrameNum) const
	{
		FPoseAttributeDataConstView View;
		View.AttributeTypes = AttributeTypes;
		View.AttributeNames = AttributeNames;
		View.AttributeOffsets = AttributeOffsets;
		View.AttributeActive = AttributeActive.Slice(FrameStart, FrameNum);
		View.AttributeData = AttributeData.Slice(FrameStart, FrameNum);
		return View;
	}

	EAnimDatabaseAttributeType FPoseAttributeDataConstView::GetAttributeType(const int32 AttributeIdx) const
	{
		return AttributeTypes[AttributeIdx];
	}

	FName FPoseAttributeDataConstView::GetAttributeName(const int32 AttributeIdx) const
	{
		return AttributeNames[AttributeIdx];
	}

	int32 FPoseAttributeDataConstView::GetAttributeOffset(const int32 AttributeIdx) const
	{
		return AttributeOffsets[AttributeIdx];
	}

	int32 FPoseAttributeDataConstView::GetAttributeSize(const int32 AttributeIdx) const
	{
		return UAnimDatabaseFrameAttributeLibrary::AttributeTypeSize(GetAttributeType(AttributeIdx));
	}

	bool FPoseAttributeDataConstView::GetAttributeActive(const int32 FrameIdx, const int32 AttributeIdx) const
	{
		return AttributeActive[FrameIdx][AttributeIdx];
	}

	bool FPoseAttributeDataConstView::GetBool(const int32 FrameIdx, const int32 AttributeIdx) const
	{
		check(AttributeTypes[AttributeIdx] == EAnimDatabaseAttributeType::Bool);
		check(AttributeActive[FrameIdx][AttributeIdx]);

		return AttributeData[FrameIdx][AttributeOffsets[AttributeIdx]] > 0.0f;
	}

	float FPoseAttributeDataConstView::GetFloat(const int32 FrameIdx, const int32 AttributeIdx) const
	{
		check(AttributeTypes[AttributeIdx] == EAnimDatabaseAttributeType::Float);
		check(AttributeActive[FrameIdx][AttributeIdx]);

		return AttributeData[FrameIdx][AttributeOffsets[AttributeIdx]];
	}

	float FPoseAttributeDataConstView::GetAngle(const int32 FrameIdx, const int32 AttributeIdx) const
	{
		check(AttributeTypes[AttributeIdx] == EAnimDatabaseAttributeType::Angle);
		check(AttributeActive[FrameIdx][AttributeIdx]);

		return AttributeData[FrameIdx][AttributeOffsets[AttributeIdx]];
	}

	FVector3f FPoseAttributeDataConstView::GetLocation(const int32 FrameIdx, const int32 AttributeIdx) const
	{
		check(AttributeTypes[AttributeIdx] == EAnimDatabaseAttributeType::Location);
		check(AttributeActive[FrameIdx][AttributeIdx]);

		return FVector3f(
			AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 0],
			AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 1],
			AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 2]);
	}

	FQuat4f FPoseAttributeDataConstView::GetRotation(const int32 FrameIdx, const int32 AttributeIdx) const
	{
		check(AttributeTypes[AttributeIdx] == EAnimDatabaseAttributeType::Rotation);
		check(AttributeActive[FrameIdx][AttributeIdx]);

		return FQuat4f(
			AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 0],
			AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 1],
			AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 2],
			AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 3]);
	}

	FVector3f FPoseAttributeDataConstView::GetScale(const int32 FrameIdx, const int32 AttributeIdx) const
	{
		check(AttributeTypes[AttributeIdx] == EAnimDatabaseAttributeType::Scale);
		check(AttributeActive[FrameIdx][AttributeIdx]);

		return FVector3f(
			AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 0],
			AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 1],
			AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 2]);
	}

	FVector3f FPoseAttributeDataConstView::GetLinearVelocity(const int32 FrameIdx, const int32 AttributeIdx) const
	{
		check(AttributeTypes[AttributeIdx] == EAnimDatabaseAttributeType::LinearVelocity);
		check(AttributeActive[FrameIdx][AttributeIdx]);

		return FVector3f(
			AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 0],
			AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 1],
			AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 2]);
	}

	FVector3f FPoseAttributeDataConstView::GetAngularVelocity(const int32 FrameIdx, const int32 AttributeIdx) const
	{
		check(AttributeTypes[AttributeIdx] == EAnimDatabaseAttributeType::AngularVelocity);
		check(AttributeActive[FrameIdx][AttributeIdx]);

		return FVector3f(
			AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 0],
			AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 1],
			AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 2]);
	}

	FVector3f FPoseAttributeDataConstView::GetScalarVelocity(const int32 FrameIdx, const int32 AttributeIdx) const
	{
		check(AttributeTypes[AttributeIdx] == EAnimDatabaseAttributeType::ScalarVelocity);
		check(AttributeActive[FrameIdx][AttributeIdx]);

		return FVector3f(
			AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 0],
			AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 1],
			AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 2]);
	}

	FVector3f FPoseAttributeDataConstView::GetDirection(const int32 FrameIdx, const int32 AttributeIdx) const
	{
		check(AttributeTypes[AttributeIdx] == EAnimDatabaseAttributeType::Direction);
		check(AttributeActive[FrameIdx][AttributeIdx]);

		return FVector3f(
			AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 0],
			AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 1],
			AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 2]);
	}

	FTransform3f FPoseAttributeDataConstView::GetTransform(const int32 FrameIdx, const int32 AttributeIdx) const
	{
		check(AttributeTypes[AttributeIdx] == EAnimDatabaseAttributeType::Transform);
		check(AttributeActive[FrameIdx][AttributeIdx]);

		return FTransform3f(
			FQuat4f(
				AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 3],
				AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 4],
				AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 5],
				AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 6]),
			FVector3f(
				AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 0],
				AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 1],
				AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 2]),
			FVector3f(
				AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 7],
				AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 8],
				AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 9]));
	}

	void FPoseAttributeDataConstView::GetEvent(bool& bOutTimeUntilEventKnown, float& OutTimeUntilEvent, const int32 FrameIdx, const int32 AttributeIdx) const
	{
		check(AttributeTypes[AttributeIdx] == EAnimDatabaseAttributeType::Event);
		check(AttributeActive[FrameIdx][AttributeIdx]);

		bOutTimeUntilEventKnown = AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 0] == 1.0f;
		OutTimeUntilEvent = bOutTimeUntilEventKnown ? AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 1] : UE_MAX_FLT;
	}

	EAnimDatabaseAttributeType FPoseAttributeDataView::GetAttributeType(const int32 AttributeIdx) const { return ConstView().GetAttributeType(AttributeIdx); }
	FName FPoseAttributeDataView::GetAttributeName(const int32 AttributeIdx) const { return ConstView().GetAttributeName(AttributeIdx); }
	int32 FPoseAttributeDataView::GetAttributeOffset(const int32 AttributeIdx) const { return ConstView().GetAttributeOffset(AttributeIdx); }
	int32 FPoseAttributeDataView::GetAttributeSize(const int32 AttributeIdx) const { return ConstView().GetAttributeSize(AttributeIdx); }
	bool FPoseAttributeDataView::GetAttributeActive(const int32 FrameIdx, const int32 AttributeIdx) const { return ConstView().GetAttributeActive(FrameIdx, AttributeIdx); }
	bool FPoseAttributeDataView::GetBool(const int32 FrameIdx, const int32 AttributeIdx) const { return ConstView().GetBool(FrameIdx, AttributeIdx); }
	float FPoseAttributeDataView::GetFloat(const int32 FrameIdx, const int32 AttributeIdx) const { return ConstView().GetFloat(FrameIdx, AttributeIdx); }
	float FPoseAttributeDataView::GetAngle(const int32 FrameIdx, const int32 AttributeIdx) const { return ConstView().GetAngle(FrameIdx, AttributeIdx); }
	FVector3f FPoseAttributeDataView::GetLocation(const int32 FrameIdx, const int32 AttributeIdx) const { return ConstView().GetLocation(FrameIdx, AttributeIdx); }
	FQuat4f FPoseAttributeDataView::GetRotation(const int32 FrameIdx, const int32 AttributeIdx) const { return ConstView().GetRotation(FrameIdx, AttributeIdx); }
	FVector3f FPoseAttributeDataView::GetScale(const int32 FrameIdx, const int32 AttributeIdx) const { return ConstView().GetScale(FrameIdx, AttributeIdx); }
	FVector3f FPoseAttributeDataView::GetLinearVelocity(const int32 FrameIdx, const int32 AttributeIdx) const { return ConstView().GetLinearVelocity(FrameIdx, AttributeIdx); }
	FVector3f FPoseAttributeDataView::GetAngularVelocity(const int32 FrameIdx, const int32 AttributeIdx) const { return ConstView().GetAngularVelocity(FrameIdx, AttributeIdx); }
	FVector3f FPoseAttributeDataView::GetScalarVelocity(const int32 FrameIdx, const int32 AttributeIdx) const { return ConstView().GetScalarVelocity(FrameIdx, AttributeIdx); }
	FVector3f FPoseAttributeDataView::GetDirection(const int32 FrameIdx, const int32 AttributeIdx) const { return ConstView().GetDirection(FrameIdx, AttributeIdx); }
	FTransform3f FPoseAttributeDataView::GetTransform(const int32 FrameIdx, const int32 AttributeIdx) const { return ConstView().GetTransform(FrameIdx, AttributeIdx); }
	void FPoseAttributeDataView::GetEvent(bool& bOutTimeUntilEventKnown, float& OutTimeUntilEvent, const int32 FrameIdx, const int32 AttributeIdx) const { ConstView().GetEvent(bOutTimeUntilEventKnown, OutTimeUntilEvent, FrameIdx, AttributeIdx); }

	void FPoseAttributeDataView::SetAttributeActive(const int32 FrameIdx, const int32 AttributeIdx, const bool bActive) const
	{
		AttributeActive[FrameIdx][AttributeIdx] = bActive;
	}

	void FPoseAttributeDataView::SetBool(const int32 FrameIdx, const int32 AttributeIdx, const bool bValue) const
	{
		AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 0] = bValue ? 1.0f : 0.0f;
	}

	void FPoseAttributeDataView::SetFloat(const int32 FrameIdx, const int32 AttributeIdx, const float Value) const
	{
		AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 0] = Value;
	}

	void FPoseAttributeDataView::SetAngle(const int32 FrameIdx, const int32 AttributeIdx, const float Value) const
	{
		AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 0] = Value;
	}

	void FPoseAttributeDataView::SetLocation(const int32 FrameIdx, const int32 AttributeIdx, const FVector3f Location) const
	{
		AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 0] = Location.X;
		AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 1] = Location.Y;
		AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 2] = Location.Z;
	}

	void FPoseAttributeDataView::SetRotation(const int32 FrameIdx, const int32 AttributeIdx, const FQuat4f Rotation) const
	{
		AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 0] = Rotation.X;
		AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 1] = Rotation.Y;
		AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 2] = Rotation.Z;
		AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 3] = Rotation.W;
	}

	void FPoseAttributeDataView::SetScale(const int32 FrameIdx, const int32 AttributeIdx, const FVector3f Scale) const
	{
		AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 0] = Scale.X;
		AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 1] = Scale.Y;
		AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 2] = Scale.Z;
	}

	void FPoseAttributeDataView::SetLinearVelocity(const int32 FrameIdx, const int32 AttributeIdx, const FVector3f LinearVelocity) const
	{
		AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 0] = LinearVelocity.X;
		AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 1] = LinearVelocity.Y;
		AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 2] = LinearVelocity.Z;
	}

	void FPoseAttributeDataView::SetAngularVelocity(const int32 FrameIdx, const int32 AttributeIdx, const FVector3f AngularVelocity) const
	{
		AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 0] = AngularVelocity.X;
		AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 1] = AngularVelocity.Y;
		AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 2] = AngularVelocity.Z;
	}

	void FPoseAttributeDataView::SetScalarVelocity(const int32 FrameIdx, const int32 AttributeIdx, const FVector3f ScalarVelocity) const
	{
		AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 0] = ScalarVelocity.X;
		AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 1] = ScalarVelocity.Y;
		AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 2] = ScalarVelocity.Z;
	}

	void FPoseAttributeDataView::SetDirection(const int32 FrameIdx, const int32 AttributeIdx, const FVector3f Direction) const
	{
		AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 0] = Direction.X;
		AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 1] = Direction.Y;
		AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 2] = Direction.Z;
	}

	void FPoseAttributeDataView::SetTransform(const int32 FrameIdx, const int32 AttributeIdx, const FTransform3f Transform) const
	{
		AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 0] = Transform.GetLocation().X;
		AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 1] = Transform.GetLocation().Y;
		AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 2] = Transform.GetLocation().Z;
		AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 3] = Transform.GetRotation().X;
		AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 4] = Transform.GetRotation().Y;
		AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 5] = Transform.GetRotation().Z;
		AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 6] = Transform.GetRotation().W;
		AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 7] = Transform.GetScale3D().X;
		AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 8] = Transform.GetScale3D().Y;
		AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 9] = Transform.GetScale3D().Z;
	}

	void FPoseAttributeDataView::SetEvent(const int32 FrameIdx, const int32 AttributeIdx, const bool bTimeUntilEventKnown, const float TimeUntilEvent) const
	{
		AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 0] = bTimeUntilEventKnown ? 1.0f : 0.0f;
		AttributeData[FrameIdx][AttributeOffsets[AttributeIdx] + 1] = bTimeUntilEventKnown ? TimeUntilEvent : UE_MAX_FLT;
	}

	EAnimDatabaseAttributeType FPoseAttributeData::GetAttributeType(const int32 AttributeIdx) const { return ConstView().GetAttributeType(AttributeIdx); }
	FName FPoseAttributeData::GetAttributeName(const int32 AttributeIdx) const { return ConstView().GetAttributeName(AttributeIdx); }
	int32 FPoseAttributeData::GetAttributeOffset(const int32 AttributeIdx) const { return ConstView().GetAttributeOffset(AttributeIdx); }
	int32 FPoseAttributeData::GetAttributeSize(const int32 AttributeIdx) const { return ConstView().GetAttributeSize(AttributeIdx); }
	bool FPoseAttributeData::GetAttributeActive(const int32 FrameIdx, const int32 AttributeIdx) const { return ConstView().GetAttributeActive(FrameIdx, AttributeIdx); }
	bool FPoseAttributeData::GetBool(const int32 FrameIdx, const int32 AttributeIdx) const { return ConstView().GetBool(FrameIdx, AttributeIdx); }
	float FPoseAttributeData::GetFloat(const int32 FrameIdx, const int32 AttributeIdx) const { return ConstView().GetFloat(FrameIdx, AttributeIdx); }
	float FPoseAttributeData::GetAngle(const int32 FrameIdx, const int32 AttributeIdx) const { return ConstView().GetAngle(FrameIdx, AttributeIdx); }
	FVector3f FPoseAttributeData::GetLocation(const int32 FrameIdx, const int32 AttributeIdx) const { return ConstView().GetLocation(FrameIdx, AttributeIdx); }
	FQuat4f FPoseAttributeData::GetRotation(const int32 FrameIdx, const int32 AttributeIdx) const { return ConstView().GetRotation(FrameIdx, AttributeIdx); }
	FVector3f FPoseAttributeData::GetScale(const int32 FrameIdx, const int32 AttributeIdx) const { return ConstView().GetScale(FrameIdx, AttributeIdx); }
	FVector3f FPoseAttributeData::GetLinearVelocity(const int32 FrameIdx, const int32 AttributeIdx) const { return ConstView().GetLinearVelocity(FrameIdx, AttributeIdx); }
	FVector3f FPoseAttributeData::GetAngularVelocity(const int32 FrameIdx, const int32 AttributeIdx) const { return ConstView().GetAngularVelocity(FrameIdx, AttributeIdx); }
	FVector3f FPoseAttributeData::GetScalarVelocity(const int32 FrameIdx, const int32 AttributeIdx) const { return ConstView().GetScalarVelocity(FrameIdx, AttributeIdx); }
	FVector3f FPoseAttributeData::GetDirection(const int32 FrameIdx, const int32 AttributeIdx) const { return ConstView().GetDirection(FrameIdx, AttributeIdx); }
	FTransform3f FPoseAttributeData::GetTransform(const int32 FrameIdx, const int32 AttributeIdx) const { return ConstView().GetTransform(FrameIdx, AttributeIdx); }
	void FPoseAttributeData::GetEvent(bool& bOutTimeUntilEventKnown, float& OutTimeUntilEvent, const int32 FrameIdx, const int32 AttributeIdx) const { ConstView().GetEvent(bOutTimeUntilEventKnown, OutTimeUntilEvent, FrameIdx, AttributeIdx); }

	//--------------------------------------------------

	int32 FPoseDataConstView::GetFrameNum() const { return RootData.GetFrameNum(); }
	int32 FPoseDataConstView::GetBoneNum() const { return LocalBoneData.GetBoneNum(); }
	int32 FPoseDataConstView::GetAttributeNum() const { return AttributeData.GetAttributeNum(); }
	TLearningArrayView<1, const EAnimDatabaseAttributeType> FPoseDataConstView::GetAttributeTypes() const { return AttributeData.AttributeTypes; }
	TLearningArrayView<1, const FName> FPoseDataConstView::GetAttributeNames() const { return AttributeData.AttributeNames; }

	int32 FPoseDataView::GetFrameNum() const { return RootData.GetFrameNum(); }
	int32 FPoseDataView::GetBoneNum() const { return LocalBoneData.GetBoneNum(); }
	int32 FPoseDataView::GetAttributeNum() const { return AttributeData.GetAttributeNum(); }
	TLearningArrayView<1, const EAnimDatabaseAttributeType> FPoseDataView::GetAttributeTypes() const { return AttributeData.AttributeTypes; }
	TLearningArrayView<1, const FName> FPoseDataView::GetAttributeNames() const { return AttributeData.AttributeNames; }

	int32 FPoseData::GetFrameNum() const { return RootData.GetFrameNum(); }
	int32 FPoseData::GetBoneNum() const { return LocalBoneData.GetBoneNum(); }
	int32 FPoseData::GetAttributeNum() const { return AttributeData.GetAttributeNum(); }
	TLearningArrayView<1, const EAnimDatabaseAttributeType> FPoseData::GetAttributeTypes() const { return AttributeData.AttributeTypes; }
	TLearningArrayView<1, const FName> FPoseData::GetAttributeNames() const { return AttributeData.AttributeNames; }

	FPoseDataView FPoseData::View()
	{
		FPoseDataView View;
		View.RootData = RootData.View();
		View.LocalBoneData = LocalBoneData.View();
		View.AttributeData = AttributeData.View();
		return View;
	}

	FPoseDataConstView FPoseData::ConstView() const
	{
		FPoseDataConstView View;
		View.RootData = RootData.ConstView();
		View.LocalBoneData = LocalBoneData.ConstView();
		View.AttributeData = AttributeData.ConstView();
		return View;
	}

	FPoseDataView FPoseData::Slice(const int32 FrameStart, const int32 FrameNum)
	{
		return View().Slice(FrameStart, FrameNum);
	}

	FPoseDataConstView FPoseData::ConstSlice(const int32 FrameStart, const int32 FrameNum) const
	{
		return ConstView().Slice(FrameStart, FrameNum);
	}

	FPoseDataView FPoseDataView::Slice(const int32 FrameStart, const int32 FrameNum) const
	{
		FPoseDataView View;
		View.RootData = RootData.Slice(FrameStart, FrameNum);
		View.LocalBoneData = LocalBoneData.Slice(FrameStart, FrameNum);
		View.AttributeData = AttributeData.Slice(FrameStart, FrameNum);
		return View;
	}

	FPoseDataConstView FPoseDataConstView::Slice(const int32 FrameStart, const int32 FrameNum) const
	{
		FPoseDataConstView View;
		View.RootData = RootData.Slice(FrameStart, FrameNum);
		View.LocalBoneData = LocalBoneData.Slice(FrameStart, FrameNum);
		View.AttributeData = AttributeData.Slice(FrameStart, FrameNum);
		return View;
	}

	void FPoseData::Resize(const int32 FrameNum, const int32 BoneNum, const TLearningArrayView<1, const EAnimDatabaseAttributeType> InAttributeTypes, const TLearningArrayView<1, const FName> InAttributeNames)
	{
		RootData.Resize(FrameNum);
		LocalBoneData.Resize(FrameNum, BoneNum);
		AttributeData.Resize(FrameNum, InAttributeTypes, InAttributeNames);
	}

	bool FPoseData::IsEmpty() const
	{
		return RootData.IsEmpty();
	}

	void FPoseData::Empty()
	{
		RootData.Empty();
		LocalBoneData.Empty();
		AttributeData.Empty();
	}

	//--------------------------------------------------

	namespace PoseData
	{
		void Reset(const FPoseRootDataView& OutRootData)
		{
			Learning::Array::Zero(OutRootData.RootLocations);
			Learning::Array::Set(OutRootData.RootRotations, FQuat4f::Identity);
			Learning::Array::Set(OutRootData.RootScales, FVector3f::OneVector);
			Learning::Array::Zero(OutRootData.RootLinearVelocities);
			Learning::Array::Zero(OutRootData.RootAngularVelocities);
			Learning::Array::Zero(OutRootData.RootScalarVelocities);
		}

		void Reset(const FPoseLocalBoneDataView& OutPoseLocalBoneData, const TLearningArrayView<1, const FTransform> ReferenceTransforms, const TLearningArrayView<1, const int32> IgnoreBoneIndices)
		{
			const int32 BoneNum = ReferenceTransforms.Num();
			check(BoneNum == OutPoseLocalBoneData.GetBoneNum());

			const int32 FrameNum = OutPoseLocalBoneData.GetFrameNum();
			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
				{
					if (!IgnoreBoneIndices.Contains(BoneIdx))
					{
						OutPoseLocalBoneData.BoneLocations[FrameIdx][BoneIdx] = (FVector3f)ReferenceTransforms[BoneIdx].GetLocation();
						OutPoseLocalBoneData.BoneRotations[FrameIdx][BoneIdx] = (FQuat4f)ReferenceTransforms[BoneIdx].GetRotation();
						OutPoseLocalBoneData.BoneScales[FrameIdx][BoneIdx] = (FVector3f)ReferenceTransforms[BoneIdx].GetScale3D();
					}
				}
			}

			Learning::Array::Zero(OutPoseLocalBoneData.BoneLinearVelocities);
			Learning::Array::Zero(OutPoseLocalBoneData.BoneAngularVelocities);
			Learning::Array::Zero(OutPoseLocalBoneData.BoneScalarVelocities);
		}

		void Reset(const FPoseAttributeDataView& OutAttributeData)
		{
			Learning::Array::Zero(OutAttributeData.AttributeActive);
			Learning::Array::Zero(OutAttributeData.AttributeData);
		}

		void Reset(const FPoseDataView& OutPoseData, const TLearningArrayView<1, const FTransform> ReferenceTransforms, const TLearningArrayView<1, const int32> IgnoreBoneIndices)
		{
			Reset(OutPoseData.RootData);
			Reset(OutPoseData.LocalBoneData, ReferenceTransforms, IgnoreBoneIndices);
			Reset(OutPoseData.AttributeData);
		}

		void Copy(const FPoseRootDataView& OutRootData, const FPoseRootDataConstView& InRootData)
		{
			Learning::Array::Copy(OutRootData.RootLocations, InRootData.RootLocations);
			Learning::Array::Copy(OutRootData.RootRotations, InRootData.RootRotations);
			Learning::Array::Copy(OutRootData.RootScales, InRootData.RootScales);
			Learning::Array::Copy(OutRootData.RootLinearVelocities, InRootData.RootLinearVelocities);
			Learning::Array::Copy(OutRootData.RootAngularVelocities, InRootData.RootAngularVelocities);
			Learning::Array::Copy(OutRootData.RootScalarVelocities, InRootData.RootScalarVelocities);
		}

		void Copy(const FPoseLocalBoneDataView& OutPoseLocalBoneData, const FPoseLocalBoneDataConstView& InPoseLocalBoneData)
		{
			Learning::Array::Copy(OutPoseLocalBoneData.BoneLocations, InPoseLocalBoneData.BoneLocations);
			Learning::Array::Copy(OutPoseLocalBoneData.BoneRotations, InPoseLocalBoneData.BoneRotations);
			Learning::Array::Copy(OutPoseLocalBoneData.BoneScales, InPoseLocalBoneData.BoneScales);
			Learning::Array::Copy(OutPoseLocalBoneData.BoneLinearVelocities, InPoseLocalBoneData.BoneLinearVelocities);
			Learning::Array::Copy(OutPoseLocalBoneData.BoneAngularVelocities, InPoseLocalBoneData.BoneAngularVelocities);
			Learning::Array::Copy(OutPoseLocalBoneData.BoneScalarVelocities, InPoseLocalBoneData.BoneScalarVelocities);
		}

		void Copy(const FPoseAttributeDataView& OutAttributeData, const FPoseAttributeDataConstView& InAttributeData)
		{
			Learning::Array::Copy(OutAttributeData.AttributeActive, InAttributeData.AttributeActive);
			Learning::Array::Copy(OutAttributeData.AttributeData, InAttributeData.AttributeData);
		}

		void Copy(
			const FPoseDataView& OutPoseData,
			const FPoseDataConstView& InPoseData)
		{
			Copy(OutPoseData.RootData, InPoseData.RootData);
			Copy(OutPoseData.LocalBoneData, InPoseData.LocalBoneData);
			Copy(OutPoseData.AttributeData, InPoseData.AttributeData);
		}

		void ComputeUsedBones(
			TArray<int32>& OutBoneIndices,
			const FPoseLocalBoneDataConstView& InPoseLocalBoneData,
			const TLearningArrayView<1, const FTransform> ReferenceTransforms,
			const bool bRootIsAlwaysUsed,
			const float Eps)
		{
			const int32 FrameNum = InPoseLocalBoneData.GetFrameNum();
			const int32 BoneNum = InPoseLocalBoneData.GetBoneNum();
			check(BoneNum == ReferenceTransforms.Num());

			OutBoneIndices.Empty();
			if (bRootIsAlwaysUsed) { OutBoneIndices.Add(0); }

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
				{
					if (OutBoneIndices.Contains(BoneIdx)) { continue; }

					if (!InPoseLocalBoneData.BoneLocations[FrameIdx][BoneIdx].Equals((FVector3f)ReferenceTransforms[BoneIdx].GetLocation(), Eps))
					{
						OutBoneIndices.Add(BoneIdx);
						continue;
					}

					if (!InPoseLocalBoneData.BoneRotations[FrameIdx][BoneIdx].Equals((FQuat4f)ReferenceTransforms[BoneIdx].GetRotation(), Eps))
					{
						OutBoneIndices.Add(BoneIdx);
						continue;
					}

					if (!InPoseLocalBoneData.BoneScales[FrameIdx][BoneIdx].Equals((FVector3f)ReferenceTransforms[BoneIdx].GetScale3D(), Eps))
					{
						OutBoneIndices.Add(BoneIdx);
						continue;
					}
				}
			}
		}

		void ComputeDefaultBoneValuesAndIndices(
			TLearningArrayView<1, FVector3f> OutDefaultBoneLocations,
			TLearningArrayView<1, FQuat4f> OutDefaultBoneRotations,
			TLearningArrayView<1, FVector3f> OutDefaultBoneScales,
			TArray<int32>& OutBoneLocationIndices,
			TArray<int32>& OutBoneRotationIndices,
			TArray<int32>& OutBoneScaleIndices,
			const FPoseLocalBoneDataConstView& InPoseLocalBoneData,
			const TLearningArrayView<1, const FTransform> ReferenceTransforms,
			const TLearningArrayView<1, const int32> ExcludedBoneIndices,
			const float BoneLocationThreshold,
			const float BoneRotationThreshold,
			const float BoneLogScaleThreshold)
		{
			const int32 BoneNum = InPoseLocalBoneData.GetBoneNum();
			const int32 FrameNum = InPoseLocalBoneData.GetFrameNum();

			check(OutDefaultBoneLocations.Num() == BoneNum);
			check(OutDefaultBoneRotations.Num() == BoneNum);
			check(OutDefaultBoneScales.Num() == BoneNum);
			check(ReferenceTransforms.Num() == BoneNum);

			// Compute Averages

			TLearningArray<1, FVector3f> BoneLocationsMean;
			TLearningArray<1, FQuat4f> BoneRotationsMean;
			TLearningArray<1, FVector3f> BoneScalesMean;

			TLearningArray<1, FVector3f> BoneLocationsStd;
			TLearningArray<1, FVector3f> BoneRotationsStd;
			TLearningArray<1, FVector3f> BoneLogScalesStd;

			BoneLocationsMean.SetNumUninitialized({ BoneNum });
			BoneRotationsMean.SetNumUninitialized({ BoneNum });
			BoneScalesMean.SetNumUninitialized({ BoneNum });

			BoneLocationsStd.SetNumUninitialized({ BoneNum });
			BoneRotationsStd.SetNumUninitialized({ BoneNum });
			BoneLogScalesStd.SetNumUninitialized({ BoneNum });

			ParallelFor(BoneNum, [FrameNum, 
				&InPoseLocalBoneData, &ReferenceTransforms,
				&BoneLocationsMean, &BoneLocationsStd,
				&BoneRotationsMean, &BoneRotationsStd,
				&BoneScalesMean, &BoneLogScalesStd](int32 BoneIdx)
				{
					TLearningArray<1, FVector3f> BoneLocations; BoneLocations.SetNumUninitialized({ FrameNum });
					TLearningArray<1, FQuat4f> BoneRotations; BoneRotations.SetNumUninitialized({ FrameNum });
					TLearningArray<1, FVector3f> BoneScales; BoneScales.SetNumUninitialized({ FrameNum });

					for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
					{
						BoneLocations[FrameIdx] = InPoseLocalBoneData.BoneLocations[FrameIdx][BoneIdx];
						BoneRotations[FrameIdx] = InPoseLocalBoneData.BoneRotations[FrameIdx][BoneIdx];
						BoneScales[FrameIdx] = InPoseLocalBoneData.BoneScales[FrameIdx][BoneIdx];
					}

					const FQuat4f BoneReference = (FQuat4f)ReferenceTransforms[BoneIdx].GetRotation();

					Math::ComputeMeanStd(BoneLocationsMean[BoneIdx], BoneLocationsStd[BoneIdx], BoneLocations);
					Math::ComputeMeanStdWithReference(BoneRotationsMean[BoneIdx], BoneRotationsStd[BoneIdx], BoneRotations, BoneReference);
					Math::ComputeMeanStdOfLog(BoneScalesMean[BoneIdx], BoneLogScalesStd[BoneIdx], BoneScales);
				});

			// Compute Default Pose

			Learning::Array::Copy<1, FVector3f>(TLearningArrayView<1, FVector3f>(OutDefaultBoneLocations), BoneLocationsMean);
			Learning::Array::Copy<1, FQuat4f>(TLearningArrayView<1, FQuat4f>(OutDefaultBoneRotations), BoneRotationsMean);
			Learning::Array::Copy<1, FVector3f>(TLearningArrayView<1, FVector3f>(OutDefaultBoneScales), BoneScalesMean);

			// Excluded Bones use Reference Pose as Default

			for (const int32 BoneIdx : ExcludedBoneIndices)
			{
				if (BoneIdx >= 0 && BoneIdx < BoneNum)
				{
					OutDefaultBoneLocations[BoneIdx] = (FVector3f)ReferenceTransforms[BoneIdx].GetLocation();
					OutDefaultBoneRotations[BoneIdx] = (FQuat4f)ReferenceTransforms[BoneIdx].GetRotation();
					OutDefaultBoneScales[BoneIdx] = (FVector3f)ReferenceTransforms[BoneIdx].GetScale3D();
				}
			}

			// Compute Bone Indices

			OutBoneLocationIndices.Empty(BoneNum);
			OutBoneRotationIndices.Empty(BoneNum);
			OutBoneScaleIndices.Empty(BoneNum);

			for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
			{
				if (ExcludedBoneIndices.Contains(BoneIdx)) { continue; }

				if (BoneLocationsStd[BoneIdx].X > BoneLocationThreshold ||
					BoneLocationsStd[BoneIdx].Y > BoneLocationThreshold ||
					BoneLocationsStd[BoneIdx].Z > BoneLocationThreshold)
				{
					OutBoneLocationIndices.Add(BoneIdx);
				}

				if (BoneRotationsStd[BoneIdx].X > BoneRotationThreshold ||
					BoneRotationsStd[BoneIdx].Y > BoneRotationThreshold ||
					BoneRotationsStd[BoneIdx].Z > BoneRotationThreshold)
				{
					OutBoneRotationIndices.Add(BoneIdx);
				}

				if (BoneLogScalesStd[BoneIdx].X > BoneLogScaleThreshold ||
					BoneLogScalesStd[BoneIdx].Y > BoneLogScaleThreshold ||
					BoneLogScalesStd[BoneIdx].Z > BoneLogScaleThreshold)
				{
					OutBoneScaleIndices.Add(BoneIdx);
				}
			}

			OutBoneLocationIndices.Shrink();
			OutBoneRotationIndices.Shrink();
			OutBoneScaleIndices.Shrink();
		}

		void ComputeRequiredBoneIndices(
			TArray<int32>& OutBoneRequiredIndices,
			const TLearningArrayView<1, const int32> BoneLocationIndices,
			const TLearningArrayView<1, const int32> BoneRotationIndices,
			const TLearningArrayView<1, const int32> BoneScaleIndices)
		{
			Math::BoneUnion(OutBoneRequiredIndices, BoneLocationIndices, BoneRotationIndices);
			TArray<int32> TempRequiredBoneIndices = OutBoneRequiredIndices;
			Math::BoneUnion(OutBoneRequiredIndices, TempRequiredBoneIndices, BoneScaleIndices);
		}

		int32 AttributeTypeEncodingSize(const EAnimDatabaseAttributeType Type)
		{
			switch (Type)
			{
			case EAnimDatabaseAttributeType::Null: return 1 + 0;
			case EAnimDatabaseAttributeType::Bool: return 1 + 1;
			case EAnimDatabaseAttributeType::Float: return 1 + 1;
			case EAnimDatabaseAttributeType::Location: return 1 + 3;
			case EAnimDatabaseAttributeType::Rotation: return 1 + 6;
			case EAnimDatabaseAttributeType::Scale: return 1 + 3;
			case EAnimDatabaseAttributeType::LinearVelocity: return 1 + 3;
			case EAnimDatabaseAttributeType::AngularVelocity: return 1 + 3;
			case EAnimDatabaseAttributeType::ScalarVelocity: return 1 + 3;
			case EAnimDatabaseAttributeType::Direction: return 1 + 3;
			case EAnimDatabaseAttributeType::Transform: return 1 + 3 + 6 + 3;
			case EAnimDatabaseAttributeType::Event: return 1 + 2;
			case EAnimDatabaseAttributeType::Angle: return 1 + 2;
			default: checkNoEntry(); return 0;
			}
		}

		int32 PoseVectorSize(
			const int32 BoneLocationNum,
			const int32 BoneRotationNum,
			const int32 BoneScaleNum,
			const TLearningArrayView<1, const EAnimDatabaseAttributeType> AttributeTypes)
		{
			// Bones
			int32 Total =
				3 + // Root Log Scale
				3 + // Root Linear Velocity
				3 + // Root Angular Velocity
				3 + // Root Scalar Velocity
				BoneLocationNum * 3 + // Bone Locations
				BoneRotationNum * 6 + // Bone Rotations
				BoneScaleNum    * 3 + // Bone Scales
				BoneLocationNum * 3 + // Bone Linear Velocities
				BoneRotationNum * 3 + // Bone Angular Velocities
				BoneScaleNum    * 3;  // Bone Scalar Velocities

			// Attributes
			for (const EAnimDatabaseAttributeType& Type : AttributeTypes)
			{
				Total += AttributeTypeEncodingSize(Type);
			}

			return Total;
		}

		void SetDefaultPoseData(
			const FPoseDataView& OutPoseData,
			const TLearningArrayView<1, const FVector3f> DefaultBoneLocations,
			const TLearningArrayView<1, const FQuat4f> DefaultBoneRotations,
			const TLearningArrayView<1, const FVector3f> DefaultBoneScales)
		{
			SetDefaultPoseData(
				OutPoseData, 
				DefaultBoneLocations,
				DefaultBoneRotations,
				DefaultBoneScales,
				Learning::FIndexSet(0, OutPoseData.GetBoneNum()));
		}

		void SetDefaultPoseData(
			const FPoseDataView& OutPoseData,
			const TLearningArrayView<1, const FVector3f> DefaultBoneLocations,
			const TLearningArrayView<1, const FQuat4f> DefaultBoneRotations,
			const TLearningArrayView<1, const FVector3f> DefaultBoneScales,
			const Learning::FIndexSet BoneIndices)
		{
			check(Math::BoneIndicesAreSortedAndUnique(BoneIndices));

			Learning::Array::Set(OutPoseData.RootData.RootLocations, FVector::ZeroVector);
			Learning::Array::Set(OutPoseData.RootData.RootRotations, FQuat4f::Identity);
			Learning::Array::Set(OutPoseData.RootData.RootScales, FVector3f::OneVector);
			Learning::Array::Set(OutPoseData.RootData.RootLinearVelocities, FVector3f::ZeroVector);
			Learning::Array::Set(OutPoseData.RootData.RootAngularVelocities, FVector3f::ZeroVector);
			Learning::Array::Set(OutPoseData.RootData.RootScalarVelocities, FVector3f::ZeroVector);

			const int32 FrameNum = OutPoseData.GetFrameNum();
			const int32 BoneNum = OutPoseData.GetBoneNum();
			check(BoneNum == BoneIndices.Num());

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
				{
					OutPoseData.LocalBoneData.BoneLocations[FrameIdx][BoneIdx] = DefaultBoneLocations[BoneIndices[BoneIdx]];
					OutPoseData.LocalBoneData.BoneRotations[FrameIdx][BoneIdx] = DefaultBoneRotations[BoneIndices[BoneIdx]];
					OutPoseData.LocalBoneData.BoneScales[FrameIdx][BoneIdx] = DefaultBoneScales[BoneIndices[BoneIdx]];
				}
			}

			Learning::Array::Set(OutPoseData.LocalBoneData.BoneLinearVelocities, FVector3f::ZeroVector);
			Learning::Array::Set(OutPoseData.LocalBoneData.BoneAngularVelocities, FVector3f::ZeroVector);
			Learning::Array::Set(OutPoseData.LocalBoneData.BoneScalarVelocities, FVector3f::ZeroVector);

			Learning::Array::Zero(OutPoseData.AttributeData.AttributeActive);
			Learning::Array::Zero(OutPoseData.AttributeData.AttributeData);
		}

		void ToPoseVectors(
			const TLearningArrayView<2, float> OutPoseVectors,
			const FPoseDataConstView& InPoseData,
			const Learning::FIndexSet BoneLocationIndices,
			const Learning::FIndexSet BoneRotationIndices,
			const Learning::FIndexSet BoneScaleIndices,
			const TLearningArrayView<1, const FVector3f> DefaultBoneLocations,
			const TLearningArrayView<1, const FQuat4f> DefaultBoneRotations,
			const TLearningArrayView<1, const FVector3f> DefaultBoneScales)
		{
			ToPoseVectors(
				OutPoseVectors, 
				InPoseData, 
				BoneLocationIndices,
				BoneRotationIndices,
				BoneScaleIndices,
				DefaultBoneLocations,
				DefaultBoneRotations,
				DefaultBoneScales,
				Learning::FIndexSet(0, InPoseData.GetBoneNum()));
		}

		void ToPoseVectors(
			const TLearningArrayView<2, float> OutPoseVectors,
			const FPoseDataConstView& InPoseData,
			const Learning::FIndexSet BoneLocationIndices,
			const Learning::FIndexSet BoneRotationIndices,
			const Learning::FIndexSet BoneScaleIndices,
			const TLearningArrayView<1, const FVector3f> DefaultBoneLocations,
			const TLearningArrayView<1, const FQuat4f> DefaultBoneRotations,
			const TLearningArrayView<1, const FVector3f> DefaultBoneScales,
			const Learning::FIndexSet BoneIndices)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(AnimDatabase::PoseData::ToPoseVectors);

			const int32 BoneLocationNum = BoneLocationIndices.Num();
			const int32 BoneRotationNum = BoneRotationIndices.Num();
			const int32 BoneScaleNum = BoneScaleIndices.Num();
			const int32 FrameNum = InPoseData.GetFrameNum();
			const int32 AttributeNum = InPoseData.GetAttributeNum();
			const int32 BoneNum = BoneIndices.Num();

			check(InPoseData.GetBoneNum() == BoneNum);
			check(OutPoseVectors.Num<0>() == FrameNum);
			check(OutPoseVectors.Num<1>() == PoseVectorSize(BoneLocationNum, BoneRotationNum, BoneScaleNum, InPoseData.GetAttributeTypes()));
			check(Math::BoneIndicesAreSortedAndUnique(BoneIndices));

			TArray<int32, TInlineAllocator<128>> BoneLocationPoseDataIndices;
			TArray<int32, TInlineAllocator<128>> BoneRotationPoseDataIndices;
			TArray<int32, TInlineAllocator<128>> BoneScalePoseDataIndices;

			BoneLocationPoseDataIndices.SetNumUninitialized(BoneLocationNum);
			BoneRotationPoseDataIndices.SetNumUninitialized(BoneRotationNum);
			BoneScalePoseDataIndices.SetNumUninitialized(BoneScaleNum);

			Math::BoneFindIndicesOf(BoneLocationPoseDataIndices, BoneLocationIndices, BoneIndices);
			Math::BoneFindIndicesOf(BoneRotationPoseDataIndices, BoneRotationIndices, BoneIndices);
			Math::BoneFindIndicesOf(BoneScalePoseDataIndices, BoneScaleIndices, BoneIndices);

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				int32 Offset = 0;

				OutPoseVectors[FrameIdx][0] = FMath::Loge(FMath::Max(InPoseData.RootData.RootScales[FrameIdx].X, UE_SMALL_NUMBER));
				OutPoseVectors[FrameIdx][1] = FMath::Loge(FMath::Max(InPoseData.RootData.RootScales[FrameIdx].Y, UE_SMALL_NUMBER));
				OutPoseVectors[FrameIdx][2] = FMath::Loge(FMath::Max(InPoseData.RootData.RootScales[FrameIdx].Z, UE_SMALL_NUMBER));

				const FVector3f LocalRootLinearVelocity = InPoseData.RootData.RootRotations[FrameIdx].UnrotateVector(InPoseData.RootData.RootLinearVelocities[FrameIdx]);
				const FVector3f LocalRootAngularVelocity = InPoseData.RootData.RootRotations[FrameIdx].UnrotateVector(InPoseData.RootData.RootAngularVelocities[FrameIdx]);

				OutPoseVectors[FrameIdx][3] = LocalRootLinearVelocity.X;
				OutPoseVectors[FrameIdx][4] = LocalRootLinearVelocity.Y;
				OutPoseVectors[FrameIdx][5] = LocalRootLinearVelocity.Z;

				OutPoseVectors[FrameIdx][6] = LocalRootAngularVelocity.X;
				OutPoseVectors[FrameIdx][7] = LocalRootAngularVelocity.Y;
				OutPoseVectors[FrameIdx][8] = LocalRootAngularVelocity.Z;

				OutPoseVectors[FrameIdx][9] = InPoseData.RootData.RootScalarVelocities[FrameIdx].X;
				OutPoseVectors[FrameIdx][10] = InPoseData.RootData.RootScalarVelocities[FrameIdx].Y;
				OutPoseVectors[FrameIdx][11] = InPoseData.RootData.RootScalarVelocities[FrameIdx].Z;

				Offset += 12;

				for (int32 BoneIdx = 0; BoneIdx < BoneLocationNum; BoneIdx++)
				{
					const int32 BoneLocationIdx = BoneLocationPoseDataIndices[BoneIdx];
					const FVector3f BoneLocation = BoneLocationIdx != INDEX_NONE ?
						InPoseData.LocalBoneData.BoneLocations[FrameIdx][BoneLocationIdx] :
						DefaultBoneLocations[BoneLocationIndices[BoneIdx]];

					OutPoseVectors[FrameIdx][Offset + BoneIdx * 3 + 0] = BoneLocation.X;
					OutPoseVectors[FrameIdx][Offset + BoneIdx * 3 + 1] = BoneLocation.Y;
					OutPoseVectors[FrameIdx][Offset + BoneIdx * 3 + 2] = BoneLocation.Z;
				}
				Offset += BoneLocationNum * 3;

				for (int32 BoneIdx = 0; BoneIdx < BoneRotationNum; BoneIdx++)
				{
					const int32 BoneRotationIdx = BoneRotationPoseDataIndices[BoneIdx];
					const FQuat4f BoneRotation = BoneRotationIdx != INDEX_NONE ?
						InPoseData.LocalBoneData.BoneRotations[FrameIdx][BoneRotationIdx] :
						DefaultBoneRotations[BoneRotationIndices[BoneIdx]];

					const FVector3f BoneRotationForward = BoneRotation.RotateVector(FVector3f::ForwardVector);
					const FVector3f BoneRotationRight = BoneRotation.RotateVector(FVector3f::RightVector);

					OutPoseVectors[FrameIdx][Offset + BoneIdx * 6 + 0] = BoneRotationForward.X;
					OutPoseVectors[FrameIdx][Offset + BoneIdx * 6 + 1] = BoneRotationForward.Y;
					OutPoseVectors[FrameIdx][Offset + BoneIdx * 6 + 2] = BoneRotationForward.Z;
					OutPoseVectors[FrameIdx][Offset + BoneIdx * 6 + 3] = BoneRotationRight.X;
					OutPoseVectors[FrameIdx][Offset + BoneIdx * 6 + 4] = BoneRotationRight.Y;
					OutPoseVectors[FrameIdx][Offset + BoneIdx * 6 + 5] = BoneRotationRight.Z;
				}
				Offset += BoneRotationNum * 6;

				for (int32 BoneIdx = 0; BoneIdx < BoneScaleNum; BoneIdx++)
				{
					const int32 BoneScaleIdx = BoneScalePoseDataIndices[BoneIdx];
					const FVector3f BoneScale = BoneScaleIdx != INDEX_NONE ?
						InPoseData.LocalBoneData.BoneScales[FrameIdx][BoneScaleIdx] :
						DefaultBoneScales[BoneScaleIndices[BoneIdx]];

					OutPoseVectors[FrameIdx][Offset + BoneIdx * 3 + 0] = FMath::Loge(FMath::Max(BoneScale.X, UE_SMALL_NUMBER));
					OutPoseVectors[FrameIdx][Offset + BoneIdx * 3 + 1] = FMath::Loge(FMath::Max(BoneScale.Y, UE_SMALL_NUMBER));
					OutPoseVectors[FrameIdx][Offset + BoneIdx * 3 + 2] = FMath::Loge(FMath::Max(BoneScale.Z, UE_SMALL_NUMBER));
				}
				Offset += BoneScaleNum * 3;

				for (int32 BoneIdx = 0; BoneIdx < BoneLocationNum; BoneIdx++)
				{
					const int32 BoneLocationIdx = BoneLocationPoseDataIndices[BoneIdx];
					const FVector3f BoneLinearVelocity = BoneLocationIdx != INDEX_NONE ?
						InPoseData.LocalBoneData.BoneLinearVelocities[FrameIdx][BoneLocationIdx] :
						FVector3f::ZeroVector;

					OutPoseVectors[FrameIdx][Offset + BoneIdx * 3 + 0] = BoneLinearVelocity.X;
					OutPoseVectors[FrameIdx][Offset + BoneIdx * 3 + 1] = BoneLinearVelocity.Y;
					OutPoseVectors[FrameIdx][Offset + BoneIdx * 3 + 2] = BoneLinearVelocity.Z;
				}
				Offset += BoneLocationNum * 3;

				for (int32 BoneIdx = 0; BoneIdx < BoneRotationNum; BoneIdx++)
				{
					const int32 BoneRotationIdx = BoneRotationPoseDataIndices[BoneIdx];
					const FVector3f BoneAngularVelocity = BoneRotationIdx != INDEX_NONE ?
						InPoseData.LocalBoneData.BoneAngularVelocities[FrameIdx][BoneRotationIdx] :
						FVector3f::ZeroVector;

					OutPoseVectors[FrameIdx][Offset + BoneIdx * 3 + 0] = BoneAngularVelocity.X;
					OutPoseVectors[FrameIdx][Offset + BoneIdx * 3 + 1] = BoneAngularVelocity.Y;
					OutPoseVectors[FrameIdx][Offset + BoneIdx * 3 + 2] = BoneAngularVelocity.Z;
				}
				Offset += BoneRotationNum * 3;

				for (int32 BoneIdx = 0; BoneIdx < BoneScaleNum; BoneIdx++)
				{
					const int32 BoneScaleIdx = BoneScalePoseDataIndices[BoneIdx];
					const FVector3f BoneScalarVelocity = BoneScaleIdx != INDEX_NONE ?
						InPoseData.LocalBoneData.BoneScalarVelocities[FrameIdx][BoneScaleIdx] :
						FVector3f::ZeroVector;

					OutPoseVectors[FrameIdx][Offset + BoneIdx * 3 + 0] = BoneScalarVelocity.X;
					OutPoseVectors[FrameIdx][Offset + BoneIdx * 3 + 1] = BoneScalarVelocity.Y;
					OutPoseVectors[FrameIdx][Offset + BoneIdx * 3 + 2] = BoneScalarVelocity.Z;
				}
				Offset += BoneScaleNum * 3;

				for (int32 AttributeIdx = 0; AttributeIdx < AttributeNum; AttributeIdx++)
				{
					const EAnimDatabaseAttributeType AttributeType = InPoseData.AttributeData.GetAttributeType(AttributeIdx);
					const int32 AttributeOffet = InPoseData.AttributeData.GetAttributeOffset(AttributeIdx);
					const int32 AttributeSize = InPoseData.AttributeData.GetAttributeSize(AttributeIdx);
					const int32 PoseVectorFrameAttributeSize = AttributeTypeEncodingSize(AttributeType);

					OutPoseVectors[FrameIdx][Offset + 0] = InPoseData.AttributeData.AttributeActive[FrameIdx][AttributeIdx] ? 1.0 : -1.0f;

					if (InPoseData.AttributeData.AttributeActive[FrameIdx][AttributeIdx])
					{
						switch (AttributeType)
						{
						case EAnimDatabaseAttributeType::Null:
						{
							break;
						}

						case EAnimDatabaseAttributeType::Bool:
						{
							OutPoseVectors[FrameIdx][Offset + 1] = (InPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffet + 0] > 0.0f) ? 1.0f : -1.0f;
							break;
						}

						case EAnimDatabaseAttributeType::Float:
						{
							OutPoseVectors[FrameIdx][Offset + 1] = InPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffet + 0];
							break;
						}

						case EAnimDatabaseAttributeType::Location:
						case EAnimDatabaseAttributeType::LinearVelocity:
						case EAnimDatabaseAttributeType::AngularVelocity:
						case EAnimDatabaseAttributeType::ScalarVelocity:
						case EAnimDatabaseAttributeType::Direction:
						{
							OutPoseVectors[FrameIdx][Offset + 1] = InPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffet + 0];
							OutPoseVectors[FrameIdx][Offset + 2] = InPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffet + 1];
							OutPoseVectors[FrameIdx][Offset + 3] = InPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffet + 2];
							break;
						}

						case EAnimDatabaseAttributeType::Rotation:
						{
							const FQuat4f FrameAttributeRotation = FQuat4f(
								InPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffet + 0],
								InPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffet + 1],
								InPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffet + 2],
								InPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffet + 3]);

							const FVector3f BoneRotationForward = FrameAttributeRotation.RotateVector(FVector3f::ForwardVector);
							const FVector3f BoneRotationRight = FrameAttributeRotation.RotateVector(FVector3f::RightVector);

							OutPoseVectors[FrameIdx][Offset + 1] = BoneRotationForward.X;
							OutPoseVectors[FrameIdx][Offset + 2] = BoneRotationForward.Y;
							OutPoseVectors[FrameIdx][Offset + 3] = BoneRotationForward.Z;
							OutPoseVectors[FrameIdx][Offset + 4] = BoneRotationRight.X;
							OutPoseVectors[FrameIdx][Offset + 5] = BoneRotationRight.Y;
							OutPoseVectors[FrameIdx][Offset + 6] = BoneRotationRight.Z;
							break;
						}

						case EAnimDatabaseAttributeType::Scale:
						{
							OutPoseVectors[FrameIdx][Offset + 1] = FMath::Loge(FMath::Max(InPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffet + 0], UE_SMALL_NUMBER));
							OutPoseVectors[FrameIdx][Offset + 2] = FMath::Loge(FMath::Max(InPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffet + 1], UE_SMALL_NUMBER));
							OutPoseVectors[FrameIdx][Offset + 3] = FMath::Loge(FMath::Max(InPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffet + 2], UE_SMALL_NUMBER));
							break;
						}

						case EAnimDatabaseAttributeType::Transform:
						{
							const FQuat4f FrameAttributeTransform = FQuat4f(
								InPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffet + 3],
								InPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffet + 4],
								InPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffet + 5],
								InPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffet + 6]);

							const FVector3f BoneTransformForward = FrameAttributeTransform.RotateVector(FVector3f::ForwardVector);
							const FVector3f BoneTransformRight = FrameAttributeTransform.RotateVector(FVector3f::RightVector);

							OutPoseVectors[FrameIdx][Offset + 1] = InPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffet + 0];
							OutPoseVectors[FrameIdx][Offset + 2] = InPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffet + 1];
							OutPoseVectors[FrameIdx][Offset + 3] = InPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffet + 2];
							OutPoseVectors[FrameIdx][Offset + 4] = BoneTransformForward.X;
							OutPoseVectors[FrameIdx][Offset + 5] = BoneTransformForward.Y;
							OutPoseVectors[FrameIdx][Offset + 6] = BoneTransformForward.Z;
							OutPoseVectors[FrameIdx][Offset + 7] = BoneTransformRight.X;
							OutPoseVectors[FrameIdx][Offset + 8] = BoneTransformRight.Y;
							OutPoseVectors[FrameIdx][Offset + 9] = BoneTransformRight.Z;
							OutPoseVectors[FrameIdx][Offset + 10] = FMath::Loge(FMath::Max(InPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffet + 7], UE_SMALL_NUMBER));
							OutPoseVectors[FrameIdx][Offset + 11] = FMath::Loge(FMath::Max(InPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffet + 8], UE_SMALL_NUMBER));
							OutPoseVectors[FrameIdx][Offset + 12] = FMath::Loge(FMath::Max(InPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffet + 9], UE_SMALL_NUMBER));
							break;
						}

						case EAnimDatabaseAttributeType::Event:
						{
							const float TimeUntilEventKnown = InPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffet + 0];
							const float TimeUntilEvent = InPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffet + 1];

							if (TimeUntilEventKnown == 0.0f)
							{
								OutPoseVectors[FrameIdx][Offset + 1] = 0.0f;
								OutPoseVectors[FrameIdx][Offset + 2] = -1.0f;
							}
							else
							{
								const float Aprehension = 1.0f; // TODO: Don't hard Code
								const float AprehensionPadding = 0.1f;
								const float ClampedTimeUntilEvent = FMath::Clamp(TimeUntilEvent / (Aprehension + AprehensionPadding), -1.0f, 1.0f);
								OutPoseVectors[FrameIdx][Offset + 1] = FMath::Sin(UE_PI * ClampedTimeUntilEvent);
								OutPoseVectors[FrameIdx][Offset + 2] = FMath::Cos(UE_PI * ClampedTimeUntilEvent);
							}

							break;
						}

						case EAnimDatabaseAttributeType::Angle:
						{
							OutPoseVectors[FrameIdx][Offset + 1] = FMath::Sin(InPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffet + 0]);
							OutPoseVectors[FrameIdx][Offset + 2] = FMath::Cos(InPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffet + 0]);
							break;
						}

						default:
						{
							checkNoEntry();
							break;
						}
						}
					}
					else
					{
						Learning::Array::Zero(OutPoseVectors[FrameIdx].Slice(Offset + 1, PoseVectorFrameAttributeSize - 1));
					}

					Offset += PoseVectorFrameAttributeSize;
				}

				check(Offset == PoseVectorSize(BoneLocationNum, BoneRotationNum, BoneScaleNum, InPoseData.GetAttributeTypes()));
			}
		}

		void FromPoseVectors(
			const FPoseDataView& OutPoseData,
			const TLearningArrayView<2, const float> InPoseVectors,
			const TLearningArrayView<1, const FVector> InRootLocations,
			const TLearningArrayView<1, const FQuat4f> InRootRotations,
			const Learning::FIndexSet BoneLocationIndices,
			const Learning::FIndexSet BoneRotationIndices,
			const Learning::FIndexSet BoneScaleIndices,
			const TLearningArrayView<1, const FVector3f> DefaultBoneLocations,
			const TLearningArrayView<1, const FQuat4f> DefaultBoneRotations,
			const TLearningArrayView<1, const FVector3f> DefaultBoneScales)
		{
			FromPoseVectors(
				OutPoseData,
				InPoseVectors,
				InRootLocations,
				InRootRotations,
				BoneLocationIndices,
				BoneRotationIndices,
				BoneScaleIndices,
				DefaultBoneLocations,
				DefaultBoneRotations,
				DefaultBoneScales,
				Learning::FIndexSet(0, OutPoseData.GetBoneNum()));
		}

		void FromPoseVectors(
			const FPoseDataView& OutPoseData,
			const TLearningArrayView<2, const float> InPoseVectors,
			const TLearningArrayView<1, const FVector> InRootLocations,
			const TLearningArrayView<1, const FQuat4f> InRootRotations,
			const Learning::FIndexSet BoneLocationIndices,
			const Learning::FIndexSet BoneRotationIndices,
			const Learning::FIndexSet BoneScaleIndices,
			const TLearningArrayView<1, const FVector3f> DefaultBoneLocations,
			const TLearningArrayView<1, const FQuat4f> DefaultBoneRotations,
			const TLearningArrayView<1, const FVector3f> DefaultBoneScales,
			const Learning::FIndexSet BoneIndices)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(AnimDatabase::PoseData::FromPoseVectors);

			const int32 BoneLocationNum = BoneLocationIndices.Num();
			const int32 BoneRotationNum = BoneRotationIndices.Num();
			const int32 BoneScaleNum = BoneScaleIndices.Num();
			const int32 FrameNum = InPoseVectors.Num<0>();
			const int32 BoneNum = BoneIndices.Num();
			const int32 AttributeNum = OutPoseData.GetAttributeNum();

			check(OutPoseData.GetFrameNum() == FrameNum);
			check(OutPoseData.GetBoneNum() == BoneNum);
			check(InPoseVectors.Num<1>() == PoseVectorSize(BoneLocationNum, BoneRotationNum, BoneScaleNum, OutPoseData.GetAttributeTypes()));
			check(InRootLocations.Num<0>() == FrameNum);
			check(InRootRotations.Num<0>() == FrameNum);
			check(Math::BoneIndicesAreSortedAndUnique(BoneIndices));

			Learning::Array::Copy(OutPoseData.RootData.RootLocations, InRootLocations);
			Learning::Array::Copy(OutPoseData.RootData.RootRotations, InRootRotations);

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
				{
					OutPoseData.LocalBoneData.BoneLocations[FrameIdx][BoneIdx] = DefaultBoneLocations[BoneIndices[BoneIdx]];
					OutPoseData.LocalBoneData.BoneRotations[FrameIdx][BoneIdx] = DefaultBoneRotations[BoneIndices[BoneIdx]];
					OutPoseData.LocalBoneData.BoneScales[FrameIdx][BoneIdx] = DefaultBoneScales[BoneIndices[BoneIdx]];
				}
			}

			Learning::Array::Zero(OutPoseData.LocalBoneData.BoneLinearVelocities);
			Learning::Array::Zero(OutPoseData.LocalBoneData.BoneAngularVelocities);
			Learning::Array::Zero(OutPoseData.LocalBoneData.BoneScalarVelocities);

			TArray<int32, TInlineAllocator<128>> BoneLocationPoseDataIndices;
			TArray<int32, TInlineAllocator<128>> BoneRotationPoseDataIndices;
			TArray<int32, TInlineAllocator<128>> BoneScalePoseDataIndices;

			BoneLocationPoseDataIndices.SetNumUninitialized(BoneLocationNum);
			BoneRotationPoseDataIndices.SetNumUninitialized(BoneRotationNum);
			BoneScalePoseDataIndices.SetNumUninitialized(BoneScaleNum);

			Math::BoneFindIndicesOf(BoneLocationPoseDataIndices, BoneLocationIndices, BoneIndices);
			Math::BoneFindIndicesOf(BoneRotationPoseDataIndices, BoneRotationIndices, BoneIndices);
			Math::BoneFindIndicesOf(BoneScalePoseDataIndices, BoneScaleIndices, BoneIndices);

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				int32 Offset = 0;

				static constexpr float MaxLogScale = 10.0f;

				OutPoseData.RootData.RootScales[FrameIdx] = FVector3f(
					FMath::Exp(FMath::Min(InPoseVectors[FrameIdx][0], MaxLogScale)),
					FMath::Exp(FMath::Min(InPoseVectors[FrameIdx][1], MaxLogScale)),
					FMath::Exp(FMath::Min(InPoseVectors[FrameIdx][2], MaxLogScale)));

				const FVector3f LocalRootLinearVelocity = FVector3f(InPoseVectors[FrameIdx][3], InPoseVectors[FrameIdx][4], InPoseVectors[FrameIdx][5]);
				const FVector3f LocalRootAngularVelocity = FVector3f(InPoseVectors[FrameIdx][6], InPoseVectors[FrameIdx][7], InPoseVectors[FrameIdx][8]);

				OutPoseData.RootData.RootLinearVelocities[FrameIdx] = OutPoseData.RootData.RootRotations[FrameIdx].RotateVector(LocalRootLinearVelocity);
				OutPoseData.RootData.RootAngularVelocities[FrameIdx] = OutPoseData.RootData.RootRotations[FrameIdx].RotateVector(LocalRootAngularVelocity);
				OutPoseData.RootData.RootScalarVelocities[FrameIdx] = FVector3f(InPoseVectors[FrameIdx][9], InPoseVectors[FrameIdx][10], InPoseVectors[FrameIdx][11]);

				Offset += 12;

#if UE_ANIMDATABASE_ISPC
				ispc::AnimDatabaseFromPoseVectorCopyVector3(
					(float*)OutPoseData.LocalBoneData.BoneLocations[FrameIdx].GetData(),
					InPoseVectors[FrameIdx].GetData(),
					BoneLocationPoseDataIndices.GetData(),
					BoneLocationNum,
					Offset);
#else
				for (int32 BoneIdx = 0; BoneIdx < BoneLocationNum; BoneIdx++)
				{
					const int32 BoneLocationIdx = BoneLocationPoseDataIndices[BoneIdx];

					if (BoneLocationIdx != INDEX_NONE)
					{
						OutPoseData.LocalBoneData.BoneLocations[FrameIdx][BoneLocationIdx].X = InPoseVectors[FrameIdx][Offset + BoneIdx * 3 + 0];
						OutPoseData.LocalBoneData.BoneLocations[FrameIdx][BoneLocationIdx].Y = InPoseVectors[FrameIdx][Offset + BoneIdx * 3 + 1];
						OutPoseData.LocalBoneData.BoneLocations[FrameIdx][BoneLocationIdx].Z = InPoseVectors[FrameIdx][Offset + BoneIdx * 3 + 2];
					}
				}
#endif
				Offset += BoneLocationNum * 3;

#if UE_ANIMDATABASE_ISPC
				ispc::AnimDatabaseFromPoseVectorCopyRotation(
					(float*)OutPoseData.LocalBoneData.BoneRotations[FrameIdx].GetData(),
					InPoseVectors[FrameIdx].GetData(),
					BoneRotationPoseDataIndices.GetData(),
					BoneRotationNum,
					Offset);
#else
				for (int32 BoneIdx = 0; BoneIdx < BoneRotationNum; BoneIdx++)
				{
					const int32 BoneRotationIdx = BoneRotationPoseDataIndices[BoneIdx];

					if (BoneRotationIdx != INDEX_NONE)
					{
						FVector3f BoneRotationForward, BoneRotationRight, BoneRotationUp;

						BoneRotationForward = FVector3f(
							InPoseVectors[FrameIdx][Offset + BoneIdx * 6 + 0],
							InPoseVectors[FrameIdx][Offset + BoneIdx * 6 + 1],
							InPoseVectors[FrameIdx][Offset + BoneIdx * 6 + 2]);

						BoneRotationRight = FVector3f(
							InPoseVectors[FrameIdx][Offset + BoneIdx * 6 + 3],
							InPoseVectors[FrameIdx][Offset + BoneIdx * 6 + 4],
							InPoseVectors[FrameIdx][Offset + BoneIdx * 6 + 5]);

						BoneRotationUp = BoneRotationForward.Cross(BoneRotationRight).GetSafeNormal(UE_SMALL_NUMBER, FVector3f::UpVector);
						BoneRotationRight = BoneRotationUp.Cross(BoneRotationForward).GetSafeNormal(UE_SMALL_NUMBER, FVector3f::RightVector);
						BoneRotationForward = BoneRotationForward.GetSafeNormal(UE_SMALL_NUMBER, FVector3f::ForwardVector);

						FMatrix44f RotationMatrix = FMatrix44f::Identity;
						RotationMatrix.SetAxis(0, BoneRotationForward);
						RotationMatrix.SetAxis(1, BoneRotationRight);
						RotationMatrix.SetAxis(2, BoneRotationUp);

						OutPoseData.LocalBoneData.BoneRotations[FrameIdx][BoneRotationIdx] = RotationMatrix.ToQuat();
					}
				}
#endif

				Offset += BoneRotationNum * 6;

#if UE_ANIMDATABASE_ISPC
				ispc::AnimDatabaseFromPoseVectorCopyScale(
					(float*)OutPoseData.LocalBoneData.BoneScales[FrameIdx].GetData(),
					InPoseVectors[FrameIdx].GetData(),
					BoneScalePoseDataIndices.GetData(),
					BoneScaleNum,
					Offset,
					MaxLogScale);
#else
				for (int32 BoneIdx = 0; BoneIdx < BoneScaleNum; BoneIdx++)
				{
					const int32 BoneScaleIdx = BoneScalePoseDataIndices[BoneIdx];

					if (BoneScaleIdx != INDEX_NONE)
					{
						OutPoseData.LocalBoneData.BoneScales[FrameIdx][BoneScaleIdx].X = FMath::Exp(FMath::Min(InPoseVectors[FrameIdx][Offset + BoneIdx * 3 + 0], MaxLogScale));
						OutPoseData.LocalBoneData.BoneScales[FrameIdx][BoneScaleIdx].Y = FMath::Exp(FMath::Min(InPoseVectors[FrameIdx][Offset + BoneIdx * 3 + 1], MaxLogScale));
						OutPoseData.LocalBoneData.BoneScales[FrameIdx][BoneScaleIdx].Z = FMath::Exp(FMath::Min(InPoseVectors[FrameIdx][Offset + BoneIdx * 3 + 2], MaxLogScale));
					}
				}
#endif

				Offset += BoneScaleNum * 3;

#if UE_ANIMDATABASE_ISPC
				ispc::AnimDatabaseFromPoseVectorCopyVector3(
					(float*)OutPoseData.LocalBoneData.BoneLinearVelocities[FrameIdx].GetData(),
					InPoseVectors[FrameIdx].GetData(),
					BoneLocationPoseDataIndices.GetData(),
					BoneLocationNum,
					Offset);
#else
				for (int32 BoneIdx = 0; BoneIdx < BoneLocationNum; BoneIdx++)
				{
					const int32 BoneLocationIdx = BoneLocationPoseDataIndices[BoneIdx];

					if (BoneLocationIdx != INDEX_NONE)
					{
						OutPoseData.LocalBoneData.BoneLinearVelocities[FrameIdx][BoneLocationIdx].X = InPoseVectors[FrameIdx][Offset + BoneIdx * 3 + 0];
						OutPoseData.LocalBoneData.BoneLinearVelocities[FrameIdx][BoneLocationIdx].Y = InPoseVectors[FrameIdx][Offset + BoneIdx * 3 + 1];
						OutPoseData.LocalBoneData.BoneLinearVelocities[FrameIdx][BoneLocationIdx].Z = InPoseVectors[FrameIdx][Offset + BoneIdx * 3 + 2];
					}
				}
#endif

				Offset += BoneLocationNum * 3;

#if UE_ANIMDATABASE_ISPC
				ispc::AnimDatabaseFromPoseVectorCopyVector3(
					(float*)OutPoseData.LocalBoneData.BoneAngularVelocities[FrameIdx].GetData(),
					InPoseVectors[FrameIdx].GetData(),
					BoneRotationPoseDataIndices.GetData(),
					BoneRotationNum,
					Offset);
#else
				for (int32 BoneIdx = 0; BoneIdx < BoneRotationNum; BoneIdx++)
				{
					const int32 BoneRotationIdx = BoneRotationPoseDataIndices[BoneIdx];

					if (BoneRotationIdx != INDEX_NONE)
					{
						OutPoseData.LocalBoneData.BoneAngularVelocities[FrameIdx][BoneRotationIdx].X = InPoseVectors[FrameIdx][Offset + BoneIdx * 3 + 0];
						OutPoseData.LocalBoneData.BoneAngularVelocities[FrameIdx][BoneRotationIdx].Y = InPoseVectors[FrameIdx][Offset + BoneIdx * 3 + 1];
						OutPoseData.LocalBoneData.BoneAngularVelocities[FrameIdx][BoneRotationIdx].Z = InPoseVectors[FrameIdx][Offset + BoneIdx * 3 + 2];
					}
				}
#endif

				Offset += BoneRotationNum * 3;

#if UE_ANIMDATABASE_ISPC
				ispc::AnimDatabaseFromPoseVectorCopyVector3(
					(float*)OutPoseData.LocalBoneData.BoneScalarVelocities[FrameIdx].GetData(),
					InPoseVectors[FrameIdx].GetData(),
					BoneScalePoseDataIndices.GetData(),
					BoneScaleNum,
					Offset);
#else
				for (int32 BoneIdx = 0; BoneIdx < BoneScaleNum; BoneIdx++)
				{
					const int32 BoneScaleIdx = BoneScalePoseDataIndices[BoneIdx];

					if (BoneScaleIdx != INDEX_NONE)
					{
						OutPoseData.LocalBoneData.BoneScalarVelocities[FrameIdx][BoneScaleIdx].X = InPoseVectors[FrameIdx][Offset + BoneIdx * 3 + 0];
						OutPoseData.LocalBoneData.BoneScalarVelocities[FrameIdx][BoneScaleIdx].Y = InPoseVectors[FrameIdx][Offset + BoneIdx * 3 + 1];
						OutPoseData.LocalBoneData.BoneScalarVelocities[FrameIdx][BoneScaleIdx].Z = InPoseVectors[FrameIdx][Offset + BoneIdx * 3 + 2];
					}
				}
#endif

				Offset += BoneScaleNum * 3;

				for (int32 AttributeIdx = 0; AttributeIdx < AttributeNum; AttributeIdx++)
				{
					const EAnimDatabaseAttributeType AttributeType = OutPoseData.AttributeData.GetAttributeType(AttributeIdx);
					const int32 AttributeOffset = OutPoseData.AttributeData.GetAttributeOffset(AttributeIdx);
					const int32 AttributeSize = OutPoseData.AttributeData.GetAttributeSize(AttributeIdx);
					const int32 PoseVectorFrameAttributeSize = AttributeTypeEncodingSize(AttributeType);

					OutPoseData.AttributeData.AttributeActive[FrameIdx][AttributeIdx] = InPoseVectors[FrameIdx][Offset + 0] > 0.0f;

					if (OutPoseData.AttributeData.AttributeActive[FrameIdx][AttributeIdx])
					{
						switch (AttributeType)
						{
						case EAnimDatabaseAttributeType::Null:
						{
							break;
						}

						case EAnimDatabaseAttributeType::Bool:
						{
							OutPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffset + 0] = InPoseVectors[FrameIdx][Offset + 1] > 0.0f ? 1.0f : 0.0f;
							break;
						}

						case EAnimDatabaseAttributeType::Float:
						{
							OutPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffset + 0] = InPoseVectors[FrameIdx][Offset + 1];
							break;
						}

						case EAnimDatabaseAttributeType::Location:
						case EAnimDatabaseAttributeType::LinearVelocity:
						case EAnimDatabaseAttributeType::AngularVelocity:
						case EAnimDatabaseAttributeType::ScalarVelocity:
						{
							OutPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffset + 0] = InPoseVectors[FrameIdx][Offset + 1];
							OutPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffset + 1] = InPoseVectors[FrameIdx][Offset + 2];
							OutPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffset + 2] = InPoseVectors[FrameIdx][Offset + 3];
							break;
						}

						case EAnimDatabaseAttributeType::Direction:
						{
							const FVector3f FrameAttributeDirection = FVector3f(
								InPoseVectors[FrameIdx][Offset + 1],
								InPoseVectors[FrameIdx][Offset + 2],
								InPoseVectors[FrameIdx][Offset + 3]).GetSafeNormal(UE_SMALL_NUMBER, FVector3f::ForwardVector);

							OutPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffset + 0] = FrameAttributeDirection.X;
							OutPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffset + 1] = FrameAttributeDirection.Y;
							OutPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffset + 2] = FrameAttributeDirection.Z;
							break;
						}

						case EAnimDatabaseAttributeType::Rotation:
						{
							FVector3f FrameAttributeRotationForward, FrameAttributeRotationRight, FrameAttributeRotationUp;

							FrameAttributeRotationForward = FVector3f(
								InPoseVectors[FrameIdx][Offset + 1],
								InPoseVectors[FrameIdx][Offset + 2],
								InPoseVectors[FrameIdx][Offset + 3]);

							FrameAttributeRotationRight = FVector3f(
								InPoseVectors[FrameIdx][Offset + 4],
								InPoseVectors[FrameIdx][Offset + 5],
								InPoseVectors[FrameIdx][Offset + 6]);

							FrameAttributeRotationUp = FrameAttributeRotationForward.Cross(FrameAttributeRotationRight).GetSafeNormal(UE_SMALL_NUMBER, FVector3f::UpVector);
							FrameAttributeRotationRight = FrameAttributeRotationUp.Cross(FrameAttributeRotationForward).GetSafeNormal(UE_SMALL_NUMBER, FVector3f::RightVector);
							FrameAttributeRotationForward = FrameAttributeRotationForward.GetSafeNormal(UE_SMALL_NUMBER, FVector3f::ForwardVector);

							FMatrix44f FrameAttributeRotationMatrix = FMatrix44f::Identity;
							FrameAttributeRotationMatrix.SetAxis(0, FrameAttributeRotationForward);
							FrameAttributeRotationMatrix.SetAxis(1, FrameAttributeRotationRight);
							FrameAttributeRotationMatrix.SetAxis(2, FrameAttributeRotationUp);

							const FQuat4f FrameAttributeRotationQuat = FrameAttributeRotationMatrix.ToQuat();

							OutPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffset + 0] = FrameAttributeRotationQuat.X;
							OutPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffset + 1] = FrameAttributeRotationQuat.Y;
							OutPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffset + 2] = FrameAttributeRotationQuat.Z;
							OutPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffset + 3] = FrameAttributeRotationQuat.W;
							break;
						}

						case EAnimDatabaseAttributeType::Scale:
						{
							OutPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffset + 0] = FMath::Exp(FMath::Min(InPoseVectors[FrameIdx][Offset + 1], MaxLogScale));
							OutPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffset + 1] = FMath::Exp(FMath::Min(InPoseVectors[FrameIdx][Offset + 2], MaxLogScale));
							OutPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffset + 2] = FMath::Exp(FMath::Min(InPoseVectors[FrameIdx][Offset + 3], MaxLogScale));
							break;
						}

						case EAnimDatabaseAttributeType::Transform:
						{
							FVector3f FrameAttributeTransformForward, FrameAttributeTransformRight, FrameAttributeTransformUp;

							FrameAttributeTransformForward = FVector3f(
								InPoseVectors[FrameIdx][Offset + 4],
								InPoseVectors[FrameIdx][Offset + 5],
								InPoseVectors[FrameIdx][Offset + 6]);

							FrameAttributeTransformRight = FVector3f(
								InPoseVectors[FrameIdx][Offset + 7],
								InPoseVectors[FrameIdx][Offset + 8],
								InPoseVectors[FrameIdx][Offset + 9]);

							FrameAttributeTransformUp = FrameAttributeTransformForward.Cross(FrameAttributeTransformRight).GetSafeNormal(UE_SMALL_NUMBER, FVector3f::UpVector);
							FrameAttributeTransformRight = FrameAttributeTransformUp.Cross(FrameAttributeTransformForward).GetSafeNormal(UE_SMALL_NUMBER, FVector3f::RightVector);
							FrameAttributeTransformForward = FrameAttributeTransformForward.GetSafeNormal(UE_SMALL_NUMBER, FVector3f::ForwardVector);

							FMatrix44f FrameAttributeTransformMatrix = FMatrix44f::Identity;
							FrameAttributeTransformMatrix.SetAxis(0, FrameAttributeTransformForward);
							FrameAttributeTransformMatrix.SetAxis(1, FrameAttributeTransformRight);
							FrameAttributeTransformMatrix.SetAxis(2, FrameAttributeTransformUp);

							const FQuat4f FrameAttributeTransformQuat = FrameAttributeTransformMatrix.ToQuat();

							OutPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffset + 0] = InPoseVectors[FrameIdx][Offset + 1];
							OutPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffset + 1] = InPoseVectors[FrameIdx][Offset + 2];
							OutPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffset + 2] = InPoseVectors[FrameIdx][Offset + 3];
							OutPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffset + 3] = FrameAttributeTransformQuat.X;
							OutPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffset + 4] = FrameAttributeTransformQuat.Y;
							OutPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffset + 5] = FrameAttributeTransformQuat.Z;
							OutPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffset + 6] = FrameAttributeTransformQuat.W;
							OutPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffset + 7] = FMath::Exp(FMath::Min(InPoseVectors[FrameIdx][Offset + 10], MaxLogScale));
							OutPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffset + 8] = FMath::Exp(FMath::Min(InPoseVectors[FrameIdx][Offset + 11], MaxLogScale));
							OutPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffset + 9] = FMath::Exp(FMath::Min(InPoseVectors[FrameIdx][Offset + 12], MaxLogScale));
							break;
						}

						case EAnimDatabaseAttributeType::Event:
						{
							const float Aprehension = 1.0f; // TODO: Don't hard Code
							const float AprehensionPadding = 0.1f;
							const float TimeUntilEvent = (Aprehension + AprehensionPadding) * (FMath::Atan2(
								InPoseVectors[FrameIdx][Offset + 1],
								InPoseVectors[FrameIdx][Offset + 2]) / UE_PI);

							if (TimeUntilEvent < +Aprehension && TimeUntilEvent > -Aprehension)
							{
								OutPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffset + 0] = 1.0f;
								OutPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffset + 1] = TimeUntilEvent;
							}
							else
							{
								OutPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffset + 0] = 0.0f;
								OutPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffset + 1] = UE_MAX_FLT;
							}

							break;
						}

						case EAnimDatabaseAttributeType::Angle:
						{
							OutPoseData.AttributeData.AttributeData[FrameIdx][AttributeOffset + 0] = FMath::Atan2(InPoseVectors[FrameIdx][Offset + 1], InPoseVectors[FrameIdx][Offset + 2]);
							break;
						}

						default:
						{
							checkNoEntry();
							break;
						}
						}
					}
					else
					{
						Learning::Array::Zero(OutPoseData.AttributeData.AttributeData[FrameIdx].Slice(AttributeOffset, AttributeSize));
					}

					Offset += PoseVectorFrameAttributeSize;
				}

				check(Offset == PoseVectorSize(BoneLocationNum, BoneRotationNum, BoneScaleNum, OutPoseData.GetAttributeTypes()));
			}
		}

		void FitPoseVectorNormalization(
			TLearningArrayView<1, float> OutPoseVectorOffset,
			TLearningArrayView<1, float> OutPoseVectorScale,
			const TLearningArrayView<2, const float> InPoseVectors,
			const int32 BoneLocationNum,
			const int32 BoneRotationNum,
			const int32 BoneScaleNum,
			const TLearningArrayView<1, const EAnimDatabaseAttributeType> AttributeTypes)
		{
			check(OutPoseVectorOffset.Num() == InPoseVectors.Num<1>());
			check(OutPoseVectorScale.Num() == InPoseVectors.Num<1>());
			check(InPoseVectors.Num<1>() == PoseVectorSize(BoneLocationNum, BoneRotationNum, BoneScaleNum, AttributeTypes));

			UE::Learning::SlicedParallelFor(InPoseVectors.Num<1>(), 16, [&OutPoseVectorOffset, &OutPoseVectorScale, &InPoseVectors](const int32 Start, const int32 Length)
			{
				Math::ComputeMeanStd(
					OutPoseVectorOffset, 
					OutPoseVectorScale, 
					InPoseVectors, 
					Start, Length);
			});

			// Average scales across different value types and max with small epsilon

			const float Epsilon = UE_SMALL_NUMBER;

			int32 Offset = 0;

			float RootLogScaleStdMean = 0.0f;
			Math::ComputeMean(RootLogScaleStdMean, OutPoseVectorScale.Slice(Offset, 3));
			Learning::Array::Set(OutPoseVectorScale.Slice(Offset, 3), FMath::Max(RootLogScaleStdMean, Epsilon));
			Offset += 3;

			float LocalRootLinearVelocityStdMean = 0.0f;
			Math::ComputeMean(LocalRootLinearVelocityStdMean, OutPoseVectorScale.Slice(Offset, 3));
			Learning::Array::Set(OutPoseVectorScale.Slice(Offset, 3), FMath::Max(LocalRootLinearVelocityStdMean, Epsilon));
			Offset += 3;

			float LocalRootAngularVelocityStdMean = 0.0f;
			Math::ComputeMean(LocalRootAngularVelocityStdMean, OutPoseVectorScale.Slice(Offset, 3));
			Learning::Array::Set(OutPoseVectorScale.Slice(Offset, 3), FMath::Max(LocalRootAngularVelocityStdMean, Epsilon));
			Offset += 3;

			float RootScalarVelocityStdMean = 0.0f;
			Math::ComputeMean(RootScalarVelocityStdMean, OutPoseVectorScale.Slice(Offset, 3));
			Learning::Array::Set(OutPoseVectorScale.Slice(Offset, 3), FMath::Max(RootScalarVelocityStdMean, Epsilon));
			Offset += 3;

			float BoneLocationStdMean = 0.0f;
			Math::ComputeMean(BoneLocationStdMean, OutPoseVectorScale.Slice(Offset, BoneLocationNum * 3));
			Learning::Array::Set(OutPoseVectorScale.Slice(Offset, BoneLocationNum * 3), FMath::Max(BoneLocationStdMean, Epsilon));
			Offset += BoneLocationNum * 3;

			float BoneRotationStdMean = 0.0f;
			Math::ComputeMean(BoneRotationStdMean, OutPoseVectorScale.Slice(Offset, BoneRotationNum * 6));
			Learning::Array::Set(OutPoseVectorScale.Slice(Offset, BoneRotationNum * 6), FMath::Max(BoneRotationStdMean, Epsilon));
			Offset += BoneRotationNum * 6;

			float BoneScaleStdMean = 0.0f;
			Math::ComputeMean(BoneScaleStdMean, OutPoseVectorScale.Slice(Offset, BoneScaleNum * 3));
			Learning::Array::Set(OutPoseVectorScale.Slice(Offset, BoneScaleNum * 3), FMath::Max(BoneScaleStdMean, Epsilon));
			Offset += BoneScaleNum * 3;

			float BoneLinearVelocityStdMean = 0.0f;
			Math::ComputeMean(BoneLinearVelocityStdMean, OutPoseVectorScale.Slice(Offset, BoneLocationNum * 3));
			Learning::Array::Set(OutPoseVectorScale.Slice(Offset, BoneLocationNum * 3), FMath::Max(BoneLinearVelocityStdMean, Epsilon));
			Offset += BoneLocationNum * 3;

			float BoneAngularVelocityStdMean = 0.0f;
			Math::ComputeMean(BoneAngularVelocityStdMean, OutPoseVectorScale.Slice(Offset, BoneRotationNum * 3));
			Learning::Array::Set(OutPoseVectorScale.Slice(Offset, BoneRotationNum * 3), FMath::Max(BoneAngularVelocityStdMean, Epsilon));
			Offset += BoneRotationNum * 3;

			float BoneScalarVelocityStdMean = 0.0f;
			Math::ComputeMean(BoneScalarVelocityStdMean, OutPoseVectorScale.Slice(Offset, BoneScaleNum * 3));
			Learning::Array::Set(OutPoseVectorScale.Slice(Offset, BoneScaleNum * 3), FMath::Max(BoneScalarVelocityStdMean, Epsilon));
			Offset += BoneScaleNum * 3;

			// Handle Attributes

			const int32 AttributeNum = AttributeTypes.Num();

			for (int32 AttributeIdx = 0; AttributeIdx < AttributeNum; AttributeIdx++)
			{
				const EAnimDatabaseAttributeType AttributeType = AttributeTypes[AttributeIdx];
				const int32 AttributeEncodingSize = AttributeTypeEncodingSize(AttributeType);
				check(AttributeEncodingSize >= 1);

				// Normalization for attribute active flag is always 0 and 1
				OutPoseVectorOffset[Offset] = 0.0f;
				OutPoseVectorScale[Offset] = 1.0f;
				Offset += 1;

				// Compute average across scales

				switch (AttributeType)
				{
				case EAnimDatabaseAttributeType::Null:
				{
					break;
				}

				case EAnimDatabaseAttributeType::Bool:
				{
					check(AttributeEncodingSize == 1 + 1);

					// Always normalize bools with 1 and 0
					Learning::Array::Set(OutPoseVectorOffset.Slice(Offset, 1), 0.0f);
					Learning::Array::Set(OutPoseVectorScale.Slice(Offset, 1), 1.0f);
					Offset += 1;
					break;
				}

				case EAnimDatabaseAttributeType::Float:
				case EAnimDatabaseAttributeType::Angle:
				case EAnimDatabaseAttributeType::Location:
				case EAnimDatabaseAttributeType::LinearVelocity:
				case EAnimDatabaseAttributeType::AngularVelocity:
				case EAnimDatabaseAttributeType::ScalarVelocity:
				case EAnimDatabaseAttributeType::Direction:
				case EAnimDatabaseAttributeType::Rotation:
				case EAnimDatabaseAttributeType::Scale:
				{
					// Normalization for attribute value is done as normal
					float BoneAttributeStdMean = 0.0f;
					Math::ComputeMean(BoneAttributeStdMean, OutPoseVectorScale.Slice(Offset, AttributeEncodingSize - 1));
					Learning::Array::Set(OutPoseVectorScale.Slice(Offset, AttributeEncodingSize - 1), FMath::Max(BoneAttributeStdMean, Epsilon));
					Offset += AttributeEncodingSize - 1;
					break;
				}

				case EAnimDatabaseAttributeType::Transform:
				{
					// Need to normalize transform attribute in parts
					check(AttributeEncodingSize == 1 + 3 + 6 + 3);

					float BoneAttributeLocationStdMean = 0.0f;
					Math::ComputeMean(BoneAttributeLocationStdMean, OutPoseVectorScale.Slice(Offset, 3));
					Learning::Array::Set(OutPoseVectorScale.Slice(Offset, 3), FMath::Max(BoneAttributeLocationStdMean, Epsilon));
					Offset += 3;

					float BoneAttributeRotationStdMean = 0.0f;
					Math::ComputeMean(BoneAttributeRotationStdMean, OutPoseVectorScale.Slice(Offset, 6));
					Learning::Array::Set(OutPoseVectorScale.Slice(Offset, 6), FMath::Max(BoneAttributeRotationStdMean, Epsilon));
					Offset += 6;

					float BoneAttributeScaleStdMean = 0.0f;
					Math::ComputeMean(BoneAttributeScaleStdMean, OutPoseVectorScale.Slice(Offset, 3));
					Learning::Array::Set(OutPoseVectorScale.Slice(Offset, 3), FMath::Max(BoneAttributeScaleStdMean, Epsilon));
					Offset += 3;

					break;
				}

				case EAnimDatabaseAttributeType::Event:
				{
					check(AttributeEncodingSize == 1 + 2);

					// Always normalize events with just 0 and 1
					Learning::Array::Set(OutPoseVectorOffset.Slice(Offset, 2), 0.0f);
					Learning::Array::Set(OutPoseVectorScale.Slice(Offset, 2), 1.0f);
					Offset += 2;
					break;
				}

				default:
				{
					checkNoEntry();
					break;
				}
				}
			}

			check(Offset == PoseVectorSize(BoneLocationNum, BoneRotationNum, BoneScaleNum, AttributeTypes));
		}

		void ComputePoseVectorWeightsFromDescendantBoneLengths(
			TLearningArrayView<1, float> OutPoseVectorWeights,
			const TLearningArrayView<1, const FTransform> ReferenceTransforms,
			const TLearningArrayView<1, const int32> BoneParents,
			const Learning::FIndexSet BoneLocationIndices,
			const Learning::FIndexSet BoneRotationIndices,
			const Learning::FIndexSet BoneScaleIndices,
			const float RootWeight)
		{
			Learning::Array::Set(OutPoseVectorWeights, 1.0f);
			Learning::Array::Set(OutPoseVectorWeights.Slice(0, 12), RootWeight);

			const int32 BoneNum = ReferenceTransforms.Num();
			const int32 BoneLocationNum = BoneLocationIndices.Num();
			const int32 BoneRotationNum = BoneRotationIndices.Num();
			const int32 BoneScaleNum = BoneScaleIndices.Num();

			check(BoneNum == BoneParents.Num());

			// Assume all joints have a cylinder attached with 4cm diameter and 6cm height, which is 10cm away
			const float BaseCylinderSurfaceArea = UE_PI * 6.0f * 4.0f;
			const float BaseCylinderDistance = 10.0f;

			const float Discount = 0.9f;

			TLearningArray<1, float> BoneWeights;
			BoneWeights.SetNumUninitialized({ BoneNum });
			Learning::Array::Set(BoneWeights, BaseCylinderSurfaceArea * BaseCylinderDistance);

			for (int32 BoneIdx = BoneNum - 1; BoneIdx >= 0; BoneIdx--)
			{
				check(BoneParents[BoneIdx] < BoneIdx);
				if (BoneParents[BoneIdx] != INDEX_NONE)
				{
					// Assume that the diameter of the cylinder is approximately 1/3 of the bone length
					const float CylinderDiameterFromBoneLength = 0.33f;

					// Approximate area with cylinder (minus caps) where diameter is half of the height
					const float BoneLength = ReferenceTransforms[BoneIdx].GetLocation().Length();
					const float CylinderHeight = BoneLength;
					const float CylinderDiameter = BoneLength * CylinderDiameterFromBoneLength;

					BoneWeights[BoneParents[BoneIdx]] += Discount * (CylinderHeight * CylinderDiameter + BoneWeights[BoneIdx]);
				}
			}

			// Find Location Weights

			TLearningArray<1, float> BoneLocationWeights;
			BoneLocationWeights.SetNumUninitialized({ BoneLocationNum });

			float TotalLocationWeight = UE_KINDA_SMALL_NUMBER;
			for (int32 LocBoneIdx = 0; LocBoneIdx < BoneLocationNum; LocBoneIdx++)
			{
				BoneLocationWeights[LocBoneIdx] = BoneWeights[BoneLocationIndices[LocBoneIdx]];
				TotalLocationWeight += BoneLocationWeights[LocBoneIdx];
			}

			for (int32 LocBoneIdx = 0; LocBoneIdx < BoneLocationNum; LocBoneIdx++)
			{
				BoneLocationWeights[LocBoneIdx] = BoneLocationNum * (BoneLocationWeights[LocBoneIdx] / TotalLocationWeight);
			}

			// Find Rotation Weights

			TLearningArray<1, float> BoneRotationWeights;
			BoneRotationWeights.SetNumUninitialized({ BoneRotationNum });

			float TotalRotationWeight = UE_KINDA_SMALL_NUMBER;
			for (int32 RotBoneIdx = 0; RotBoneIdx < BoneRotationNum; RotBoneIdx++)
			{
				BoneRotationWeights[RotBoneIdx] = BoneWeights[BoneRotationIndices[RotBoneIdx]];
				TotalRotationWeight += BoneRotationWeights[RotBoneIdx];
			}

			for (int32 RotBoneIdx = 0; RotBoneIdx < BoneRotationNum; RotBoneIdx++)
			{
				BoneRotationWeights[RotBoneIdx] = BoneRotationNum * (BoneRotationWeights[RotBoneIdx] / TotalRotationWeight);
			}

			// Insert into pose vector

			const int32 BoneLocationOffset = 12;
			for (int32 BoneIdx = 0; BoneIdx < BoneLocationNum; BoneIdx++)
			{
				Learning::Array::Set(OutPoseVectorWeights.Slice(BoneLocationOffset + BoneIdx * 3, 3), BoneLocationWeights[BoneIdx]);
			}

			const int32 BoneRotationOffset = 12 + BoneLocationNum * 3;
			for (int32 BoneIdx = 0; BoneIdx < BoneRotationNum; BoneIdx++)
			{
				Learning::Array::Set(OutPoseVectorWeights.Slice(BoneRotationOffset + BoneIdx * 6, 6), BoneRotationWeights[BoneIdx]);
			}

			const int32 BoneLinearVelocityOffset = 12 + BoneLocationNum * 3 + BoneRotationNum * 6 + BoneScaleNum * 3;
			for (int32 BoneIdx = 0; BoneIdx < BoneLocationNum; BoneIdx++)
			{
				Learning::Array::Set(OutPoseVectorWeights.Slice(BoneLinearVelocityOffset + BoneIdx * 3, 3), BoneLocationWeights[BoneIdx]);
			}

			const int32 BoneAngularVelocityOffset = 12 + BoneLocationNum * 3 + BoneRotationNum * 6 + BoneScaleNum * 3 + BoneLocationNum * 3;
			for (int32 BoneIdx = 0; BoneIdx < BoneRotationNum; BoneIdx++)
			{
				Learning::Array::Set(OutPoseVectorWeights.Slice(BoneAngularVelocityOffset + BoneIdx * 3, 3), BoneRotationWeights[BoneIdx]);
			}
		}

		void ComputePoseVectorWeightsCylinderApprox(
			TLearningArrayView<1, float> OutPoseVectorWeights,
			const TLearningArrayView<1, const FTransform> ReferenceTransforms,
			const TLearningArrayView<1, const int32> BoneParents,
			const Learning::FIndexSet RequiredIndices,
			const Learning::FIndexSet BoneLocationIndices,
			const Learning::FIndexSet BoneRotationIndices,
			const Learning::FIndexSet BoneScaleIndices,
			const float RootWeight,
			const float BoneBaseWeightMultiplier,
			const float AttributeWeight)
		{
			Learning::Array::Set(OutPoseVectorWeights, 1.0f);
			Learning::Array::Set(OutPoseVectorWeights.Slice(0, 12), RootWeight);

			const int32 BoneNum = ReferenceTransforms.Num();
			const int32 BoneLocationNum = BoneLocationIndices.Num();
			const int32 BoneRotationNum = BoneRotationIndices.Num();
			const int32 BoneScaleNum = BoneScaleIndices.Num();

			check(BoneNum == BoneParents.Num());

			// Compute Global Bone Reference Transforms

			TLearningArray<1, FTransform> GlobalReferenceTransforms;
			GlobalReferenceTransforms.SetNumUninitialized({ BoneNum });
			Learning::Array::Copy(GlobalReferenceTransforms, ReferenceTransforms);
			for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
			{
				if (BoneParents[BoneIdx] != INDEX_NONE)
				{
					GlobalReferenceTransforms[BoneIdx] = GlobalReferenceTransforms[BoneIdx] * GlobalReferenceTransforms[BoneParents[BoneIdx]];
				}
			}

			// Find Surface area attached to each bone

			TLearningArray<1, float> BoneSurfaceAreas;
			BoneSurfaceAreas.SetNumUninitialized({ BoneNum });
			Learning::Array::Zero(BoneSurfaceAreas);

			TLearningArray<1, float> BoneAverageChildDistances;
			BoneAverageChildDistances.SetNumUninitialized({ BoneNum });
			Learning::Array::Zero(BoneAverageChildDistances);

			for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
			{
				int32 ActiveChildNum = 0;

				for (int32 ChildIdx = 0; ChildIdx < BoneNum; ChildIdx++)
				{
					if (BoneParents[ChildIdx] == BoneIdx)
					{
						// Assume that the diameter of the cylinder is approximately 1/3 of the bone length
						const float CylinderDiameterFromBoneLength = 0.33f;

						const float BoneLength = ReferenceTransforms[ChildIdx].GetLocation().Length();
						const float CylinderHeight = BoneLength;
						const float CylinderDiameter = BoneLength * CylinderDiameterFromBoneLength;
						BoneSurfaceAreas[BoneIdx] += UE_PI * CylinderHeight * CylinderDiameter;
						BoneAverageChildDistances[BoneIdx] += BoneLength / 2.0f;
						ActiveChildNum++;
					}
				}

				if (ActiveChildNum > 0)
				{
					BoneAverageChildDistances[BoneIdx] /= ActiveChildNum;
				}
			}

			// Compute Descendants Matrix 

			TLearningArray<2, bool> BoneDescendants;
			BoneDescendants.SetNumUninitialized({ BoneNum, BoneNum });
			Math::BoneDescedantsMatrix(BoneDescendants, BoneParents);

			// Assume all joints have a cylinder attached with 10cm diameter and 20cm height, which is 10cm away
			const float BaseCylinderHeight = 20.0f;
			const float BaseCylinderDiameter = 10.0f;
			const float BaseCylinderSurfaceArea = UE_PI * BaseCylinderHeight * BaseCylinderDiameter;
			const float BaseCylinderDistance = 10.0f;

			// Find Location Weights

			TLearningArray<1, float> BoneLocationWeights;
			BoneLocationWeights.SetNumUninitialized({ BoneLocationNum });
			float BoneLocationWeightTotal = 0.0f;

			for (int32 LocBoneIdx = 0; LocBoneIdx < BoneLocationNum; LocBoneIdx++)
			{
				const int32 BoneIdx = BoneLocationIndices[LocBoneIdx];
				BoneLocationWeights[LocBoneIdx] = BoneBaseWeightMultiplier * BaseCylinderSurfaceArea;

				for (int32 DescIdx = 0; DescIdx < BoneNum; DescIdx++)
				{
					if (BoneIdx == DescIdx || (BoneDescendants[BoneIdx][DescIdx] && RequiredIndices.Contains(DescIdx)))
					{
						BoneLocationWeights[LocBoneIdx] += BoneSurfaceAreas[DescIdx];
					}
				}

				BoneLocationWeightTotal += BoneLocationWeights[LocBoneIdx];
			}

			for (int32 LocBoneIdx = 0; LocBoneIdx < BoneLocationNum; LocBoneIdx++)
			{
				BoneLocationWeights[LocBoneIdx] = BoneLocationNum * (BoneLocationWeights[LocBoneIdx] / FMath::Max(BoneLocationWeightTotal, UE_SMALL_NUMBER));
			}

			// Find Rotation Weights

			TLearningArray<1, float> BoneRotationWeights;
			BoneRotationWeights.SetNumUninitialized({ BoneRotationNum });
			float BoneRotationWeightTotal = 0.0f;

			for (int32 RotBoneIdx = 0; RotBoneIdx < BoneRotationNum; RotBoneIdx++)
			{
				const int32 BoneIdx = BoneRotationIndices[RotBoneIdx];
				BoneRotationWeights[RotBoneIdx] = BoneBaseWeightMultiplier * BaseCylinderDistance * BaseCylinderSurfaceArea;

				for (int32 DescIdx = 0; DescIdx < BoneNum; DescIdx++)
				{
					if (BoneIdx == DescIdx || (BoneDescendants[BoneIdx][DescIdx] && RequiredIndices.Contains(DescIdx)))
					{
						const float Distance = FVector::Distance(
							GlobalReferenceTransforms[BoneIdx].GetLocation(),
							GlobalReferenceTransforms[DescIdx].GetLocation()) + BoneAverageChildDistances[DescIdx];

						BoneRotationWeights[RotBoneIdx] += Distance * BoneSurfaceAreas[DescIdx];
					}
				}

				BoneRotationWeightTotal += BoneRotationWeights[RotBoneIdx];
			}

			for (int32 RotBoneIdx = 0; RotBoneIdx < BoneRotationNum; RotBoneIdx++)
			{
				BoneRotationWeights[RotBoneIdx] = BoneRotationNum * (BoneRotationWeights[RotBoneIdx] / FMath::Max(BoneRotationWeightTotal, UE_SMALL_NUMBER));
			}

			// Insert into pose vector

			const int32 BoneLocationOffset = 12;
			for (int32 BoneIdx = 0; BoneIdx < BoneLocationNum; BoneIdx++)
			{
				Learning::Array::Set(OutPoseVectorWeights.Slice(BoneLocationOffset + BoneIdx * 3, 3), BoneLocationWeights[BoneIdx]);
			}

			const int32 BoneRotationOffset = 12 + BoneLocationNum * 3;
			for (int32 BoneIdx = 0; BoneIdx < BoneRotationNum; BoneIdx++)
			{
				Learning::Array::Set(OutPoseVectorWeights.Slice(BoneRotationOffset + BoneIdx * 6, 6), BoneRotationWeights[BoneIdx]);
			}

			const int32 BoneLinearVelocityOffset = 12 + BoneLocationNum * 3 + BoneRotationNum * 6 + BoneScaleNum * 3;
			for (int32 BoneIdx = 0; BoneIdx < BoneLocationNum; BoneIdx++)
			{
				Learning::Array::Set(OutPoseVectorWeights.Slice(BoneLinearVelocityOffset + BoneIdx * 3, 3), BoneLocationWeights[BoneIdx]);
			}

			const int32 BoneAngularVelocityOffset = 12 + BoneLocationNum * 3 + BoneRotationNum * 6 + BoneScaleNum * 3 + BoneLocationNum * 3;
			for (int32 BoneIdx = 0; BoneIdx < BoneRotationNum; BoneIdx++)
			{
				Learning::Array::Set(OutPoseVectorWeights.Slice(BoneAngularVelocityOffset + BoneIdx * 3, 3), BoneRotationWeights[BoneIdx]);
			}

			const int32 AttributeOffset = 12 + BoneLocationNum * 3 + BoneRotationNum * 6 + BoneScaleNum * 3 + BoneLocationNum * 3 + BoneRotationNum * 3 + BoneScaleNum * 3;
			Learning::Array::Set(OutPoseVectorWeights.Slice(AttributeOffset, OutPoseVectorWeights.Num() - AttributeOffset), AttributeWeight);
		}

		void MakeLooped(
			const FPoseDataView& InOutPoseData, 
			const FFrameRate FrameRate,
			const float StartEndRatio,
			const float BlendInTime,
			const float BlendOutTime)
		{
			const int32 FrameNum = InOutPoseData.GetFrameNum();
			const int32 BoneNum = InOutPoseData.GetBoneNum();
			check(FrameNum >= 1);

			const float DeltaTime = 1.0f / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER);
			const float InTime = FMath::Clamp(BlendInTime, 0.0f, FrameNum * DeltaTime / 2.0f);
			const float OutTime = FMath::Clamp(BlendOutTime, 0.0f, FrameNum * DeltaTime / 2.0f);
			const float StartAmount = 1.0f - StartEndRatio;
			const float EndAmount = StartEndRatio;

			FVector3f LocationOffsetStart = FVector3f::ZeroVector;
			FVector3f LinearVelocityOffsetStart = FVector3f::ZeroVector;
			FVector3f RotationOffsetStart = FVector3f::ZeroVector;
			FVector3f AngularVelocityOffsetStart = FVector3f::ZeroVector;
			FVector3f ScaleOffsetStart = FVector3f::ZeroVector;
			FVector3f ScalarVelocityOffsetStart = FVector3f::ZeroVector;

			FVector3f LocationOffsetEnd = FVector3f::ZeroVector;
			FVector3f LinearVelocityOffsetEnd = FVector3f::ZeroVector;
			FVector3f RotationOffsetEnd = FVector3f::ZeroVector;
			FVector3f AngularVelocityOffsetEnd = FVector3f::ZeroVector;
			FVector3f ScaleOffsetEnd = FVector3f::ZeroVector;
			FVector3f ScalarVelocityOffsetEnd = FVector3f::ZeroVector;

			/* Loop Root */

			const FQuat4f RootStartRotation = InOutPoseData.RootData.RootRotations[0];
			const FQuat4f RootEndRotation = InOutPoseData.RootData.RootRotations[FrameNum - 1];

			const FVector3f RootLocalLinearVelocityOffset = 
				RootEndRotation.UnrotateVector(InOutPoseData.RootData.RootLinearVelocities[FrameNum - 1]) -
				RootStartRotation.UnrotateVector(InOutPoseData.RootData.RootLinearVelocities[0]);

			const FVector3f RootLocalAngularVelocityOffset =
				RootEndRotation.UnrotateVector(InOutPoseData.RootData.RootAngularVelocities[FrameNum - 1]) -
				RootStartRotation.UnrotateVector(InOutPoseData.RootData.RootAngularVelocities[0]);

			const FVector3f RootScaleOffset = Math::VectorLogSafe(Math::VectorDivMax(
				InOutPoseData.RootData.RootScales[FrameNum - 1],
				InOutPoseData.RootData.RootScales[0]));

			const FVector3f RootScalarVelocityOffset =
				InOutPoseData.RootData.RootScalarVelocities[FrameNum - 1] - 
				InOutPoseData.RootData.RootScalarVelocities[0];

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				Math::DecayCubic(LocationOffsetStart, LinearVelocityOffsetStart, FVector3f::ZeroVector, StartAmount * RootStartRotation.RotateVector(RootLocalLinearVelocityOffset), FrameIdx * DeltaTime, InTime);
				Math::DecayCubic(RotationOffsetStart, AngularVelocityOffsetStart, FVector3f::ZeroVector, StartAmount * RootStartRotation.RotateVector(RootLocalAngularVelocityOffset), FrameIdx * DeltaTime, InTime);
				Math::DecayCubic(ScaleOffsetStart, ScalarVelocityOffsetStart, StartAmount * RootScaleOffset, StartAmount * RootScalarVelocityOffset, FrameIdx * DeltaTime, InTime);

				Math::DecayCubic(LocationOffsetEnd, LinearVelocityOffsetEnd, FVector3f::ZeroVector, EndAmount * RootEndRotation.RotateVector(RootLocalLinearVelocityOffset), (FrameNum - 1 - FrameIdx) * DeltaTime, OutTime);
				Math::DecayCubic(RotationOffsetEnd, AngularVelocityOffsetEnd, FVector3f::ZeroVector, EndAmount * RootEndRotation.RotateVector(RootLocalAngularVelocityOffset), (FrameNum - 1 - FrameIdx) * DeltaTime, OutTime);
				Math::DecayCubic(ScaleOffsetEnd, ScalarVelocityOffsetEnd, EndAmount * -RootScaleOffset, EndAmount * RootScalarVelocityOffset, (FrameNum - 1 - FrameIdx) * DeltaTime, OutTime);

				InOutPoseData.RootData.RootLocations[FrameIdx] = InOutPoseData.RootData.RootLocations[FrameIdx] + (FVector)(LocationOffsetStart + LocationOffsetEnd);
				InOutPoseData.RootData.RootLinearVelocities[FrameIdx] = InOutPoseData.RootData.RootLinearVelocities[FrameIdx] + (LinearVelocityOffsetStart + LinearVelocityOffsetEnd);
				InOutPoseData.RootData.RootRotations[FrameIdx] = FQuat4f::MakeFromRotationVector(RotationOffsetStart + RotationOffsetEnd) * InOutPoseData.RootData.RootRotations[FrameIdx];
				InOutPoseData.RootData.RootAngularVelocities[FrameIdx] = InOutPoseData.RootData.RootAngularVelocities[FrameIdx] + (AngularVelocityOffsetStart + AngularVelocityOffsetEnd);
				InOutPoseData.RootData.RootScales[FrameIdx] = Math::VectorExpSafe(ScaleOffsetStart + ScaleOffsetEnd) * InOutPoseData.RootData.RootScales[FrameIdx];
				InOutPoseData.RootData.RootScalarVelocities[FrameIdx] = InOutPoseData.RootData.RootScalarVelocities[FrameIdx] + (ScalarVelocityOffsetStart + ScalarVelocityOffsetEnd);
			}

			/* Loop Transforms */

			TLearningArray<1, FVector3f> BoneLocationOffset;
			TLearningArray<1, FVector3f> BoneLinearVelocityOffset;
			TLearningArray<1, FVector3f> BoneRotationOffset;
			TLearningArray<1, FVector3f> BoneAngularVelocityOffset;
			TLearningArray<1, FVector3f> BoneScaleOffset;
			TLearningArray<1, FVector3f> BoneScalarVelocityOffset;

			BoneLocationOffset.SetNumUninitialized({ BoneNum });
			BoneLinearVelocityOffset.SetNumUninitialized({ BoneNum });
			BoneRotationOffset.SetNumUninitialized({ BoneNum });
			BoneAngularVelocityOffset.SetNumUninitialized({ BoneNum });
			BoneScaleOffset.SetNumUninitialized({ BoneNum });
			BoneScalarVelocityOffset.SetNumUninitialized({ BoneNum });

			for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
			{
				BoneLocationOffset[BoneIdx] = InOutPoseData.LocalBoneData.BoneLocations[FrameNum - 1][BoneIdx] - InOutPoseData.LocalBoneData.BoneLocations[0][BoneIdx];
				BoneLinearVelocityOffset[BoneIdx] = InOutPoseData.LocalBoneData.BoneLinearVelocities[FrameNum - 1][BoneIdx] - InOutPoseData.LocalBoneData.BoneLinearVelocities[0][BoneIdx];

				BoneRotationOffset[BoneIdx] = (InOutPoseData.LocalBoneData.BoneRotations[FrameNum - 1][BoneIdx] * InOutPoseData.LocalBoneData.BoneRotations[0][BoneIdx].Inverse()).GetShortestArcWith(FQuat4f::Identity).ToRotationVector();
				BoneAngularVelocityOffset[BoneIdx] = InOutPoseData.LocalBoneData.BoneAngularVelocities[FrameNum - 1][BoneIdx] - InOutPoseData.LocalBoneData.BoneAngularVelocities[0][BoneIdx];

				BoneScaleOffset[BoneIdx] = Math::VectorLogSafe(Math::VectorDivMax(InOutPoseData.LocalBoneData.BoneScales[FrameNum - 1][BoneIdx], InOutPoseData.LocalBoneData.BoneScales[0][BoneIdx]));
				BoneScalarVelocityOffset[BoneIdx] = InOutPoseData.LocalBoneData.BoneScalarVelocities[FrameNum - 1][BoneIdx] - InOutPoseData.LocalBoneData.BoneScalarVelocities[0][BoneIdx];
			}

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
				{
					Math::DecayCubic(LocationOffsetStart, LinearVelocityOffsetStart, StartAmount * BoneLocationOffset[BoneIdx], StartAmount * BoneLinearVelocityOffset[BoneIdx], FrameIdx * DeltaTime, InTime);
					Math::DecayCubic(RotationOffsetStart, AngularVelocityOffsetStart, StartAmount * BoneRotationOffset[BoneIdx], StartAmount * BoneAngularVelocityOffset[BoneIdx], FrameIdx * DeltaTime, InTime);
					Math::DecayCubic(ScaleOffsetStart, ScalarVelocityOffsetStart, StartAmount * BoneScaleOffset[BoneIdx], StartAmount * BoneScalarVelocityOffset[BoneIdx], FrameIdx * DeltaTime, InTime);

					Math::DecayCubic(LocationOffsetEnd, LinearVelocityOffsetEnd, EndAmount * -BoneLocationOffset[BoneIdx], EndAmount * BoneLinearVelocityOffset[BoneIdx], (FrameNum - 1 - FrameIdx) * DeltaTime, OutTime);
					Math::DecayCubic(RotationOffsetEnd, AngularVelocityOffsetEnd, EndAmount * -BoneRotationOffset[BoneIdx], EndAmount * BoneAngularVelocityOffset[BoneIdx], (FrameNum - 1 - FrameIdx) * DeltaTime, OutTime);
					Math::DecayCubic(ScaleOffsetEnd, ScalarVelocityOffsetEnd, EndAmount * -BoneScaleOffset[BoneIdx], EndAmount * BoneScalarVelocityOffset[BoneIdx], (FrameNum - 1 - FrameIdx) * DeltaTime, OutTime);

					InOutPoseData.LocalBoneData.BoneLocations[FrameIdx][BoneIdx] = InOutPoseData.LocalBoneData.BoneLocations[FrameIdx][BoneIdx] + (LocationOffsetStart + LocationOffsetEnd);
					InOutPoseData.LocalBoneData.BoneLinearVelocities[FrameIdx][BoneIdx] = InOutPoseData.LocalBoneData.BoneLinearVelocities[FrameIdx][BoneIdx] + (LinearVelocityOffsetStart + LinearVelocityOffsetEnd);
					InOutPoseData.LocalBoneData.BoneRotations[FrameIdx][BoneIdx] = FQuat4f::MakeFromRotationVector(RotationOffsetStart + RotationOffsetEnd) * InOutPoseData.LocalBoneData.BoneRotations[FrameIdx][BoneIdx];
					InOutPoseData.LocalBoneData.BoneAngularVelocities[FrameIdx][BoneIdx] = InOutPoseData.LocalBoneData.BoneAngularVelocities[FrameIdx][BoneIdx] + (AngularVelocityOffsetStart + AngularVelocityOffsetEnd);
					InOutPoseData.LocalBoneData.BoneScales[FrameIdx][BoneIdx] = Math::VectorExpSafe(ScaleOffsetStart + ScaleOffsetEnd) * InOutPoseData.LocalBoneData.BoneScales[FrameIdx][BoneIdx];
					InOutPoseData.LocalBoneData.BoneScalarVelocities[FrameIdx][BoneIdx] = InOutPoseData.LocalBoneData.BoneScalarVelocities[FrameIdx][BoneIdx] + (ScalarVelocityOffsetStart + ScalarVelocityOffsetEnd);
				}
			}
		}

		void PatchInterpolate(const FPoseDataView& InOutPoseData, const FFrameRate FrameRate, const bool bApplyToRoot, const UE::Learning::FIndexSet Bones)
		{
			const int32 FrameNum = InOutPoseData.GetFrameNum();
			check(FrameNum >= 1);

			const float DeltaTime = 1.0f / FMath::Max(FrameRate.AsDecimal(), UE_SMALL_NUMBER);

			if (bApplyToRoot)
			{
				for (int32 FrameIdx = 1; FrameIdx < FrameNum - 1; FrameIdx++)
				{
					Math::HermiteLocationInterpolate(
						InOutPoseData.RootData.RootLocations[FrameIdx],
						InOutPoseData.RootData.RootLinearVelocities[FrameIdx],
						InOutPoseData.RootData.RootLocations[0],
						InOutPoseData.RootData.RootLocations[FrameNum - 1],
						InOutPoseData.RootData.RootLinearVelocities[0],
						InOutPoseData.RootData.RootLinearVelocities[FrameNum - 1],
						(float)FrameIdx / (FrameNum - 1),
						DeltaTime);

					Math::HermiteRotationInterpolate(
						InOutPoseData.RootData.RootRotations[FrameIdx],
						InOutPoseData.RootData.RootAngularVelocities[FrameIdx],
						InOutPoseData.RootData.RootRotations[0],
						InOutPoseData.RootData.RootRotations[FrameNum - 1],
						InOutPoseData.RootData.RootAngularVelocities[0],
						InOutPoseData.RootData.RootAngularVelocities[FrameNum - 1],
						(float)FrameIdx / (FrameNum - 1),
						DeltaTime);

					Math::HermiteScaleInterpolate(
						InOutPoseData.RootData.RootScales[FrameIdx],
						InOutPoseData.RootData.RootScalarVelocities[FrameIdx],
						InOutPoseData.RootData.RootScales[0],
						InOutPoseData.RootData.RootScales[FrameNum - 1],
						InOutPoseData.RootData.RootScalarVelocities[0],
						InOutPoseData.RootData.RootScalarVelocities[FrameNum - 1],
						(float)FrameIdx / (FrameNum - 1),
						DeltaTime);
				}
			}

			for (int32 FrameIdx = 1; FrameIdx < FrameNum - 1; FrameIdx++)
			{
				for (int32 BoneIdx : Bones)
				{
					if (BoneIdx == INDEX_NONE) { continue; }

					Math::HermiteLocationInterpolate(
						InOutPoseData.LocalBoneData.BoneLocations[FrameIdx][BoneIdx],
						InOutPoseData.LocalBoneData.BoneLinearVelocities[FrameIdx][BoneIdx],
						InOutPoseData.LocalBoneData.BoneLocations[0][BoneIdx],
						InOutPoseData.LocalBoneData.BoneLocations[FrameNum - 1][BoneIdx],
						InOutPoseData.LocalBoneData.BoneLinearVelocities[0][BoneIdx],
						InOutPoseData.LocalBoneData.BoneLinearVelocities[FrameNum - 1][BoneIdx],
						(float)FrameIdx / (FrameNum - 1),
						DeltaTime);

					Math::HermiteRotationInterpolate(
						InOutPoseData.LocalBoneData.BoneRotations[FrameIdx][BoneIdx],
						InOutPoseData.LocalBoneData.BoneAngularVelocities[FrameIdx][BoneIdx],
						InOutPoseData.LocalBoneData.BoneRotations[0][BoneIdx],
						InOutPoseData.LocalBoneData.BoneRotations[FrameNum - 1][BoneIdx],
						InOutPoseData.LocalBoneData.BoneAngularVelocities[0][BoneIdx],
						InOutPoseData.LocalBoneData.BoneAngularVelocities[FrameNum - 1][BoneIdx],
						(float)FrameIdx / (FrameNum - 1),
						DeltaTime);

					Math::HermiteScaleInterpolate(
						InOutPoseData.LocalBoneData.BoneScales[FrameIdx][BoneIdx],
						InOutPoseData.LocalBoneData.BoneScalarVelocities[FrameIdx][BoneIdx],
						InOutPoseData.LocalBoneData.BoneScales[0][BoneIdx],
						InOutPoseData.LocalBoneData.BoneScales[FrameNum - 1][BoneIdx],
						InOutPoseData.LocalBoneData.BoneScalarVelocities[0][BoneIdx],
						InOutPoseData.LocalBoneData.BoneScalarVelocities[FrameNum - 1][BoneIdx],
						(float)FrameIdx / (FrameNum - 1),
						DeltaTime);
				}
			}
		}

		void RecomputeVelocities(const FPoseRootDataView& InOutRootData, const float DeltaTime)
		{
			const int32 FrameNum = InOutRootData.GetFrameNum();
			if (!ensure(FrameNum >= 2)) { return; }

			InOutRootData.RootLinearVelocities[0] = (FVector3f)(InOutRootData.RootLocations[1] - InOutRootData.RootLocations[0]) / FMath::Max(DeltaTime, UE_SMALL_NUMBER);
			InOutRootData.RootAngularVelocities[0] = (InOutRootData.RootRotations[1] * InOutRootData.RootRotations[0].Inverse()).GetShortestArcWith(FQuat4f::Identity).ToRotationVector() / FMath::Max(DeltaTime, UE_SMALL_NUMBER);
			InOutRootData.RootScalarVelocities[0] = Math::VectorLogSafe(Math::VectorDivMax(InOutRootData.RootScales[1], InOutRootData.RootScales[0])) / FMath::Max(DeltaTime, UE_SMALL_NUMBER);

			InOutRootData.RootLinearVelocities[FrameNum - 1] = (FVector3f)(InOutRootData.RootLocations[FrameNum - 1] - InOutRootData.RootLocations[FrameNum - 2]) / FMath::Max(DeltaTime, UE_SMALL_NUMBER);
			InOutRootData.RootAngularVelocities[FrameNum - 1] = (InOutRootData.RootRotations[FrameNum - 1] * InOutRootData.RootRotations[FrameNum - 2].Inverse()).GetShortestArcWith(FQuat4f::Identity).ToRotationVector() / FMath::Max(DeltaTime, UE_SMALL_NUMBER);
			InOutRootData.RootScalarVelocities[FrameNum - 1] = Math::VectorLogSafe(Math::VectorDivMax(InOutRootData.RootScales[FrameNum - 1], InOutRootData.RootScales[FrameNum - 2])) / FMath::Max(DeltaTime, UE_SMALL_NUMBER);

			for (int32 FrameIdx = 1; FrameIdx < FrameNum - 1; FrameIdx++)
			{
				const FVector3f LinearVelocity0 = (FVector3f)(InOutRootData.RootLocations[FrameIdx] - InOutRootData.RootLocations[FrameIdx - 1]) / FMath::Max(DeltaTime, UE_SMALL_NUMBER);
				const FVector3f AngularVelocity0 = (InOutRootData.RootRotations[FrameIdx] * InOutRootData.RootRotations[FrameIdx - 1].Inverse()).GetShortestArcWith(FQuat4f::Identity).ToRotationVector() / FMath::Max(DeltaTime, UE_SMALL_NUMBER);
				const FVector3f ScalarVelocity0 = Math::VectorLogSafe(Math::VectorDivMax(InOutRootData.RootScales[FrameIdx], InOutRootData.RootScales[FrameIdx - 1])) / FMath::Max(DeltaTime, UE_SMALL_NUMBER);
				
				const FVector3f LinearVelocity1 = (FVector3f)(InOutRootData.RootLocations[FrameIdx + 1] - InOutRootData.RootLocations[FrameIdx]) / FMath::Max(DeltaTime, UE_SMALL_NUMBER);
				const FVector3f AngularVelocity1 = (InOutRootData.RootRotations[FrameIdx + 1] * InOutRootData.RootRotations[FrameIdx].Inverse()).GetShortestArcWith(FQuat4f::Identity).ToRotationVector() / FMath::Max(DeltaTime, UE_SMALL_NUMBER);
				const FVector3f ScalarVelocity1 = Math::VectorLogSafe(Math::VectorDivMax(InOutRootData.RootScales[FrameIdx + 1], InOutRootData.RootScales[FrameIdx])) / FMath::Max(DeltaTime, UE_SMALL_NUMBER);

				InOutRootData.RootLinearVelocities[FrameIdx] = (LinearVelocity0 + LinearVelocity1) / 2.0f;
				InOutRootData.RootAngularVelocities[FrameIdx] = (AngularVelocity0 + AngularVelocity1) / 2.0f;
				InOutRootData.RootScalarVelocities[FrameIdx] = (ScalarVelocity0 + ScalarVelocity1) / 2.0f;
			}
		}


		void ForwardKinematics(
			const FPoseGlobalBoneDataView& OutPoseGlobalBoneData,
			const FPoseLocalBoneDataConstView& InPoseLocalBoneData,
			const FPoseRootDataConstView& InPoseRootData,
			const TLearningArrayView<1, const int32> BoneParents,
			const Learning::FIndexSet BoneIndices)
		{
			ForwardKinematicsPartial(OutPoseGlobalBoneData, InPoseLocalBoneData, InPoseRootData, BoneParents, BoneIndices, Learning::FIndexSet(0, BoneIndices.Num()));
		}

		void ForwardKinematics(
			const FPoseGlobalBoneDataView& OutGlobalPoseData,
			const FPoseLocalBoneDataConstView& InPoseLocalBoneData,
			const FPoseRootDataConstView& InPoseRootData,
			const TLearningArrayView<1, const int32> BoneParents)
		{
			ForwardKinematics(OutGlobalPoseData, InPoseLocalBoneData, InPoseRootData, BoneParents, Learning::FIndexSet(0, BoneParents.Num()));
		}

		void ForwardKinematicsPartial(
			const FPoseGlobalBoneDataView& OutPoseGlobalBoneData,
			const FPoseLocalBoneDataConstView& InPoseLocalBoneData,
			const FPoseRootDataConstView& InPoseRootData,
			const TLearningArrayView<1, const int32> BoneParents,
			const UE::Learning::FIndexSet BoneIndices,
			const UE::Learning::FIndexSet BonesToUpdate)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(AnimDatabase::PoseData::ForwardKinematicsPartial);

			const int32 FrameNum = InPoseLocalBoneData.GetFrameNum();
			const int32 BoneNum = InPoseLocalBoneData.GetBoneNum();
			const int32 RefBoneNum = BoneParents.Num();
			check(OutPoseGlobalBoneData.GetFrameNum() == FrameNum);
			check(OutPoseGlobalBoneData.GetBoneNum() == BoneNum);
			check(BoneIndices.Num() == BoneNum);
			check(Math::BoneIndicesAreSortedAndUnique(BoneIndices));

			TArray<int32, TInlineAllocator<256>> RefToBoneIndices;
			RefToBoneIndices.SetNumUninitialized(RefBoneNum);
			Math::BoneFindIndicesOf(RefToBoneIndices, Learning::FIndexSet(0, RefBoneNum), BoneIndices);

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				for (int32 BoneIdx : BonesToUpdate)
				{
					const FVector3f CurrBoneLocation = InPoseLocalBoneData.BoneLocations[FrameIdx][BoneIdx];
					const FQuat4f CurrBoneRotation = InPoseLocalBoneData.BoneRotations[FrameIdx][BoneIdx];
					const FVector3f CurrBoneScale = InPoseLocalBoneData.BoneScales[FrameIdx][BoneIdx];
					const FVector3f CurrBoneLinearVelocity = InPoseLocalBoneData.BoneLinearVelocities[FrameIdx][BoneIdx];
					const FVector3f CurrBoneAngularVelocity = InPoseLocalBoneData.BoneAngularVelocities[FrameIdx][BoneIdx];
					const FVector3f CurrBoneScalarVelocity = InPoseLocalBoneData.BoneScalarVelocities[FrameIdx][BoneIdx];

					FVector ParentBoneLocation = FVector::ZeroVector;
					FQuat4f ParentBoneRotation = FQuat4f::Identity;
					FVector3f ParentBoneScale = FVector3f::OneVector;
					FVector3f ParentBoneLinearVelocity = FVector3f::ZeroVector;
					FVector3f ParentBoneAngularVelocity = FVector3f::ZeroVector;
					FVector3f ParentBoneScalarVelocity = FVector3f::ZeroVector;

					const int32 RefParentBoneIdx = BoneParents[BoneIndices[BoneIdx]];

					if (RefParentBoneIdx == INDEX_NONE)
					{
						ParentBoneLocation = InPoseRootData.RootLocations[FrameIdx];
						ParentBoneRotation = InPoseRootData.RootRotations[FrameIdx];
						ParentBoneScale = InPoseRootData.RootScales[FrameIdx];
						ParentBoneLinearVelocity = InPoseRootData.RootLinearVelocities[FrameIdx];
						ParentBoneAngularVelocity = InPoseRootData.RootAngularVelocities[FrameIdx];
						ParentBoneScalarVelocity = InPoseRootData.RootScalarVelocities[FrameIdx];
					}
					else
					{
						const int32 ParentBoneIdx = RefToBoneIndices[RefParentBoneIdx];
						check(ParentBoneIdx != INDEX_NONE);
						check(ParentBoneIdx < BoneIdx);
						check(BonesToUpdate.Contains(ParentBoneIdx));

						ParentBoneLocation = OutPoseGlobalBoneData.BoneLocations[FrameIdx][ParentBoneIdx];
						ParentBoneRotation = OutPoseGlobalBoneData.BoneRotations[FrameIdx][ParentBoneIdx];
						ParentBoneScale = OutPoseGlobalBoneData.BoneScales[FrameIdx][ParentBoneIdx];
						ParentBoneLinearVelocity = OutPoseGlobalBoneData.BoneLinearVelocities[FrameIdx][ParentBoneIdx];
						ParentBoneAngularVelocity = OutPoseGlobalBoneData.BoneAngularVelocities[FrameIdx][ParentBoneIdx];
						ParentBoneScalarVelocity = OutPoseGlobalBoneData.BoneScalarVelocities[FrameIdx][ParentBoneIdx];
					}

					OutPoseGlobalBoneData.BoneLocations[FrameIdx][BoneIdx] = ParentBoneLocation + (FVector)ParentBoneRotation.RotateVector(CurrBoneLocation * ParentBoneScale);
					OutPoseGlobalBoneData.BoneRotations[FrameIdx][BoneIdx] = ParentBoneRotation * CurrBoneRotation;
					OutPoseGlobalBoneData.BoneScales[FrameIdx][BoneIdx] = ParentBoneScale * CurrBoneScale;
					OutPoseGlobalBoneData.BoneLinearVelocities[FrameIdx][BoneIdx] = ParentBoneLinearVelocity +
						ParentBoneRotation.RotateVector(CurrBoneLinearVelocity * ParentBoneScale) +
						ParentBoneAngularVelocity.Cross(ParentBoneRotation.RotateVector(CurrBoneLocation * ParentBoneScale)) +
						ParentBoneRotation.RotateVector(CurrBoneLocation * ParentBoneScale * ParentBoneScalarVelocity);
					OutPoseGlobalBoneData.BoneAngularVelocities[FrameIdx][BoneIdx] = ParentBoneAngularVelocity + ParentBoneRotation.RotateVector(CurrBoneAngularVelocity);
					OutPoseGlobalBoneData.BoneScalarVelocities[FrameIdx][BoneIdx] = ParentBoneScalarVelocity + CurrBoneScalarVelocity;
				}
			}
		}

		void Teleport(
			const FPoseRootDataView& InOutPoseRootData,
			const TLearningArrayView<1, const FVector> RootLocations,
			const TLearningArrayView<1, const FQuat4f> RootRotations)
		{
			check(InOutPoseRootData.GetFrameNum() == RootLocations.Num());
			check(InOutPoseRootData.GetFrameNum() == RootRotations.Num());

			const int32 FrameNum = InOutPoseRootData.GetFrameNum();

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				const FVector RootLocationDifference = RootLocations[FrameIdx] - InOutPoseRootData.RootLocations[FrameIdx];
				const FQuat4f RootRotationDifference = RootRotations[FrameIdx] * InOutPoseRootData.RootRotations[FrameIdx].Inverse();

				InOutPoseRootData.RootLocations[FrameIdx] = RootLocationDifference + InOutPoseRootData.RootLocations[FrameIdx];
				InOutPoseRootData.RootRotations[FrameIdx] = RootRotationDifference * InOutPoseRootData.RootRotations[FrameIdx];
				InOutPoseRootData.RootLinearVelocities[FrameIdx] = RootRotationDifference.RotateVector(InOutPoseRootData.RootLinearVelocities[FrameIdx]);
				InOutPoseRootData.RootAngularVelocities[FrameIdx] = RootRotationDifference.RotateVector(InOutPoseRootData.RootAngularVelocities[FrameIdx]);
			}
		}

		void BlendRoot(
			const FPoseRootDataView& OutPoseData,
			const FPoseRootDataConstView& InPoseA,
			const FPoseRootDataConstView& InPoseB,
			const float Alpha)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(AnimDatabase::PoseData::Blend);

			check(OutPoseData.GetFrameNum() == InPoseA.GetFrameNum());
			check(OutPoseData.GetFrameNum() == InPoseB.GetFrameNum());

			const int32 FrameNum = OutPoseData.GetFrameNum();

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				OutPoseData.RootLocations[FrameIdx] = FMath::Lerp(InPoseA.RootLocations[FrameIdx], InPoseB.RootLocations[FrameIdx], Alpha);
				OutPoseData.RootRotations[FrameIdx] = FQuat4f::Slerp(InPoseA.RootRotations[FrameIdx], InPoseB.RootRotations[FrameIdx], Alpha);
				OutPoseData.RootScales[FrameIdx] = Math::VectorEerp(InPoseA.RootScales[FrameIdx], InPoseB.RootScales[FrameIdx], Alpha);
			}
		}

		void IntegrateRoot(const FPoseRootDataView& InOutRootData, const float DeltaTime)
		{
			const int32 FrameNum = InOutRootData.GetFrameNum();

			for (int32 FrameIdx = 1; FrameIdx < FrameNum; FrameIdx++)
			{
				const FVector3f RootLocalLinearVelocity = InOutRootData.RootRotations[FrameIdx].UnrotateVector(InOutRootData.RootLinearVelocities[FrameIdx]);
				const FVector3f RootLocalAngularVelocity = InOutRootData.RootRotations[FrameIdx].UnrotateVector(InOutRootData.RootAngularVelocities[FrameIdx]);

				InOutRootData.RootLocations[FrameIdx] = InOutRootData.RootLocations[FrameIdx - 1] + DeltaTime * (FVector)InOutRootData.RootLinearVelocities[FrameIdx - 1];
				InOutRootData.RootRotations[FrameIdx] = FQuat4f::MakeFromRotationVector(DeltaTime * InOutRootData.RootAngularVelocities[FrameIdx - 1]) * InOutRootData.RootRotations[FrameIdx - 1];
				InOutRootData.RootLinearVelocities[FrameIdx] = InOutRootData.RootRotations[FrameIdx].RotateVector(RootLocalLinearVelocity);
				InOutRootData.RootAngularVelocities[FrameIdx] = InOutRootData.RootRotations[FrameIdx].RotateVector(RootLocalAngularVelocity);
			}
		}

		void BlendVia(
			const FPoseLocalBoneDataView& OutPoseData,
			const TLearningArrayView<2, FQuat4f> InOutViaRotations,
			const FPoseLocalBoneDataConstView& InPoseA,
			const FPoseLocalBoneDataConstView& InPoseB,
			const float Alpha)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(AnimDatabase::PoseData::BlendVia);

			check(OutPoseData.GetFrameNum() == InPoseA.GetFrameNum());
			check(OutPoseData.GetFrameNum() == InPoseB.GetFrameNum());
			check(OutPoseData.GetBoneNum() == InPoseA.GetBoneNum());
			check(OutPoseData.GetBoneNum() == InPoseB.GetBoneNum());
			check(OutPoseData.GetFrameNum() == InOutViaRotations.Num<0>());
			check(OutPoseData.GetBoneNum() == InOutViaRotations.Num<1>());

			const int32 FrameNum = OutPoseData.GetFrameNum();
			const int32 BoneNum = OutPoseData.GetBoneNum();

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
				{
					const FQuat4f RotationDiff = (InPoseB.BoneRotations[FrameIdx][BoneIdx] * InPoseA.BoneRotations[FrameIdx][BoneIdx].Inverse()).GetShortestArcWith(InOutViaRotations[FrameIdx][BoneIdx]);

					InOutViaRotations[FrameIdx][BoneIdx] = RotationDiff;

					OutPoseData.BoneLocations[FrameIdx][BoneIdx] = FMath::Lerp(InPoseA.BoneLocations[FrameIdx][BoneIdx], InPoseB.BoneLocations[FrameIdx][BoneIdx], Alpha);
					OutPoseData.BoneRotations[FrameIdx][BoneIdx] = FQuat4f::MakeFromRotationVector(RotationDiff.ToRotationVector() * Alpha) * InPoseA.BoneRotations[FrameIdx][BoneIdx];
					OutPoseData.BoneScales[FrameIdx][BoneIdx] = Math::VectorEerp(InPoseA.BoneScales[FrameIdx][BoneIdx], InPoseB.BoneScales[FrameIdx][BoneIdx], Alpha);
				}
			}
		}

		void FitExtrapolationHalfLives(
			const TLearningArrayView<2, FVector3f> OutBoneLocationHalfLives,
			const TLearningArrayView<2, FVector3f> OutBoneRotationHalfLives,
			const TLearningArrayView<2, FVector3f> OutBoneScaleHalfLives,
			const FPoseLocalBoneDataConstView& InPoseSource,
			const FPoseLocalBoneDataConstView& InPoseTarget,
			const float ExtrapolationHalfLife,
			const float ExtrapolationHalfLifeMin,
			const float ExtrapolationHalfLifeMax,
			const float MaxLinearVelocity,
			const float MaxAngularVelocity,
			const float MaxScalarVelocity)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(AnimDatabase::PoseData::FitExtrapolationHalfLives);

			check(InPoseSource.GetFrameNum() == InPoseTarget.GetFrameNum());
			check(InPoseSource.GetBoneNum() == InPoseTarget.GetBoneNum());
			check(InPoseSource.GetFrameNum() == OutBoneLocationHalfLives.Num<0>());
			check(InPoseSource.GetFrameNum() == OutBoneRotationHalfLives.Num<0>());
			check(InPoseSource.GetFrameNum() == OutBoneScaleHalfLives.Num<0>());
			check(InPoseSource.GetBoneNum() == OutBoneLocationHalfLives.Num<1>());
			check(InPoseSource.GetBoneNum() == OutBoneRotationHalfLives.Num<1>());
			check(InPoseSource.GetBoneNum() == OutBoneScaleHalfLives.Num<1>());

			const int32 FrameNum = InPoseSource.GetFrameNum();
			const int32 BoneNum = InPoseSource.GetBoneNum();

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
				{
					const FVector3f ClampedBoneLinearVelocity = InPoseSource.BoneLinearVelocities[FrameIdx][BoneIdx].GetClampedToMaxSize(MaxLinearVelocity);
					const FVector3f ClampedBoneAngularVelocity = InPoseSource.BoneAngularVelocities[FrameIdx][BoneIdx].GetClampedToMaxSize(MaxAngularVelocity);
					const FVector3f ClampedBoneScalarVelocity = InPoseSource.BoneScalarVelocities[FrameIdx][BoneIdx].GetClampedToMaxSize(MaxScalarVelocity);

					const FVector3f BoneLocationDifference = InPoseTarget.BoneLocations[FrameIdx][BoneIdx] - InPoseSource.BoneLocations[FrameIdx][BoneIdx];
					const FVector3f BoneRotationDifference = (InPoseTarget.BoneRotations[FrameIdx][BoneIdx] * InPoseSource.BoneRotations[FrameIdx][BoneIdx].Inverse()).GetShortestArcWith(FQuat4f::Identity).ToRotationVector();
					const FVector3f BoneScaleDifference = Math::VectorDivMax(InPoseTarget.BoneScales[FrameIdx][BoneIdx], InPoseSource.BoneScales[FrameIdx][BoneIdx]);

					OutBoneLocationHalfLives[FrameIdx][BoneIdx] = Math::ComputeDecayHalfLifeFromDiffAndVelocity(
						BoneLocationDifference,
						ClampedBoneLinearVelocity,
						ExtrapolationHalfLife,
						ExtrapolationHalfLifeMin,
						ExtrapolationHalfLifeMax);

					OutBoneRotationHalfLives[FrameIdx][BoneIdx] = Math::ComputeDecayHalfLifeFromDiffAndVelocity(
						BoneRotationDifference,
						ClampedBoneAngularVelocity,
						ExtrapolationHalfLife,
						ExtrapolationHalfLifeMin,
						ExtrapolationHalfLifeMax);

					OutBoneScaleHalfLives[FrameIdx][BoneIdx] = Math::ComputeDecayHalfLifeFromDiffAndVelocity(
						BoneScaleDifference,
						ClampedBoneScalarVelocity,
						ExtrapolationHalfLife,
						ExtrapolationHalfLifeMin,
						ExtrapolationHalfLifeMax);
				}
			}
		}

		void ExtrapolateRoot(
			const FPoseRootDataView& OutPoseData,
			const FPoseRootDataConstView& InPoseData,
			const float Time,
			const float RootHalfLife)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(AnimDatabase::PoseData::ExtrapolateRoot);

			check(OutPoseData.GetFrameNum() == InPoseData.GetFrameNum());

			const int32 FrameNum = OutPoseData.GetFrameNum();

			const FVector3f RootHalfLifeVector = FVector3f(RootHalfLife, RootHalfLife, RootHalfLife);

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				Math::ExtrapolateTranslation(
					OutPoseData.RootLocations[FrameIdx],
					OutPoseData.RootLinearVelocities[FrameIdx],
					InPoseData.RootLocations[FrameIdx],
					InPoseData.RootLinearVelocities[FrameIdx],
					Time,
					RootHalfLifeVector);

				Math::ExtrapolateRotation(
					OutPoseData.RootRotations[FrameIdx],
					OutPoseData.RootAngularVelocities[FrameIdx],
					InPoseData.RootRotations[FrameIdx],
					InPoseData.RootAngularVelocities[FrameIdx],
					Time,
					RootHalfLifeVector);

				Math::ExtrapolateScale(
					OutPoseData.RootScales[FrameIdx],
					OutPoseData.RootScalarVelocities[FrameIdx],
					InPoseData.RootScales[FrameIdx],
					InPoseData.RootScalarVelocities[FrameIdx],
					Time,
					RootHalfLifeVector);
			}
		}

		void Extrapolate(
			const FPoseLocalBoneDataView& OutPoseData,
			const FPoseLocalBoneDataConstView& InPoseData,
			const float Time,
			const TLearningArrayView<2, const FVector3f> BoneLocationHalfLives,
			const TLearningArrayView<2, const FVector3f> BoneRotationHalfLives,
			const TLearningArrayView<2, const FVector3f> BoneScaleHalfLives)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(AnimDatabase::PoseData::Extrapolate);

			check(OutPoseData.GetFrameNum() == InPoseData.GetFrameNum());
			check(OutPoseData.GetBoneNum() == InPoseData.GetBoneNum());
			check(OutPoseData.GetFrameNum() == BoneLocationHalfLives.Num<0>());
			check(OutPoseData.GetFrameNum() == BoneRotationHalfLives.Num<0>());
			check(OutPoseData.GetFrameNum() == BoneScaleHalfLives.Num<0>());
			check(OutPoseData.GetBoneNum() == BoneLocationHalfLives.Num<1>());
			check(OutPoseData.GetBoneNum() == BoneRotationHalfLives.Num<1>());
			check(OutPoseData.GetBoneNum() == BoneScaleHalfLives.Num<1>());

			const int32 FrameNum = OutPoseData.GetFrameNum();
			const int32 BoneNum = OutPoseData.GetBoneNum();

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
				{
					Math::ExtrapolateTranslation(
						OutPoseData.BoneLocations[FrameIdx][BoneIdx],
						OutPoseData.BoneLinearVelocities[FrameIdx][BoneIdx],
						InPoseData.BoneLocations[FrameIdx][BoneIdx],
						InPoseData.BoneLinearVelocities[FrameIdx][BoneIdx],
						Time,
						BoneLocationHalfLives[FrameIdx][BoneIdx]);

					Math::ExtrapolateRotation(
						OutPoseData.BoneRotations[FrameIdx][BoneIdx],
						OutPoseData.BoneAngularVelocities[FrameIdx][BoneIdx],
						InPoseData.BoneRotations[FrameIdx][BoneIdx],
						InPoseData.BoneAngularVelocities[FrameIdx][BoneIdx],
						Time,
						BoneRotationHalfLives[FrameIdx][BoneIdx]);

					Math::ExtrapolateScale(
						OutPoseData.BoneScales[FrameIdx][BoneIdx],
						OutPoseData.BoneScalarVelocities[FrameIdx][BoneIdx],
						InPoseData.BoneScales[FrameIdx][BoneIdx],
						InPoseData.BoneScalarVelocities[FrameIdx][BoneIdx],
						Time,
						BoneScaleHalfLives[FrameIdx][BoneIdx]);
				}
			}
		}

		void FitInertializationOffsetsRoot(
			const FPoseRootDataView& OutOffsetPose,
			const FPoseRootDataConstView& InPoseSource,
			const FPoseRootDataConstView& InPoseTarget)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(AnimDatabase::PoseData::FitInertializationOffsetsRoot);

			check(OutOffsetPose.GetFrameNum() == InPoseSource.GetFrameNum());
			check(OutOffsetPose.GetFrameNum() == InPoseTarget.GetFrameNum());

			const int32 FrameNum = OutOffsetPose.GetFrameNum();

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				const FVector RootLocationDifference = InPoseSource.RootLocations[FrameIdx] - InPoseTarget.RootLocations[FrameIdx];
				const FQuat4f RootRotationDifference = (InPoseSource.RootRotations[FrameIdx] * InPoseTarget.RootRotations[FrameIdx].Inverse()).GetShortestArcWith(FQuat4f::Identity);
				const FVector3f RootScaleDifference = Math::VectorDivMax(InPoseSource.RootScales[FrameIdx], InPoseTarget.RootScales[FrameIdx]);

				OutOffsetPose.RootLocations[FrameIdx] = RootLocationDifference;
				OutOffsetPose.RootRotations[FrameIdx] = RootRotationDifference;
				OutOffsetPose.RootScales[FrameIdx] = RootScaleDifference;
				OutOffsetPose.RootLinearVelocities[FrameIdx] = InPoseSource.RootLinearVelocities[FrameIdx] - InPoseTarget.RootLinearVelocities[FrameIdx];
				OutOffsetPose.RootAngularVelocities[FrameIdx] = InPoseSource.RootAngularVelocities[FrameIdx] - InPoseTarget.RootAngularVelocities[FrameIdx];
				OutOffsetPose.RootScalarVelocities[FrameIdx] = InPoseSource.RootScalarVelocities[FrameIdx] - InPoseTarget.RootScalarVelocities[FrameIdx];
			}
		}

		void ApplyInertializationOffsetsRoot(
			const FPoseRootDataView& OutFinalPose,
			const FPoseRootDataConstView& InOffsetPose,
			const FPoseRootDataConstView& InPoseTarget,
			const float BlendTime,
			const float BlendDuration)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(AnimDatabase::PoseData::ApplyInertializationOffsetsRoot);

			check(OutFinalPose.GetFrameNum() == InOffsetPose.GetFrameNum());
			check(OutFinalPose.GetFrameNum() == InPoseTarget.GetFrameNum());

			const int32 FrameNum = OutFinalPose.GetFrameNum();

			// Cubic Weights
			const float RT = FMath::Clamp(BlendTime / (BlendDuration + UE_SMALL_NUMBER), 0.0f, 1.0f);
			const float RW0 = 2.0f * RT * RT * RT - 3.0f * RT * RT + 1.0f;
			const float RW1 = (RT * RT * RT - 2.0f * RT * RT + RT) * BlendDuration;
			const float RW2 = (6.0f * RT * RT - 6.0f * RT) / (BlendDuration + UE_SMALL_NUMBER);
			const float RW3 = 3.0f * RT * RT - 4.0f * RT + 1.0f;

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				const FVector RootLocationOffset = RW0 * InOffsetPose.RootLocations[FrameIdx] + RW1 * (FVector)InOffsetPose.RootLinearVelocities[FrameIdx];
				const FQuat4f RootRotationOffset = FQuat4f::MakeFromRotationVector(
					RW0 * InOffsetPose.RootRotations[FrameIdx].ToRotationVector() + RW1 * InOffsetPose.RootAngularVelocities[FrameIdx]);
				const FVector3f RootScaleOffset =Math::VectorExp(RW0 * Math::VectorLogSafe(InOffsetPose.RootScales[FrameIdx]) + RW1 * InOffsetPose.RootScalarVelocities[FrameIdx]);
				const FVector3f RootLinearVelocityOffset = RW2 * (FVector3f)InOffsetPose.RootLocations[FrameIdx] + RW3 * InOffsetPose.RootLinearVelocities[FrameIdx];
				const FVector3f RootAngularVelocityOffset = RW2 * InOffsetPose.RootRotations[FrameIdx].ToRotationVector() +
					RW3 * InOffsetPose.RootAngularVelocities[FrameIdx];
				const FVector3f RootScalarVelocityOffset = RW2 * Math::VectorLogSafe(InOffsetPose.RootScales[FrameIdx]) + RW3 * InOffsetPose.RootScalarVelocities[FrameIdx];

				OutFinalPose.RootLocations[FrameIdx] = RootLocationOffset + InPoseTarget.RootLocations[FrameIdx];
				OutFinalPose.RootRotations[FrameIdx] = RootRotationOffset * InPoseTarget.RootRotations[FrameIdx];
				OutFinalPose.RootScales[FrameIdx] = RootScaleOffset * InPoseTarget.RootScales[FrameIdx];
				OutFinalPose.RootLinearVelocities[FrameIdx] = RootLinearVelocityOffset + InPoseTarget.RootLinearVelocities[FrameIdx];
				OutFinalPose.RootAngularVelocities[FrameIdx] = RootAngularVelocityOffset + RootRotationOffset.RotateVector(InPoseTarget.RootAngularVelocities[FrameIdx]);
				OutFinalPose.RootScalarVelocities[FrameIdx] = RootScalarVelocityOffset + InPoseTarget.RootScalarVelocities[FrameIdx];
			}
		}

		void FitInertializationOffsets(
			const FPoseLocalBoneDataView& OutOffsetPose,
			const FPoseLocalBoneDataConstView& InPoseSource,
			const FPoseLocalBoneDataConstView& InPoseTarget)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(AnimDatabase::PoseData::FitInertializationOffsets);

			check(OutOffsetPose.GetFrameNum() == InPoseSource.GetFrameNum());
			check(OutOffsetPose.GetBoneNum() == InPoseSource.GetBoneNum());
			check(OutOffsetPose.GetFrameNum() == InPoseTarget.GetFrameNum());
			check(OutOffsetPose.GetBoneNum() == InPoseTarget.GetBoneNum());

			const int32 FrameNum = OutOffsetPose.GetFrameNum();
			const int32 BoneNum = OutOffsetPose.GetBoneNum();

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
				{
					const FVector3f BoneLocationDifference = InPoseSource.BoneLocations[FrameIdx][BoneIdx] - InPoseTarget.BoneLocations[FrameIdx][BoneIdx];
					const FQuat4f BoneRotationDifference = (InPoseSource.BoneRotations[FrameIdx][BoneIdx] * InPoseTarget.BoneRotations[FrameIdx][BoneIdx].Inverse()).GetShortestArcWith(FQuat4f::Identity);
					const FVector3f BoneScaleDifference = Math::VectorDivMax(InPoseSource.BoneScales[FrameIdx][BoneIdx], InPoseTarget.BoneScales[FrameIdx][BoneIdx]);

					OutOffsetPose.BoneLocations[FrameIdx][BoneIdx] = BoneLocationDifference;
					OutOffsetPose.BoneRotations[FrameIdx][BoneIdx] = BoneRotationDifference;
					OutOffsetPose.BoneScales[FrameIdx][BoneIdx] = BoneScaleDifference;
					OutOffsetPose.BoneLinearVelocities[FrameIdx][BoneIdx] = InPoseSource.BoneLinearVelocities[FrameIdx][BoneIdx] - InPoseTarget.BoneLinearVelocities[FrameIdx][BoneIdx];
					OutOffsetPose.BoneAngularVelocities[FrameIdx][BoneIdx] = InPoseSource.BoneAngularVelocities[FrameIdx][BoneIdx] - InPoseTarget.BoneAngularVelocities[FrameIdx][BoneIdx];
					OutOffsetPose.BoneScalarVelocities[FrameIdx][BoneIdx] = InPoseSource.BoneScalarVelocities[FrameIdx][BoneIdx] - InPoseTarget.BoneScalarVelocities[FrameIdx][BoneIdx];
				}
			}
		}

		void ApplyInertializationOffsets(
			const FPoseLocalBoneDataView& OutFinalPose,
			const FPoseLocalBoneDataConstView& InOffsetPose,
			const FPoseLocalBoneDataConstView& InPoseTarget,
			const float BlendTime,
			const float BlendDuration)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(AnimDatabase::PoseData::ApplyInertializationOffsets);

			check(OutFinalPose.GetFrameNum() == InOffsetPose.GetFrameNum());
			check(OutFinalPose.GetBoneNum() == InOffsetPose.GetBoneNum());
			check(OutFinalPose.GetFrameNum() == InPoseTarget.GetFrameNum());
			check(OutFinalPose.GetBoneNum() == InPoseTarget.GetBoneNum());

			const int32 FrameNum = OutFinalPose.GetFrameNum();
			const int32 BoneNum = OutFinalPose.GetBoneNum();

			// Cubic Weights
			const float T = FMath::Clamp(BlendTime / (BlendDuration + UE_SMALL_NUMBER), 0.0f, 1.0f);
			const float W0 = 2.0f * T * T * T - 3.0f * T * T + 1.0f;
			const float W1 = (T * T * T - 2.0f * T * T + T) * BlendDuration;
			const float W2 = (6.0f * T * T - 6.0f * T) / (BlendDuration + UE_SMALL_NUMBER);
			const float W3 = 3.0f * T * T - 4.0f * T + 1.0f;

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
				{
					const FVector3f BoneLocationOffset = W0 * InOffsetPose.BoneLocations[FrameIdx][BoneIdx] + W1 * InOffsetPose.BoneLinearVelocities[FrameIdx][BoneIdx];
					const FQuat4f BoneRotationOffset = FQuat4f::MakeFromRotationVector(
						W0 * InOffsetPose.BoneRotations[FrameIdx][BoneIdx].ToRotationVector() + W1 * InOffsetPose.BoneAngularVelocities[FrameIdx][BoneIdx]);
					const FVector3f BoneScaleOffset = Math::VectorExp(W0 * Math::VectorLogSafe(InOffsetPose.BoneScales[FrameIdx][BoneIdx]) + W1 * InOffsetPose.BoneScalarVelocities[FrameIdx][BoneIdx]);
					const FVector3f BoneLinearVelocityOffset = W2 * InOffsetPose.BoneLocations[FrameIdx][BoneIdx] + W3 * InOffsetPose.BoneLinearVelocities[FrameIdx][BoneIdx];
					const FVector3f BoneAngularVelocityOffset = W2 * InOffsetPose.BoneRotations[FrameIdx][BoneIdx].ToRotationVector() +
						W3 * InOffsetPose.BoneAngularVelocities[FrameIdx][BoneIdx];
					const FVector3f BoneScalarVelocityOffset = W2 * Math::VectorLogSafe(InOffsetPose.BoneScales[FrameIdx][BoneIdx]) + W3 * InOffsetPose.BoneScalarVelocities[FrameIdx][BoneIdx];

					OutFinalPose.BoneLocations[FrameIdx][BoneIdx] = BoneLocationOffset + InPoseTarget.BoneLocations[FrameIdx][BoneIdx];
					OutFinalPose.BoneRotations[FrameIdx][BoneIdx] = BoneRotationOffset * InPoseTarget.BoneRotations[FrameIdx][BoneIdx];
					OutFinalPose.BoneScales[FrameIdx][BoneIdx] = BoneScaleOffset * InPoseTarget.BoneScales[FrameIdx][BoneIdx];
					OutFinalPose.BoneLinearVelocities[FrameIdx][BoneIdx] = BoneLinearVelocityOffset + InPoseTarget.BoneLinearVelocities[FrameIdx][BoneIdx];
					OutFinalPose.BoneAngularVelocities[FrameIdx][BoneIdx] = BoneAngularVelocityOffset + BoneRotationOffset.RotateVector(InPoseTarget.BoneAngularVelocities[FrameIdx][BoneIdx]);
					OutFinalPose.BoneScalarVelocities[FrameIdx][BoneIdx] = BoneScalarVelocityOffset + InPoseTarget.BoneScalarVelocities[FrameIdx][BoneIdx];
				}
			}
		}

		void FitInertializationOffsetsAttributes(
			const FPoseAttributeDataView& OutOffsetPose,
			const FPoseAttributeDataConstView& InPoseSource,
			const FPoseAttributeDataConstView& InPoseTarget)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(AnimDatabase::PoseData::FitInertializationOffsetsAttributes);

			check(OutOffsetPose.GetFrameNum() == InPoseSource.GetFrameNum());
			check(UE::Learning::Array::Equal(OutOffsetPose.GetAttributeTypes(), InPoseSource.GetAttributeTypes()));
			check(OutOffsetPose.GetFrameNum() == InPoseTarget.GetFrameNum());
			check(UE::Learning::Array::Equal(OutOffsetPose.GetAttributeTypes(), InPoseTarget.GetAttributeTypes()));

			const int32 FrameNum = OutOffsetPose.GetFrameNum();
			const int32 AttributeNum = OutOffsetPose.GetAttributeNum();

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				for (int32 AttributeIdx = 0; AttributeIdx < AttributeNum; AttributeIdx++)
				{
					const EAnimDatabaseAttributeType AttributeType = InPoseSource.GetAttributeType(AttributeIdx);
					const bool bSourceActive = InPoseSource.GetAttributeActive(FrameIdx, AttributeIdx);
					const bool bTargetActive = InPoseTarget.GetAttributeActive(FrameIdx, AttributeIdx);

					OutOffsetPose.SetAttributeActive(FrameIdx, AttributeIdx, bSourceActive && bTargetActive);

					if (OutOffsetPose.GetAttributeActive(FrameIdx, AttributeIdx))
					{
						switch (AttributeType)
						{
						case EAnimDatabaseAttributeType::Null: break;
						case EAnimDatabaseAttributeType::Bool: OutOffsetPose.SetBool(FrameIdx, AttributeIdx, false); break;
						case EAnimDatabaseAttributeType::Float: OutOffsetPose.SetFloat(FrameIdx, AttributeIdx, InPoseSource.GetFloat(FrameIdx, AttributeIdx) - InPoseTarget.GetFloat(FrameIdx, AttributeIdx)); break;
						case EAnimDatabaseAttributeType::Angle: OutOffsetPose.SetAngle(FrameIdx, AttributeIdx, FMath::FindDeltaAngleRadians(InPoseTarget.GetAngle(FrameIdx, AttributeIdx), InPoseSource.GetAngle(FrameIdx, AttributeIdx))); break;
						case EAnimDatabaseAttributeType::Location: OutOffsetPose.SetLocation(FrameIdx, AttributeIdx, InPoseSource.GetLocation(FrameIdx, AttributeIdx) - InPoseTarget.GetLocation(FrameIdx, AttributeIdx)); break;
						case EAnimDatabaseAttributeType::Rotation: OutOffsetPose.SetRotation(FrameIdx, AttributeIdx, (InPoseSource.GetRotation(FrameIdx, AttributeIdx) * InPoseTarget.GetRotation(FrameIdx, AttributeIdx).Inverse()).GetShortestArcWith(FQuat4f::Identity)); break;
						case EAnimDatabaseAttributeType::Scale: OutOffsetPose.SetScale(FrameIdx, AttributeIdx, Math::VectorDivMax(InPoseSource.GetScale(FrameIdx, AttributeIdx), InPoseTarget.GetScale(FrameIdx, AttributeIdx))); break;
						case EAnimDatabaseAttributeType::LinearVelocity: OutOffsetPose.SetLinearVelocity(FrameIdx, AttributeIdx, InPoseSource.GetLinearVelocity(FrameIdx, AttributeIdx) - InPoseTarget.GetLinearVelocity(FrameIdx, AttributeIdx)); break;
						case EAnimDatabaseAttributeType::AngularVelocity: OutOffsetPose.SetAngularVelocity(FrameIdx, AttributeIdx, InPoseSource.GetAngularVelocity(FrameIdx, AttributeIdx) - InPoseTarget.GetAngularVelocity(FrameIdx, AttributeIdx)); break;
						case EAnimDatabaseAttributeType::ScalarVelocity: OutOffsetPose.SetScalarVelocity(FrameIdx, AttributeIdx, InPoseSource.GetScalarVelocity(FrameIdx, AttributeIdx) - InPoseTarget.GetScalarVelocity(FrameIdx, AttributeIdx)); break;
						case EAnimDatabaseAttributeType::Direction: OutOffsetPose.SetDirection(FrameIdx, AttributeIdx, InPoseSource.GetDirection(FrameIdx, AttributeIdx) - InPoseTarget.GetDirection(FrameIdx, AttributeIdx)); break;
						case EAnimDatabaseAttributeType::Transform:
						{
							const FTransform3f TargetTransform = InPoseTarget.GetTransform(FrameIdx, AttributeIdx);
							const FTransform3f SourceTransform = InPoseSource.GetTransform(FrameIdx, AttributeIdx);
							FTransform3f OutTransform = FTransform3f::Identity;
							OutTransform.SetLocation(SourceTransform.GetLocation() - TargetTransform.GetLocation());
							OutTransform.SetRotation((SourceTransform.GetRotation() * TargetTransform.GetRotation().Inverse()).GetShortestArcWith(FQuat4f::Identity));
							OutTransform.SetScale3D(Math::VectorDivMax(SourceTransform.GetScale3D(), TargetTransform.GetScale3D()));
							OutOffsetPose.SetTransform(FrameIdx, AttributeIdx, OutTransform);
							break;
						}

						case EAnimDatabaseAttributeType::Event:
						{
							bool bTargetEventKnown = false, bSourceEventKnown = false;
							float TargetTimeUntilEvent = UE_MAX_FLT, SourceTimeUntilEvent = UE_MAX_FLT;
							InPoseTarget.GetEvent(bTargetEventKnown, TargetTimeUntilEvent, FrameIdx, AttributeIdx);
							InPoseSource.GetEvent(bSourceEventKnown, SourceTimeUntilEvent, FrameIdx, AttributeIdx);

							if (bTargetEventKnown && bSourceEventKnown)
							{
								OutOffsetPose.SetEvent(FrameIdx, AttributeIdx, true, SourceTimeUntilEvent - TargetTimeUntilEvent);
							}
							else
							{
								OutOffsetPose.SetEvent(FrameIdx, AttributeIdx, false, UE_MAX_FLT);
							}
						}
						}
					}
					else
					{
						UE::Learning::Array::Zero(
							OutOffsetPose.AttributeData[FrameIdx].Slice(
								OutOffsetPose.GetAttributeOffset(AttributeIdx),
								OutOffsetPose.GetAttributeSize(AttributeIdx)));
					}
				}
			}
		}

		void ApplyInertializationOffsetsAttributes(
			const FPoseAttributeDataView& OutFinalPose,
			const FPoseAttributeDataConstView& InOffsetPose,
			const FPoseAttributeDataConstView& InPoseTarget,
			const float BlendTime,
			const float BlendDuration)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(AnimDatabase::PoseData::ApplyInertializationOffsetsAttributes);

			check(OutFinalPose.GetFrameNum() == InOffsetPose.GetFrameNum());
			check(Learning::Array::Equal(OutFinalPose.GetAttributeTypes(), InOffsetPose.GetAttributeTypes()));
			check(OutFinalPose.GetFrameNum() == InPoseTarget.GetFrameNum());
			check(Learning::Array::Equal(OutFinalPose.GetAttributeTypes(), InPoseTarget.GetAttributeTypes()));

			const int32 FrameNum = OutFinalPose.GetFrameNum();
			const int32 AttributeNum = OutFinalPose.GetAttributeNum();

			const float T = FMath::Clamp(BlendTime / (BlendDuration + UE_SMALL_NUMBER), 0.0f, 1.0f);
			const float Alpha = 1.0f - T * T * (3.0f - 2.0f * T);

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				for (int32 AttributeIdx = 0; AttributeIdx < AttributeNum; AttributeIdx++)
				{
					const EAnimDatabaseAttributeType AttributeType = InPoseTarget.GetAttributeType(AttributeIdx);
					const bool bTargetActive = InPoseTarget.GetAttributeActive(FrameIdx, AttributeIdx);
					const bool bOffsetActive = InOffsetPose.GetAttributeActive(FrameIdx, AttributeIdx);

					OutFinalPose.SetAttributeActive(FrameIdx, AttributeIdx, bTargetActive);

					if (bTargetActive && bOffsetActive)
					{
						switch (AttributeType)
						{
						case EAnimDatabaseAttributeType::Null: break;
						case EAnimDatabaseAttributeType::Bool: OutFinalPose.SetBool(FrameIdx, AttributeIdx, InPoseTarget.GetBool(FrameIdx, AttributeIdx)); break;
						case EAnimDatabaseAttributeType::Float: OutFinalPose.SetFloat(FrameIdx, AttributeIdx, Alpha * InOffsetPose.GetFloat(FrameIdx, AttributeIdx) + InPoseTarget.GetFloat(FrameIdx, AttributeIdx)); break;
						case EAnimDatabaseAttributeType::Angle: OutFinalPose.SetAngle(FrameIdx, AttributeIdx, Alpha * InOffsetPose.GetAngle(FrameIdx, AttributeIdx) + InPoseTarget.GetAngle(FrameIdx, AttributeIdx)); break;
						case EAnimDatabaseAttributeType::Location: OutFinalPose.SetLocation(FrameIdx, AttributeIdx, Alpha * InOffsetPose.GetLocation(FrameIdx, AttributeIdx) + InPoseTarget.GetLocation(FrameIdx, AttributeIdx)); break;
						case EAnimDatabaseAttributeType::Rotation: OutFinalPose.SetRotation(FrameIdx, AttributeIdx, FQuat4f::MakeFromRotationVector(Alpha * InOffsetPose.GetRotation(FrameIdx, AttributeIdx).ToRotationVector()) * InPoseTarget.GetRotation(FrameIdx, AttributeIdx)); break;
						case EAnimDatabaseAttributeType::Scale: OutFinalPose.SetScale(FrameIdx, AttributeIdx, Math::VectorExp(Alpha * Math::VectorLogSafe(InOffsetPose.GetScale(FrameIdx, AttributeIdx))) * InPoseTarget.GetScale(FrameIdx, AttributeIdx)); break;
						case EAnimDatabaseAttributeType::LinearVelocity: OutFinalPose.SetLinearVelocity(FrameIdx, AttributeIdx, Alpha * InOffsetPose.GetLinearVelocity(FrameIdx, AttributeIdx) + InPoseTarget.GetLinearVelocity(FrameIdx, AttributeIdx)); break;
						case EAnimDatabaseAttributeType::AngularVelocity: OutFinalPose.SetAngularVelocity(FrameIdx, AttributeIdx, Alpha * InOffsetPose.GetAngularVelocity(FrameIdx, AttributeIdx) + InPoseTarget.GetAngularVelocity(FrameIdx, AttributeIdx)); break;
						case EAnimDatabaseAttributeType::ScalarVelocity: OutFinalPose.SetScalarVelocity(FrameIdx, AttributeIdx, Alpha * InOffsetPose.GetScalarVelocity(FrameIdx, AttributeIdx) + InPoseTarget.GetScalarVelocity(FrameIdx, AttributeIdx)); break;
						case EAnimDatabaseAttributeType::Direction: OutFinalPose.SetDirection(FrameIdx, AttributeIdx, (Alpha * InOffsetPose.GetDirection(FrameIdx, AttributeIdx) + InPoseTarget.GetDirection(FrameIdx, AttributeIdx)).GetSafeNormal(UE_SMALL_NUMBER, FVector3f::ForwardVector)); break;
						case EAnimDatabaseAttributeType::Transform:
						{
							const FTransform3f TargetTransform = InPoseTarget.GetTransform(FrameIdx, AttributeIdx);
							const FTransform3f OffsetTransform = InOffsetPose.GetTransform(FrameIdx, AttributeIdx);
							FTransform3f OutTransform = FTransform3f::Identity;
							OutTransform.SetLocation(Alpha * OffsetTransform.GetLocation() + TargetTransform.GetLocation());
							OutTransform.SetRotation(FQuat4f::MakeFromRotationVector(Alpha * OffsetTransform.GetRotation().ToRotationVector()) * TargetTransform.GetRotation());
							OutTransform.SetScale3D(Math::VectorExp(Alpha * Math::VectorLogSafe(OffsetTransform.GetScale3D())) * TargetTransform.GetScale3D());
							OutFinalPose.SetTransform(FrameIdx, AttributeIdx, OutTransform);
							break;
						}

						case EAnimDatabaseAttributeType::Event:
						{
							bool bTargetEventKnown = false, bOffsetEventKnown = false;
							float TargetTimeUntilEvent = UE_MAX_FLT, OffsetTimeUntilEvent = UE_MAX_FLT;
							InPoseTarget.GetEvent(bTargetEventKnown, TargetTimeUntilEvent, FrameIdx, AttributeIdx);
							InOffsetPose.GetEvent(bOffsetEventKnown, OffsetTimeUntilEvent, FrameIdx, AttributeIdx);

							if (bTargetEventKnown)
							{
								if (bOffsetEventKnown)
								{
									OutFinalPose.SetEvent(FrameIdx, AttributeIdx, true, Alpha * OffsetTimeUntilEvent + TargetTimeUntilEvent);
								}
								else
								{
									OutFinalPose.SetEvent(FrameIdx, AttributeIdx, true, TargetTimeUntilEvent);
								}
							}
							else
							{
								OutFinalPose.SetEvent(FrameIdx, AttributeIdx, false, UE_MAX_FLT);
							}
							break;
						}
						default:
							checkNoEntry();
						}
					}
					else
					{
						UE::Learning::Array::Copy(
							OutFinalPose.AttributeData[FrameIdx].Slice(
								OutFinalPose.GetAttributeOffset(AttributeIdx),
								OutFinalPose.GetAttributeSize(AttributeIdx)),
							InPoseTarget.AttributeData[FrameIdx].Slice(
								InPoseTarget.GetAttributeOffset(AttributeIdx),
								InPoseTarget.GetAttributeSize(AttributeIdx)));
					}
				}
			}
		}

		void UpdateInplaceUsingVelocityIntegrationMix(
			const FPoseRootDataView& InOutPose,
			const FPoseRootDataConstView& InTargetPose,
			const float DeltaTime,
			const float VelocityAlpha)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(AnimDatabase::PoseData::UpdateInplaceUsingVelocityIntegrationMix);

			check(InOutPose.GetFrameNum() == InTargetPose.GetFrameNum());
			check(InOutPose.GetFrameNum() == InTargetPose.GetFrameNum());

			const int32 FrameNum = InOutPose.GetFrameNum();

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				const FVector RootLocationIntegrated = InOutPose.RootLocations[FrameIdx] + DeltaTime * (FVector)InTargetPose.RootLinearVelocities[FrameIdx];
				const FQuat4f RootRotationIntegrated = FQuat4f::MakeFromRotationVector(DeltaTime * InTargetPose.RootAngularVelocities[FrameIdx]) * InOutPose.RootRotations[FrameIdx];
				const FVector3f RootScaleIntegrated = Math::VectorExpSafe(DeltaTime * InTargetPose.RootScalarVelocities[FrameIdx]) * InOutPose.RootScales[FrameIdx];

				InOutPose.RootLocations[FrameIdx] = FMath::Lerp(InTargetPose.RootLocations[FrameIdx], RootLocationIntegrated, VelocityAlpha);
				InOutPose.RootRotations[FrameIdx] = FQuat4f::Slerp(InTargetPose.RootRotations[FrameIdx], RootRotationIntegrated, VelocityAlpha);
				InOutPose.RootScales[FrameIdx] = Math::VectorEerp(InTargetPose.RootScales[FrameIdx], RootScaleIntegrated, VelocityAlpha);
				InOutPose.RootLinearVelocities[FrameIdx] = InTargetPose.RootLinearVelocities[FrameIdx];
				InOutPose.RootAngularVelocities[FrameIdx] = InTargetPose.RootAngularVelocities[FrameIdx];
				InOutPose.RootScalarVelocities[FrameIdx] = InTargetPose.RootScalarVelocities[FrameIdx];
			}
		}

		void UpdateInplaceUsingVelocityIntegrationMix(
			const FPoseLocalBoneDataView& InOutPose,
			const FPoseLocalBoneDataConstView& InTargetPose,
			const float DeltaTime,
			const float VelocityAlpha)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(AnimDatabase::PoseData::UpdateInplaceUsingVelocityIntegrationMix);

			check(InOutPose.GetFrameNum() == InTargetPose.GetFrameNum());
			check(InOutPose.GetBoneNum() == InTargetPose.GetBoneNum());
			check(InOutPose.GetFrameNum() == InTargetPose.GetFrameNum());
			check(InOutPose.GetBoneNum() == InTargetPose.GetBoneNum());

			const int32 FrameNum = InOutPose.GetFrameNum();
			const int32 BoneNum = InOutPose.GetBoneNum();

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
				{
					const FVector3f BoneLocationIntegrated = DeltaTime * InTargetPose.BoneLinearVelocities[FrameIdx][BoneIdx] + InOutPose.BoneLocations[FrameIdx][BoneIdx];
					const FQuat4f BoneRotationIntegrated = FQuat4f::MakeFromRotationVector(DeltaTime * InTargetPose.BoneAngularVelocities[FrameIdx][BoneIdx]) * InOutPose.BoneRotations[FrameIdx][BoneIdx];
					const FVector3f BoneScaleIntegrated = Math::VectorExpSafe(DeltaTime * InTargetPose.BoneScalarVelocities[FrameIdx][BoneIdx]) * InOutPose.BoneScales[FrameIdx][BoneIdx];

					InOutPose.BoneLocations[FrameIdx][BoneIdx] = FMath::Lerp(InTargetPose.BoneLocations[FrameIdx][BoneIdx], BoneLocationIntegrated, VelocityAlpha);
					InOutPose.BoneRotations[FrameIdx][BoneIdx] = FQuat4f::Slerp(InTargetPose.BoneRotations[FrameIdx][BoneIdx], BoneRotationIntegrated, VelocityAlpha);
					InOutPose.BoneScales[FrameIdx][BoneIdx] = Math::VectorEerp(InTargetPose.BoneScales[FrameIdx][BoneIdx], BoneScaleIntegrated, VelocityAlpha);
					InOutPose.BoneLinearVelocities[FrameIdx][BoneIdx] = InTargetPose.BoneLinearVelocities[FrameIdx][BoneIdx];
					InOutPose.BoneAngularVelocities[FrameIdx][BoneIdx] = InTargetPose.BoneAngularVelocities[FrameIdx][BoneIdx];
					InOutPose.BoneScalarVelocities[FrameIdx][BoneIdx] = InTargetPose.BoneScalarVelocities[FrameIdx][BoneIdx];
				}
			}
		}

		void DampInplaceUsingVelocityIntegrationMix(
			const FPoseLocalBoneDataView& InOutPose,
			const FPoseLocalBoneDataConstView& InTargetPose,
			const float DeltaTime,
			const float PoseSmoothingTime)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(AnimDatabase::PoseData::DampInplaceUsingVelocityIntegrationMix);

			check(InOutPose.GetFrameNum() == InTargetPose.GetFrameNum());
			check(InOutPose.GetBoneNum() == InTargetPose.GetBoneNum());
			check(InOutPose.GetFrameNum() == InTargetPose.GetFrameNum());
			check(InOutPose.GetBoneNum() == InTargetPose.GetBoneNum());

			const int32 FrameNum = InOutPose.GetFrameNum();
			const int32 BoneNum = InOutPose.GetBoneNum();

#if UE_ANIMDATABASE_ISPC
			ispc::AnimDatabasePoseDampInplaceUsingVelocityIntegrationMix(
				(float*)InOutPose.BoneLocations.GetData(),
				(float*)InOutPose.BoneRotations.GetData(),
				(float*)InOutPose.BoneScales.GetData(),
				(float*)InOutPose.BoneLinearVelocities.GetData(),
				(float*)InOutPose.BoneAngularVelocities.GetData(),
				(float*)InOutPose.BoneScalarVelocities.GetData(),
				(const float*)InTargetPose.BoneLocations.GetData(),
				(const float*)InTargetPose.BoneRotations.GetData(),
				(const float*)InTargetPose.BoneScales.GetData(),
				(const float*)InTargetPose.BoneLinearVelocities.GetData(),
				(const float*)InTargetPose.BoneAngularVelocities.GetData(),
				(const float*)InTargetPose.BoneScalarVelocities.GetData(),
				DeltaTime,
				PoseSmoothingTime,
				FrameNum,
				BoneNum);
#else

			const float Factor = 1.0f - FMath::InvExpApprox(DeltaTime / FMath::Max(PoseSmoothingTime, UE_SMALL_NUMBER));

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
				{
					// Doing the integration with a 50/50 velocity mix between start and end pose is more "implicit"
					const FVector3f LinearVelocity = FMath::Lerp(InTargetPose.BoneLinearVelocities[FrameIdx][BoneIdx], InOutPose.BoneLinearVelocities[FrameIdx][BoneIdx], 0.5f);
					const FVector3f AngularVelocity = FMath::Lerp(InTargetPose.BoneAngularVelocities[FrameIdx][BoneIdx], InOutPose.BoneAngularVelocities[FrameIdx][BoneIdx], 0.5f);
					const FVector3f ScalarVelocity = FMath::Lerp(InTargetPose.BoneScalarVelocities[FrameIdx][BoneIdx], InOutPose.BoneScalarVelocities[FrameIdx][BoneIdx], 0.5f);

					const FVector3f BoneLocationIntegrated = DeltaTime * LinearVelocity + InOutPose.BoneLocations[FrameIdx][BoneIdx];
					const FQuat4f BoneRotationIntegrated = FQuat4f::MakeFromRotationVector(DeltaTime * AngularVelocity) * InOutPose.BoneRotations[FrameIdx][BoneIdx];
					const FVector3f BoneScaleIntegrated = Math::VectorExpSafe(DeltaTime * ScalarVelocity) * InOutPose.BoneScales[FrameIdx][BoneIdx];

					InOutPose.BoneLocations[FrameIdx][BoneIdx] = FMath::Lerp(BoneLocationIntegrated, InTargetPose.BoneLocations[FrameIdx][BoneIdx], Factor);
					InOutPose.BoneRotations[FrameIdx][BoneIdx] = FQuat4f::Slerp(BoneRotationIntegrated, InTargetPose.BoneRotations[FrameIdx][BoneIdx], Factor);
					InOutPose.BoneScales[FrameIdx][BoneIdx] = Math::VectorEerp(BoneScaleIntegrated, InTargetPose.BoneScales[FrameIdx][BoneIdx], Factor);
					InOutPose.BoneLinearVelocities[FrameIdx][BoneIdx] = InTargetPose.BoneLinearVelocities[FrameIdx][BoneIdx];
					InOutPose.BoneAngularVelocities[FrameIdx][BoneIdx] = InTargetPose.BoneAngularVelocities[FrameIdx][BoneIdx];
					InOutPose.BoneScalarVelocities[FrameIdx][BoneIdx] = InTargetPose.BoneScalarVelocities[FrameIdx][BoneIdx];
				}
			}
#endif
		}

		void DampInplaceUsingAdaptiveVelocityIntegrationMix(
			const FPoseLocalBoneDataView& InOutPose,
			const FPoseLocalBoneDataConstView& InTargetPose,
			const float DeltaTime,
			const float MinPoseSmoothingTime,
			const float MaxPoseSmoothingTime,
			const float LinearVelocitySmoothingThreshold,
			const float AngularVelocitySmoothingThreshold,
			const float ScalarVelocitySmoothingThreshold)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(AnimDatabase::PoseData::DampInplaceUsingAdaptiveVelocityIntegrationMix);

			check(InOutPose.GetFrameNum() == InTargetPose.GetFrameNum());
			check(InOutPose.GetBoneNum() == InTargetPose.GetBoneNum());
			check(InOutPose.GetFrameNum() == InTargetPose.GetFrameNum());
			check(InOutPose.GetBoneNum() == InTargetPose.GetBoneNum());

			const int32 FrameNum = InOutPose.GetFrameNum();
			const int32 BoneNum = InOutPose.GetBoneNum();

#if UE_ANIMDATABASE_ISPC
			ispc::AnimDatabasePoseDampInplaceUsingAdaptiveVelocityIntegrationMix(
				(float*)InOutPose.BoneLocations.GetData(),
				(float*)InOutPose.BoneRotations.GetData(),
				(float*)InOutPose.BoneScales.GetData(),
				(float*)InOutPose.BoneLinearVelocities.GetData(),
				(float*)InOutPose.BoneAngularVelocities.GetData(),
				(float*)InOutPose.BoneScalarVelocities.GetData(),
				(const float*)InTargetPose.BoneLocations.GetData(),
				(const float*)InTargetPose.BoneRotations.GetData(),
				(const float*)InTargetPose.BoneScales.GetData(),
				(const float*)InTargetPose.BoneLinearVelocities.GetData(),
				(const float*)InTargetPose.BoneAngularVelocities.GetData(),
				(const float*)InTargetPose.BoneScalarVelocities.GetData(),
				DeltaTime,
				MinPoseSmoothingTime,
				MaxPoseSmoothingTime,
				LinearVelocitySmoothingThreshold,
				AngularVelocitySmoothingThreshold,
				ScalarVelocitySmoothingThreshold,
				FrameNum,
				BoneNum);
#else

			const float ClampedMinPoseSmoothingTime = FMath::Max(MinPoseSmoothingTime, UE_SMALL_NUMBER);
			const float ClampedMaxPoseSmoothingTime = FMath::Max(MaxPoseSmoothingTime, UE_SMALL_NUMBER);

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
				{
					// Doing the integration with a 50/50 velocity mix between start and end pose is more "implicit"
					const FVector3f LinearVelocity = FMath::Lerp(InTargetPose.BoneLinearVelocities[FrameIdx][BoneIdx], InOutPose.BoneLinearVelocities[FrameIdx][BoneIdx], 0.5f);
					const FVector3f AngularVelocity = FMath::Lerp(InTargetPose.BoneAngularVelocities[FrameIdx][BoneIdx], InOutPose.BoneAngularVelocities[FrameIdx][BoneIdx], 0.5f);
					const FVector3f ScalarVelocity = FMath::Lerp(InTargetPose.BoneScalarVelocities[FrameIdx][BoneIdx], InOutPose.BoneScalarVelocities[FrameIdx][BoneIdx], 0.5f);

					const FVector3f BoneLocationIntegrated = DeltaTime * LinearVelocity + InOutPose.BoneLocations[FrameIdx][BoneIdx];
					const FQuat4f BoneRotationIntegrated = FQuat4f::MakeFromRotationVector(DeltaTime * AngularVelocity) * InOutPose.BoneRotations[FrameIdx][BoneIdx];
					const FVector3f BoneScaleIntegrated = Math::VectorExpSafe(DeltaTime * ScalarVelocity) * InOutPose.BoneScales[FrameIdx][BoneIdx];

					const float LinearVelocitySmoothingTime = (ClampedMaxPoseSmoothingTime * LinearVelocitySmoothingThreshold) / FMath::Max(LinearVelocity.Length(), UE_SMALL_NUMBER);
					const float AngularVelocitySmoothingTime = (ClampedMaxPoseSmoothingTime * AngularVelocitySmoothingThreshold) / FMath::Max(AngularVelocity.Length(), UE_SMALL_NUMBER);
					const float ScalarVelocitySmoothingTime = (ClampedMaxPoseSmoothingTime * ScalarVelocitySmoothingThreshold) / FMath::Max(ScalarVelocity.Length(), UE_SMALL_NUMBER);

					const float LinearVelocityFactor = 1.0f - FMath::InvExpApprox(DeltaTime / FMath::Clamp(LinearVelocitySmoothingTime, ClampedMinPoseSmoothingTime, ClampedMaxPoseSmoothingTime));
					const float AngularVelocityFactor = 1.0f - FMath::InvExpApprox(DeltaTime / FMath::Clamp(AngularVelocitySmoothingTime, ClampedMinPoseSmoothingTime, ClampedMaxPoseSmoothingTime));
					const float ScalarVelocityFactor = 1.0f - FMath::InvExpApprox(DeltaTime / FMath::Clamp(ScalarVelocitySmoothingTime, ClampedMinPoseSmoothingTime, ClampedMaxPoseSmoothingTime));

					InOutPose.BoneLocations[FrameIdx][BoneIdx] = FMath::Lerp(BoneLocationIntegrated, InTargetPose.BoneLocations[FrameIdx][BoneIdx], LinearVelocityFactor);
					InOutPose.BoneRotations[FrameIdx][BoneIdx] = FQuat4f::Slerp(BoneRotationIntegrated, InTargetPose.BoneRotations[FrameIdx][BoneIdx], AngularVelocityFactor);
					InOutPose.BoneScales[FrameIdx][BoneIdx] = Math::VectorEerp(BoneScaleIntegrated, InTargetPose.BoneScales[FrameIdx][BoneIdx], ScalarVelocityFactor);
					InOutPose.BoneLinearVelocities[FrameIdx][BoneIdx] = InTargetPose.BoneLinearVelocities[FrameIdx][BoneIdx];
					InOutPose.BoneAngularVelocities[FrameIdx][BoneIdx] = InTargetPose.BoneAngularVelocities[FrameIdx][BoneIdx];
					InOutPose.BoneScalarVelocities[FrameIdx][BoneIdx] = InTargetPose.BoneScalarVelocities[FrameIdx][BoneIdx];
				}
			}

#endif
		}

		void DampFramesInplaceUsingAdaptiveVelocityIntegrationMix(
			const FPoseLocalBoneDataView& InOutPoses,
			const float DeltaTime,
			const float MinPoseSmoothingTime,
			const float MaxPoseSmoothingTime,
			const float LinearVelocitySmoothingThreshold,
			const float AngularVelocitySmoothingThreshold,
			const float ScalarVelocitySmoothingThreshold)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(AnimDatabase::PoseData::DampFramesInplaceUsingAdaptiveVelocityIntegrationMix);

			const float ClampedMinPoseSmoothingTime = FMath::Max(MinPoseSmoothingTime, UE_SMALL_NUMBER);
			const float ClampedMaxPoseSmoothingTime = FMath::Max(MaxPoseSmoothingTime, UE_SMALL_NUMBER);

			const int32 FrameNum = InOutPoses.GetFrameNum();
			const int32 BoneNum = InOutPoses.GetBoneNum();

			for (int32 FrameIdx = 1; FrameIdx < FrameNum; FrameIdx++)
			{
				for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
				{
					// Doing the integration with a 50/50 velocity mix between start and end pose is more "implicit"
					const FVector3f LinearVelocity = FMath::Lerp(InOutPoses.BoneLinearVelocities[FrameIdx - 1][BoneIdx], InOutPoses.BoneLinearVelocities[FrameIdx][BoneIdx], 0.5f);
					const FVector3f AngularVelocity = FMath::Lerp(InOutPoses.BoneAngularVelocities[FrameIdx - 1][BoneIdx], InOutPoses.BoneAngularVelocities[FrameIdx][BoneIdx], 0.5f);
					const FVector3f ScalarVelocity = FMath::Lerp(InOutPoses.BoneScalarVelocities[FrameIdx - 1][BoneIdx], InOutPoses.BoneScalarVelocities[FrameIdx][BoneIdx], 0.5f);

					const FVector3f BoneLocationIntegrated = DeltaTime * LinearVelocity + InOutPoses.BoneLocations[FrameIdx - 1][BoneIdx];
					const FQuat4f BoneRotationIntegrated = FQuat4f::MakeFromRotationVector(DeltaTime * AngularVelocity) * InOutPoses.BoneRotations[FrameIdx - 1][BoneIdx];
					const FVector3f BoneScaleIntegrated = Math::VectorExpSafe(DeltaTime * ScalarVelocity) * InOutPoses.BoneScales[FrameIdx - 1][BoneIdx];

					const float LinearVelocitySmoothingTime = (ClampedMaxPoseSmoothingTime * LinearVelocitySmoothingThreshold) / FMath::Max(LinearVelocity.Length(), UE_SMALL_NUMBER);
					const float AngularVelocitySmoothingTime = (ClampedMaxPoseSmoothingTime * AngularVelocitySmoothingThreshold) / FMath::Max(AngularVelocity.Length(), UE_SMALL_NUMBER);
					const float ScalarVelocitySmoothingTime = (ClampedMaxPoseSmoothingTime * ScalarVelocitySmoothingThreshold) / FMath::Max(ScalarVelocity.Length(), UE_SMALL_NUMBER);

					const float LinearVelocityFactor = 1.0f - FMath::InvExpApprox(DeltaTime / FMath::Clamp(LinearVelocitySmoothingTime, ClampedMinPoseSmoothingTime, ClampedMaxPoseSmoothingTime));
					const float AngularVelocityFactor = 1.0f - FMath::InvExpApprox(DeltaTime / FMath::Clamp(AngularVelocitySmoothingTime, ClampedMinPoseSmoothingTime, ClampedMaxPoseSmoothingTime));
					const float ScalarVelocityFactor = 1.0f - FMath::InvExpApprox(DeltaTime / FMath::Clamp(ScalarVelocitySmoothingTime, ClampedMinPoseSmoothingTime, ClampedMaxPoseSmoothingTime));

					InOutPoses.BoneLocations[FrameIdx][BoneIdx] = FMath::Lerp(BoneLocationIntegrated, InOutPoses.BoneLocations[FrameIdx][BoneIdx], LinearVelocityFactor);
					InOutPoses.BoneRotations[FrameIdx][BoneIdx] = FQuat4f::Slerp(BoneRotationIntegrated, InOutPoses.BoneRotations[FrameIdx][BoneIdx], AngularVelocityFactor);
					InOutPoses.BoneScales[FrameIdx][BoneIdx] = Math::VectorEerp(BoneScaleIntegrated, InOutPoses.BoneScales[FrameIdx][BoneIdx], ScalarVelocityFactor);
				}
			}
		}

		void DampAttributesInPlace(
			const FPoseAttributeDataView& InOutAttributes,
			const FPoseAttributeDataConstView& InTargetAttributes,
			const float DeltaTime,
			const float SmoothingTime)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(AnimDatabase::PoseData::DampAttributesInPlace);

			check(InOutAttributes.GetFrameNum() == InTargetAttributes.GetFrameNum());
			check(Learning::Array::Equal(InOutAttributes.GetAttributeTypes(), InTargetAttributes.GetAttributeTypes()));

			const int32 FrameNum = InOutAttributes.GetFrameNum();
			const int32 AttributeNum = InOutAttributes.GetAttributeNum();

			const float Factor = 1.0f - FMath::InvExpApprox(DeltaTime / FMath::Max(SmoothingTime, UE_SMALL_NUMBER));

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				for (int32 AttributeIdx = 0; AttributeIdx < AttributeNum; AttributeIdx++)
				{
					const EAnimDatabaseAttributeType AttributeType = InOutAttributes.GetAttributeType(AttributeIdx);
					const bool bCurrentActive = InOutAttributes.GetAttributeActive(FrameIdx, AttributeIdx);
					const bool bTargetActive = InTargetAttributes.GetAttributeActive(FrameIdx, AttributeIdx);

					InOutAttributes.SetAttributeActive(FrameIdx, AttributeIdx, bTargetActive);

					if (bTargetActive && bCurrentActive)
					{
						switch (AttributeType)
						{
						case EAnimDatabaseAttributeType::Null: break;
						case EAnimDatabaseAttributeType::Bool: InOutAttributes.SetBool(FrameIdx, AttributeIdx, InTargetAttributes.GetBool(FrameIdx, AttributeIdx)); break;
						case EAnimDatabaseAttributeType::Float: InOutAttributes.SetFloat(FrameIdx, AttributeIdx, FMath::Lerp(InOutAttributes.GetFloat(FrameIdx, AttributeIdx), InTargetAttributes.GetFloat(FrameIdx, AttributeIdx), Factor)); break;
						case EAnimDatabaseAttributeType::Angle: InOutAttributes.SetAngle(FrameIdx, AttributeIdx, InOutAttributes.GetAngle(FrameIdx, AttributeIdx) + Factor * FMath::FindDeltaAngleRadians(InOutAttributes.GetAngle(FrameIdx, AttributeIdx), InTargetAttributes.GetAngle(FrameIdx, AttributeIdx))); break;
						case EAnimDatabaseAttributeType::Location: InOutAttributes.SetLocation(FrameIdx, AttributeIdx, FMath::Lerp(InOutAttributes.GetLocation(FrameIdx, AttributeIdx), InTargetAttributes.GetLocation(FrameIdx, AttributeIdx), Factor)); break;
						case EAnimDatabaseAttributeType::Rotation: InOutAttributes.SetRotation(FrameIdx, AttributeIdx, FQuat4f::Slerp(InOutAttributes.GetRotation(FrameIdx, AttributeIdx), InTargetAttributes.GetRotation(FrameIdx, AttributeIdx), Factor)); break;
						case EAnimDatabaseAttributeType::Scale: InOutAttributes.SetScale(FrameIdx, AttributeIdx, Math::VectorEerp(InOutAttributes.GetScale(FrameIdx, AttributeIdx), InTargetAttributes.GetScale(FrameIdx, AttributeIdx), Factor)); break;
						case EAnimDatabaseAttributeType::LinearVelocity: InOutAttributes.SetLinearVelocity(FrameIdx, AttributeIdx, FMath::Lerp(InOutAttributes.GetLinearVelocity(FrameIdx, AttributeIdx), InTargetAttributes.GetLinearVelocity(FrameIdx, AttributeIdx), Factor)); break;
						case EAnimDatabaseAttributeType::AngularVelocity: InOutAttributes.SetAngularVelocity(FrameIdx, AttributeIdx, FMath::Lerp(InOutAttributes.GetAngularVelocity(FrameIdx, AttributeIdx), InTargetAttributes.GetAngularVelocity(FrameIdx, AttributeIdx), Factor)); break;
						case EAnimDatabaseAttributeType::ScalarVelocity: InOutAttributes.SetScalarVelocity(FrameIdx, AttributeIdx, FMath::Lerp(InOutAttributes.GetScalarVelocity(FrameIdx, AttributeIdx), InTargetAttributes.GetScalarVelocity(FrameIdx, AttributeIdx), Factor)); break;
						case EAnimDatabaseAttributeType::Direction: InOutAttributes.SetDirection(FrameIdx, AttributeIdx, FMath::Lerp(InOutAttributes.GetDirection(FrameIdx, AttributeIdx), InTargetAttributes.GetDirection(FrameIdx, AttributeIdx), Factor).GetSafeNormal(UE_SMALL_NUMBER, FVector3f::ForwardVector)); break;
						case EAnimDatabaseAttributeType::Transform:
						{
							const FTransform3f TargetTransform = InTargetAttributes.GetTransform(FrameIdx, AttributeIdx);
							const FTransform3f CurrentTransform = InOutAttributes.GetTransform(FrameIdx, AttributeIdx);
							FTransform3f OutTransform = FTransform3f::Identity;
							OutTransform.SetLocation(FMath::Lerp(CurrentTransform.GetLocation(), TargetTransform.GetLocation(), Factor));
							OutTransform.SetRotation(FQuat4f::Slerp(CurrentTransform.GetRotation(), TargetTransform.GetRotation(), Factor));
							OutTransform.SetScale3D(Math::VectorEerp(CurrentTransform.GetScale3D(), TargetTransform.GetScale3D(), Factor));
							InOutAttributes.SetTransform(FrameIdx, AttributeIdx, OutTransform);
							break;
						}

						case EAnimDatabaseAttributeType::Event:
						{
							bool bTargetEventKnown = false, bCurrentEventKnown = false;
							float TargetTimeUntilEvent = UE_MAX_FLT, CurrentTimeUntilEvent = UE_MAX_FLT;
							InTargetAttributes.GetEvent(bTargetEventKnown, TargetTimeUntilEvent, FrameIdx, AttributeIdx);
							InOutAttributes.GetEvent(bCurrentEventKnown, CurrentTimeUntilEvent, FrameIdx, AttributeIdx);

							if (bTargetEventKnown)
							{
								if (bCurrentEventKnown)
								{
									InOutAttributes.SetEvent(FrameIdx, AttributeIdx, true, FMath::Lerp(CurrentTimeUntilEvent, TargetTimeUntilEvent, Factor));
								}
								else
								{
									InOutAttributes.SetEvent(FrameIdx, AttributeIdx, true, TargetTimeUntilEvent);
								}
							}
							else
							{
								InOutAttributes.SetEvent(FrameIdx, AttributeIdx, false, UE_MAX_FLT);
							}
							break;
						}
						default:
							checkNoEntry();
						}
					}
					else
					{
						UE::Learning::Array::Copy(
							InOutAttributes.AttributeData[FrameIdx].Slice(
								InOutAttributes.GetAttributeOffset(AttributeIdx),
								InOutAttributes.GetAttributeSize(AttributeIdx)),
							InTargetAttributes.AttributeData[FrameIdx].Slice(
								InTargetAttributes.GetAttributeOffset(AttributeIdx),
								InTargetAttributes.GetAttributeSize(AttributeIdx)));
					}
				}
			}
		}

		namespace ContactPinning
		{
			FContactStateConstView FContactStateConstView::Slice(const int32 FrameStart, const int32 FrameNum) const
			{
				return
				{
					Locked.Slice(FrameStart, FrameNum),
					TimeSinceTransition.Slice(FrameStart, FrameNum),
					Position.Slice(FrameStart, FrameNum),
					Velocity.Slice(FrameStart, FrameNum),
					Point.Slice(FrameStart, FrameNum),
					OffsetPosition.Slice(FrameStart, FrameNum),
					OffsetVelocity.Slice(FrameStart, FrameNum)
				};
			}

			FContactStateView FContactStateView::Slice(const int32 FrameStart, const int32 FrameNum) const
			{
				return
				{
					Locked.Slice(FrameStart, FrameNum),
					TimeSinceTransition.Slice(FrameStart, FrameNum),
					Position.Slice(FrameStart, FrameNum),
					Velocity.Slice(FrameStart, FrameNum),
					Point.Slice(FrameStart, FrameNum),
					OffsetPosition.Slice(FrameStart, FrameNum),
					OffsetVelocity.Slice(FrameStart, FrameNum)
				};
			}

			void FContactState::Resize(const int32 FrameNum, const int32 BoneNum)
			{
				Locked.SetNumUninitialized({ FrameNum, BoneNum });
				TimeSinceTransition.SetNumUninitialized({ FrameNum, BoneNum });
				Position.SetNumUninitialized({ FrameNum, BoneNum });
				Velocity.SetNumUninitialized({ FrameNum, BoneNum });
				Point.SetNumUninitialized({ FrameNum, BoneNum });
				OffsetPosition.SetNumUninitialized({ FrameNum, BoneNum });
				OffsetVelocity.SetNumUninitialized({ FrameNum, BoneNum });
			}

			bool FContactState::IsEmpty() const { return Locked.IsEmpty(); }

			void FContactState::Empty()
			{
				Locked.Empty();
				TimeSinceTransition.Empty();
				Position.Empty();
				Velocity.Empty();
				Point.Empty();
				OffsetPosition.Empty();
				OffsetVelocity.Empty();
			}

			FContactStateView FContactState::View()
			{
				return
				{
					Locked,
					TimeSinceTransition,
					Position,
					Velocity,
					Point,
					OffsetPosition,
					OffsetVelocity
				};
			}

			FContactStateConstView FContactState::ConstView() const
			{
				return
				{
					Locked,
					TimeSinceTransition,
					Position,
					Velocity,
					Point,
					OffsetPosition,
					OffsetVelocity
				};
			}

			FContactStateView FContactState::Slice(const int32 FrameStart, const int32 FrameNum)
			{
				return
				{
					Locked.Slice(FrameStart, FrameNum),
					TimeSinceTransition.Slice(FrameStart, FrameNum),
					Position.Slice(FrameStart, FrameNum),
					Velocity.Slice(FrameStart, FrameNum),
					Point.Slice(FrameStart, FrameNum),
					OffsetPosition.Slice(FrameStart, FrameNum),
					OffsetVelocity.Slice(FrameStart, FrameNum)
				};
			}

			FContactStateConstView FContactState::ConstSlice(const int32 FrameStart, const int32 FrameNum) const
			{
				return
				{
					Locked.Slice(FrameStart, FrameNum),
					TimeSinceTransition.Slice(FrameStart, FrameNum),
					Position.Slice(FrameStart, FrameNum),
					Velocity.Slice(FrameStart, FrameNum),
					Point.Slice(FrameStart, FrameNum),
					OffsetPosition.Slice(FrameStart, FrameNum),
					OffsetVelocity.Slice(FrameStart, FrameNum)
				};
			}

			void ContactLookAtIK(
				FQuat4f& InOutBoneRotation,
				const FQuat4f GlobalParentRotation,
				const FQuat4f GlobalRotation,
				const FVector GlobalPosition,
				const FVector ChildPosition,
				const FVector TargetPosition,
				const float Eps)
			{
				const FVector3f CurrDir = ((FVector3f)(ChildPosition - GlobalPosition)).GetSafeNormal(Eps, FVector3f::ForwardVector);
				const FVector3f TargDir = ((FVector3f)(TargetPosition - GlobalPosition)).GetSafeNormal(Eps, FVector3f::ForwardVector);

				if (FMath::Abs(1.0f - CurrDir.Dot(TargDir)) > Eps)
				{
					InOutBoneRotation = GlobalParentRotation.Inverse() * (FQuat4f::FindBetweenNormals(CurrDir, TargDir) * GlobalRotation);
				}
			}

			void ContactTwoBoneIK(
				FQuat4f& BoneRootLocalRotation,
				FQuat4f& BoneMidLocalRotation,
				const FVector BoneRoot,
				const FVector BoneMid,
				const FVector BoneEnd,
				const FVector Target,
				const FVector3f RotationAxis,
				const FQuat4f BoneRootGlobalRotation,
				const FQuat4f BoneMidGlobalRotation,
				const FQuat4f BoneParentGlobalRotation,
				const float MaxLengthBuffer)
			{
				const float MaxExtension = FVector::Distance(BoneRoot, BoneMid) + FVector::Distance(BoneMid, BoneEnd) - MaxLengthBuffer;

				const float TargetDistance = FVector::Distance(Target, BoneRoot);
				FVector TargetClamp = Target;

				if (TargetDistance > MaxExtension)
				{
					const float Saturation = (1.0f - FMath::InvExpApprox((TargetDistance - MaxExtension) / FMath::Max(MaxLengthBuffer, UE_SMALL_NUMBER)));
					TargetClamp = BoneRoot + (MaxExtension + MaxLengthBuffer * Saturation) * (Target - BoneRoot).GetUnsafeNormal();
				}

				const FVector3f AxisDwn = ((FVector3f)(BoneEnd - BoneRoot)).GetSafeNormal(UE_SMALL_NUMBER, FVector3f::ForwardVector);
				const FVector3f AxisFwd = RotationAxis.Cross(AxisDwn).GetSafeNormal(UE_SMALL_NUMBER, FVector3f::UpVector);
				const FVector3f AxisRot = AxisFwd.Cross(AxisDwn).GetSafeNormal(UE_SMALL_NUMBER, FVector3f::UpVector);

				const FVector A = BoneRoot;
				const FVector B = BoneMid;
				const FVector C = BoneEnd;
				const FVector T = TargetClamp;

				const float LAB = FVector::Distance(B, A);
				const float LCB = FVector::Distance(B, C);
				const float LAT = FVector::Distance(T, A);

				const float ACAB0 = FMath::Acos(((FVector3f)(C - A)).GetSafeNormal(UE_SMALL_NUMBER, FVector3f::ForwardVector).Dot(((FVector3f)(B - A)).GetSafeNormal(UE_SMALL_NUMBER, FVector3f::ForwardVector)));
				const float BABC0 = FMath::Acos(((FVector3f)(A - B)).GetSafeNormal(UE_SMALL_NUMBER, FVector3f::ForwardVector).Dot(((FVector3f)(C - B)).GetSafeNormal(UE_SMALL_NUMBER, FVector3f::ForwardVector)));

				const float ACAB1 = FMath::Acos((LAB * LAB + LAT * LAT - LCB * LCB) / FMath::Max(2.0f * LAB * LAT, UE_SMALL_NUMBER));
				const float BABC1 = FMath::Acos((LAB * LAB + LCB * LCB - LAT * LAT) / FMath::Max(2.0f * LAB * LCB, UE_SMALL_NUMBER));

				const FQuat4f R0 = FQuat4f::MakeFromRotationVector((ACAB1 - ACAB0) * AxisRot);
				const FQuat4f R1 = FQuat4f::MakeFromRotationVector((BABC1 - BABC0) * AxisRot);
				const FQuat4f R2 = FQuat4f::FindBetween((FVector3f)(BoneEnd - BoneRoot), (FVector3f)(TargetClamp - BoneRoot));

				BoneRootLocalRotation = BoneParentGlobalRotation.Inverse() * (R2 * (R0 * BoneRootGlobalRotation));
				BoneMidLocalRotation = BoneRootGlobalRotation.Inverse() * (R1 * BoneMidGlobalRotation);
			}

			void ResetContactPinning(
				const FContactStateView& OutContactState,
				const FPoseGlobalBoneDataConstView& InPoseGlobalBoneData,
				const TArrayView<const int32> ToeBoneIndices)
			{
				check(OutContactState.GetFrameNum() == InPoseGlobalBoneData.GetFrameNum());
				check(OutContactState.GetBoneNum() == ToeBoneIndices.Num());

				const int32 FrameNum = OutContactState.GetFrameNum();
				const int32 BoneNum = OutContactState.GetBoneNum();

				for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
				{
					for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
					{
						OutContactState.Locked[FrameIdx][BoneIdx] = false;
						OutContactState.TimeSinceTransition[FrameIdx][BoneIdx] = 0.0f;
						OutContactState.Position[FrameIdx][BoneIdx] = InPoseGlobalBoneData.BoneLocations[FrameIdx][ToeBoneIndices[BoneIdx]];
						OutContactState.Velocity[FrameIdx][BoneIdx] = InPoseGlobalBoneData.BoneLinearVelocities[FrameIdx][ToeBoneIndices[BoneIdx]];
						OutContactState.Point[FrameIdx][BoneIdx] = InPoseGlobalBoneData.BoneLocations[FrameIdx][ToeBoneIndices[BoneIdx]];
						OutContactState.OffsetPosition[FrameIdx][BoneIdx] = FVector3f::ZeroVector;
						OutContactState.OffsetVelocity[FrameIdx][BoneIdx] = FVector3f::ZeroVector;
					}
				}
			}

			void UpdateContactPinning(
				const FContactStateView& InOutContactState,
				const FPoseGlobalBoneDataView& InOutPoseGlobalBoneData,
				const FPoseLocalBoneDataView& InOutPoseLocalBoneData,
				const FPoseRootDataConstView& InPoseRootData,
				const FPoseAttributeDataConstView& InPoseAttributeData,
				const TArrayView<const int32> InBoneParents,
				const TArrayView<const int32> ContactStateAttributeIndices,
				const TArrayView<const int32> PelvisBoneIndices,
				const TArrayView<const int32> HipBoneIndices,
				const TArrayView<const int32> KneeBoneIndices,
				const TArrayView<const int32> AnkleBoneIndices,
				const TArrayView<const int32> ToeBoneIndices,
				const TArrayView<const FVector3f> KneeSideVectors,
				const TArrayView<const FVector3f> ToeForwardVectors,
				const float DeltaTime,
				const FContactPinningSettings& Settings)
			{
				check(InOutContactState.GetFrameNum() == InOutPoseGlobalBoneData.GetFrameNum());
				check(InOutContactState.GetFrameNum() == InOutPoseLocalBoneData.GetFrameNum());
				check(InOutContactState.GetFrameNum() == InPoseRootData.GetFrameNum());
				check(InOutContactState.GetFrameNum() == InPoseAttributeData.GetFrameNum());
				check(InOutContactState.GetBoneNum() == ContactStateAttributeIndices.Num());
				check(InOutContactState.GetBoneNum() == PelvisBoneIndices.Num());
				check(InOutContactState.GetBoneNum() == HipBoneIndices.Num());
				check(InOutContactState.GetBoneNum() == KneeBoneIndices.Num());
				check(InOutContactState.GetBoneNum() == AnkleBoneIndices.Num());
				check(InOutContactState.GetBoneNum() == ToeBoneIndices.Num());
				check(InOutContactState.GetBoneNum() == KneeSideVectors.Num());
				check(InOutContactState.GetBoneNum() == ToeForwardVectors.Num());

				const int32 FrameNum = InOutContactState.GetFrameNum();
				const int32 BoneNum = InOutContactState.GetBoneNum();

				TArray<int32, TInlineAllocator<32>> RequiredBones;
				TArray<int32, TInlineAllocator<32>> Lhs;
				TArray<int32, TInlineAllocator<32>> Rhs;
				for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
				{
					Lhs = RequiredBones;
					Rhs.SetNumUninitialized(Math::BoneAscendantsInclusiveNum(ToeBoneIndices[BoneIdx], InBoneParents));
					Math::BoneAscendantsInclusive(Rhs, ToeBoneIndices[BoneIdx], InBoneParents);
					RequiredBones.SetNumUninitialized(Math::BoneUnionNum(Lhs, Rhs));
					Math::BoneUnion(RequiredBones, Lhs, Rhs);
				}

				PoseData::ForwardKinematicsPartial(
					InOutPoseGlobalBoneData,
					InOutPoseLocalBoneData.ConstView(),
					InPoseRootData,
					InBoneParents,
					UE::Learning::FIndexSet(0, InBoneParents.Num()),
					RequiredBones);

				for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
				{
					for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
					{
						const int32 AttributeIdx = ContactStateAttributeIndices[BoneIdx];

						const FVector ContactPosition = InOutPoseGlobalBoneData.BoneLocations[FrameIdx][ToeBoneIndices[BoneIdx]];
						const FVector3f ContactVelocity = InOutPoseGlobalBoneData.BoneLinearVelocities[FrameIdx][ToeBoneIndices[BoneIdx]];
						const bool bContactState =
							(AttributeIdx >= 0 && AttributeIdx < InPoseAttributeData.GetAttributeNum()) &&
							InPoseAttributeData.GetAttributeType(AttributeIdx) == EAnimDatabaseAttributeType::Float &&
							InPoseAttributeData.GetAttributeActive(FrameIdx, AttributeIdx) &&
							InPoseAttributeData.GetFloat(FrameIdx, AttributeIdx) > 0.5f;

						UE::AnimDatabase::Math::LocationInertializeCubicUpdate(
							InOutContactState.Position[FrameIdx][BoneIdx],
							InOutContactState.Velocity[FrameIdx][BoneIdx],
							InOutContactState.TimeSinceTransition[FrameIdx][BoneIdx],
							InOutContactState.Locked[FrameIdx][BoneIdx] ? InOutContactState.Point[FrameIdx][BoneIdx] : ContactPosition,
							InOutContactState.Locked[FrameIdx][BoneIdx] ? FVector3f::ZeroVector : ContactVelocity,
							InOutContactState.OffsetPosition[FrameIdx][BoneIdx],
							InOutContactState.OffsetVelocity[FrameIdx][BoneIdx],
							DeltaTime,
							Settings.BlendTime);

						const float LockDistanceSquared = (InOutContactState.Position[FrameIdx][BoneIdx] - ContactPosition).SquaredLength();

						if (!InOutContactState.Locked[FrameIdx][BoneIdx] && bContactState && LockDistanceSquared < FMath::Square(Settings.LockRadius))
						{
							InOutContactState.Locked[FrameIdx][BoneIdx] = true;
							InOutContactState.Point[FrameIdx][BoneIdx] = ContactPosition;
							InOutContactState.Point[FrameIdx][BoneIdx].Z = Settings.ContactHeight;

							UE::AnimDatabase::Math::LocationInertializeCubicTransition(
								InOutContactState.OffsetPosition[FrameIdx][BoneIdx],
								InOutContactState.OffsetVelocity[FrameIdx][BoneIdx],
								InOutContactState.TimeSinceTransition[FrameIdx][BoneIdx],
								ContactPosition,
								ContactVelocity,
								InOutContactState.Point[FrameIdx][BoneIdx],
								FVector3f::ZeroVector,
								Settings.BlendTime);
						}
						else if (InOutContactState.Locked[FrameIdx][BoneIdx] && (!bContactState || LockDistanceSquared > FMath::Square(Settings.UnlockRadius)))
						{
							InOutContactState.Locked[FrameIdx][BoneIdx] = false;

							UE::AnimDatabase::Math::LocationInertializeCubicTransition(
								InOutContactState.OffsetPosition[FrameIdx][BoneIdx],
								InOutContactState.OffsetVelocity[FrameIdx][BoneIdx],
								InOutContactState.TimeSinceTransition[FrameIdx][BoneIdx],
								InOutContactState.Point[FrameIdx][BoneIdx],
								FVector3f::ZeroVector,
								ContactPosition,
								ContactVelocity,
								Settings.BlendTime);
						}

						FVector ToePosition = InOutContactState.Position[FrameIdx][BoneIdx];
						ToePosition.Z = FMath::Max(ToePosition.Z, Settings.MinToeHeight);

						FVector AnklePosition = ToePosition + (InOutPoseGlobalBoneData.BoneLocations[FrameIdx][AnkleBoneIndices[BoneIdx]] - InOutPoseGlobalBoneData.BoneLocations[FrameIdx][ToeBoneIndices[BoneIdx]]);
						AnklePosition.Z = FMath::Max(AnklePosition.Z, Settings.MinAnkleHeight);

						ContactTwoBoneIK(
							InOutPoseLocalBoneData.BoneRotations[FrameIdx][HipBoneIndices[BoneIdx]],
							InOutPoseLocalBoneData.BoneRotations[FrameIdx][KneeBoneIndices[BoneIdx]],
							InOutPoseGlobalBoneData.BoneLocations[FrameIdx][HipBoneIndices[BoneIdx]],
							InOutPoseGlobalBoneData.BoneLocations[FrameIdx][KneeBoneIndices[BoneIdx]],
							InOutPoseGlobalBoneData.BoneLocations[FrameIdx][AnkleBoneIndices[BoneIdx]],
							AnklePosition,
							InOutPoseGlobalBoneData.BoneRotations[FrameIdx][KneeBoneIndices[BoneIdx]].RotateVector(KneeSideVectors[BoneIdx]),
							InOutPoseGlobalBoneData.BoneRotations[FrameIdx][HipBoneIndices[BoneIdx]],
							InOutPoseGlobalBoneData.BoneRotations[FrameIdx][KneeBoneIndices[BoneIdx]],
							InOutPoseGlobalBoneData.BoneRotations[FrameIdx][PelvisBoneIndices[BoneIdx]],
							Settings.HyperExtensionLimit);
					}
				}

				PoseData::ForwardKinematicsPartial(
					InOutPoseGlobalBoneData,
					InOutPoseLocalBoneData.ConstView(),
					InPoseRootData,
					InBoneParents,
					UE::Learning::FIndexSet(0, InBoneParents.Num()),
					RequiredBones);

				for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
				{
					for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
					{
						const FVector ToePosition = InOutContactState.Position[FrameIdx][BoneIdx];

						ContactLookAtIK(
							InOutPoseLocalBoneData.BoneRotations[FrameIdx][AnkleBoneIndices[BoneIdx]],
							InOutPoseGlobalBoneData.BoneRotations[FrameIdx][KneeBoneIndices[BoneIdx]],
							InOutPoseGlobalBoneData.BoneRotations[FrameIdx][AnkleBoneIndices[BoneIdx]],
							InOutPoseGlobalBoneData.BoneLocations[FrameIdx][AnkleBoneIndices[BoneIdx]],
							InOutPoseGlobalBoneData.BoneLocations[FrameIdx][ToeBoneIndices[BoneIdx]],
							ToePosition);
					}
				}

				PoseData::ForwardKinematicsPartial(
					InOutPoseGlobalBoneData,
					InOutPoseLocalBoneData.ConstView(),
					InPoseRootData,
					InBoneParents,
					UE::Learning::FIndexSet(0, InBoneParents.Num()),
					RequiredBones);

				for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
				{
					for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
					{
						const FVector ToeEndCurr =
							(FVector)InOutPoseGlobalBoneData.BoneRotations[FrameIdx][ToeBoneIndices[BoneIdx]].RotateVector(Settings.ToeLength * ToeForwardVectors[BoneIdx]) +
							InOutPoseGlobalBoneData.BoneLocations[FrameIdx][ToeBoneIndices[BoneIdx]];

						FVector ToeEndTarg = ToeEndCurr;
						ToeEndTarg.Z = FMath::Max(ToeEndTarg.Z, Settings.MinToeHeight);

						ContactLookAtIK(
							InOutPoseLocalBoneData.BoneRotations[FrameIdx][ToeBoneIndices[BoneIdx]],
							InOutPoseGlobalBoneData.BoneRotations[FrameIdx][AnkleBoneIndices[BoneIdx]],
							InOutPoseGlobalBoneData.BoneRotations[FrameIdx][ToeBoneIndices[BoneIdx]],
							InOutPoseGlobalBoneData.BoneLocations[FrameIdx][ToeBoneIndices[BoneIdx]],
							ToeEndCurr,
							ToeEndTarg);
					}
				}
			}

			void RemoveFootGroundPenetration(
				const FPoseGlobalBoneDataView& InOutPoseGlobalBoneData,
				const FPoseLocalBoneDataView& InOutPoseLocalBoneData,
				const FPoseRootDataConstView& InPoseRootData,
				const TArrayView<const int32> InBoneParents,
				const TArrayView<const int32> PelvisBoneIndices,
				const TArrayView<const int32> HipBoneIndices,
				const TArrayView<const int32> KneeBoneIndices,
				const TArrayView<const int32> AnkleBoneIndices,
				const TArrayView<const int32> ToeBoneIndices,
				const TArrayView<const FVector3f> KneeSideVectors,
				const TArrayView<const FVector3f> ToeForwardVectors,
				const FContactPinningSettings& Settings)
			{
				check(InOutPoseGlobalBoneData.GetFrameNum() == InOutPoseLocalBoneData.GetFrameNum());
				check(InOutPoseGlobalBoneData.GetFrameNum() == InPoseRootData.GetFrameNum());
				check(PelvisBoneIndices.Num() == HipBoneIndices.Num());
				check(PelvisBoneIndices.Num() == KneeBoneIndices.Num());
				check(PelvisBoneIndices.Num() == AnkleBoneIndices.Num());
				check(PelvisBoneIndices.Num() == ToeBoneIndices.Num());
				check(PelvisBoneIndices.Num() == KneeSideVectors.Num());
				check(PelvisBoneIndices.Num() == ToeForwardVectors.Num());

				const int32 FrameNum = InOutPoseGlobalBoneData.GetFrameNum();
				const int32 BoneNum = PelvisBoneIndices.Num();

				TArray<int32, TInlineAllocator<32>> RequiredBones;
				TArray<int32, TInlineAllocator<32>> Lhs;
				TArray<int32, TInlineAllocator<32>> Rhs;
				for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
				{
					Lhs = RequiredBones;
					Rhs.SetNumUninitialized(Math::BoneAscendantsInclusiveNum(ToeBoneIndices[BoneIdx], InBoneParents));
					Math::BoneAscendantsInclusive(Rhs, ToeBoneIndices[BoneIdx], InBoneParents);
					RequiredBones.SetNumUninitialized(Math::BoneUnionNum(Lhs, Rhs));
					Math::BoneUnion(RequiredBones, Lhs, Rhs);
				}

				PoseData::ForwardKinematicsPartial(
					InOutPoseGlobalBoneData,
					InOutPoseLocalBoneData.ConstView(),
					InPoseRootData,
					InBoneParents,
					UE::Learning::FIndexSet(0, InBoneParents.Num()),
					RequiredBones);

				for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
				{
					for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
					{
						FVector ToePosition = InOutPoseGlobalBoneData.BoneLocations[FrameIdx][ToeBoneIndices[BoneIdx]];
						ToePosition.Z = FMath::Max(ToePosition.Z, Settings.MinToeHeight);

						FVector AnklePosition = ToePosition + (InOutPoseGlobalBoneData.BoneLocations[FrameIdx][AnkleBoneIndices[BoneIdx]] - InOutPoseGlobalBoneData.BoneLocations[FrameIdx][ToeBoneIndices[BoneIdx]]);
						AnklePosition.Z = FMath::Max(AnklePosition.Z, Settings.MinAnkleHeight);

						ContactTwoBoneIK(
							InOutPoseLocalBoneData.BoneRotations[FrameIdx][HipBoneIndices[BoneIdx]],
							InOutPoseLocalBoneData.BoneRotations[FrameIdx][KneeBoneIndices[BoneIdx]],
							InOutPoseGlobalBoneData.BoneLocations[FrameIdx][HipBoneIndices[BoneIdx]],
							InOutPoseGlobalBoneData.BoneLocations[FrameIdx][KneeBoneIndices[BoneIdx]],
							InOutPoseGlobalBoneData.BoneLocations[FrameIdx][AnkleBoneIndices[BoneIdx]],
							AnklePosition,
							InOutPoseGlobalBoneData.BoneRotations[FrameIdx][KneeBoneIndices[BoneIdx]].RotateVector(KneeSideVectors[BoneIdx]),
							InOutPoseGlobalBoneData.BoneRotations[FrameIdx][HipBoneIndices[BoneIdx]],
							InOutPoseGlobalBoneData.BoneRotations[FrameIdx][KneeBoneIndices[BoneIdx]],
							InOutPoseGlobalBoneData.BoneRotations[FrameIdx][PelvisBoneIndices[BoneIdx]],
							Settings.HyperExtensionLimit);
					}
				}

				PoseData::ForwardKinematicsPartial(
					InOutPoseGlobalBoneData,
					InOutPoseLocalBoneData.ConstView(),
					InPoseRootData,
					InBoneParents,
					UE::Learning::FIndexSet(0, InBoneParents.Num()),
					RequiredBones);

				for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
				{
					for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
					{
						FVector ToePosition = InOutPoseGlobalBoneData.BoneLocations[FrameIdx][ToeBoneIndices[BoneIdx]];
						ToePosition.Z = FMath::Max(ToePosition.Z, Settings.MinToeHeight);

						ContactLookAtIK(
							InOutPoseLocalBoneData.BoneRotations[FrameIdx][AnkleBoneIndices[BoneIdx]],
							InOutPoseGlobalBoneData.BoneRotations[FrameIdx][KneeBoneIndices[BoneIdx]],
							InOutPoseGlobalBoneData.BoneRotations[FrameIdx][AnkleBoneIndices[BoneIdx]],
							InOutPoseGlobalBoneData.BoneLocations[FrameIdx][AnkleBoneIndices[BoneIdx]],
							InOutPoseGlobalBoneData.BoneLocations[FrameIdx][ToeBoneIndices[BoneIdx]],
							ToePosition);
					}
				}

				PoseData::ForwardKinematicsPartial(
					InOutPoseGlobalBoneData,
					InOutPoseLocalBoneData.ConstView(),
					InPoseRootData,
					InBoneParents,
					UE::Learning::FIndexSet(0, InBoneParents.Num()),
					RequiredBones);

				for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
				{
					for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
					{
						const FVector ToeEndCurr =
							(FVector)InOutPoseGlobalBoneData.BoneRotations[FrameIdx][ToeBoneIndices[BoneIdx]].RotateVector(Settings.ToeLength * ToeForwardVectors[BoneIdx]) +
							InOutPoseGlobalBoneData.BoneLocations[FrameIdx][ToeBoneIndices[BoneIdx]];

						FVector ToeEndTarg = ToeEndCurr;
						ToeEndTarg.Z = FMath::Max(ToeEndTarg.Z, Settings.MinToeHeight);

						ContactLookAtIK(
							InOutPoseLocalBoneData.BoneRotations[FrameIdx][ToeBoneIndices[BoneIdx]],
							InOutPoseGlobalBoneData.BoneRotations[FrameIdx][AnkleBoneIndices[BoneIdx]],
							InOutPoseGlobalBoneData.BoneRotations[FrameIdx][ToeBoneIndices[BoneIdx]],
							InOutPoseGlobalBoneData.BoneLocations[FrameIdx][ToeBoneIndices[BoneIdx]],
							ToeEndCurr,
							ToeEndTarg);
					}
				}
			}

		}
	}
}


bool FAnimDatabasePoseState::IsValid() const
{
	return CurrPoseData && PoseGlobalBoneData;
}

void FAnimDatabasePoseState::Init(const UE::AnimDatabase::FPoseDataConstView& PoseData, const TLearningArrayView<1, const int32> InBoneParents)
{
	CurrPoseData = MakeShared<UE::AnimDatabase::FPoseData>();
	PoseGlobalBoneData = MakeShared<UE::AnimDatabase::FPoseGlobalBoneData>();
	BoneParents.SetNumUninitialized({ InBoneParents.Num() });
	UE::Learning::Array::Copy(BoneParents, InBoneParents);

	CurrPoseData->Resize(PoseData.GetFrameNum(), PoseData.GetBoneNum(), PoseData.GetAttributeTypes(), PoseData.GetAttributeNames());
	PoseGlobalBoneData->Resize(PoseData.GetFrameNum(), PoseData.GetBoneNum());
}

FAnimDatabasePoseState UAnimDatabasePoseStateLibrary::MakePoseState()
{
	FAnimDatabasePoseState PoseState;
	PoseState.CurrPoseData = MakeShared<UE::AnimDatabase::FPoseData>();
	PoseState.PoseGlobalBoneData = MakeShared<UE::AnimDatabase::FPoseGlobalBoneData>();
	return PoseState;
}

void UAnimDatabasePoseStateLibrary::PoseStateCopy(FAnimDatabasePoseState& OutPoseState, const FAnimDatabasePoseState& InPoseState)
{
	if (!OutPoseState.IsValid())
	{
		OutPoseState = MakePoseState();
	}

	*OutPoseState.CurrPoseData = *InPoseState.CurrPoseData;
	*OutPoseState.PoseGlobalBoneData = *InPoseState.PoseGlobalBoneData;
}

FTransform UAnimDatabasePoseStateLibrary::PoseStateRootTransform(const FAnimDatabasePoseState& PoseState)
{
	return !PoseState.CurrPoseData || PoseState.CurrPoseData->IsEmpty() ?
		FTransform::Identity :
		FTransform(
			(FQuat)PoseState.CurrPoseData->RootData.RootRotations[PoseState.PoseIdx],
			(FVector)PoseState.CurrPoseData->RootData.RootLocations[PoseState.PoseIdx],
			(FVector)PoseState.CurrPoseData->RootData.RootScales[PoseState.PoseIdx]);
}

FVector UAnimDatabasePoseStateLibrary::PoseStateRootLocation(const FAnimDatabasePoseState& PoseState)
{
	if (!PoseState.CurrPoseData) { return FVector::ZeroVector; }
	if (PoseState.CurrPoseData->IsEmpty()) { return FVector::ZeroVector; }

	return (FVector)PoseState.CurrPoseData->RootData.RootLocations[PoseState.PoseIdx];
}

FRotator UAnimDatabasePoseStateLibrary::PoseStateRootRotation(const FAnimDatabasePoseState& PoseState)
{
	if (!PoseState.CurrPoseData) { return FRotator::ZeroRotator; }
	if (PoseState.CurrPoseData->IsEmpty()) { return FRotator::ZeroRotator; }

	return (FRotator)PoseState.CurrPoseData->RootData.RootRotations[PoseState.PoseIdx].Rotator();
}

void UAnimDatabasePoseStateLibrary::PoseStateRootLocationAndRotation(FVector& OutRootLocation, FRotator& OutRootRotation, const FAnimDatabasePoseState& PoseState)
{
	OutRootLocation = PoseStateRootLocation(PoseState);
	OutRootRotation = PoseStateRootRotation(PoseState);
}

FVector UAnimDatabasePoseStateLibrary::PoseStateRootScale3D(const FAnimDatabasePoseState& PoseState)
{
	if (!PoseState.CurrPoseData) { return FVector::OneVector; }
	if (PoseState.CurrPoseData->IsEmpty()) { return FVector::OneVector; }

	return (FVector)PoseState.CurrPoseData->RootData.RootScales[PoseState.PoseIdx];
}

FVector UAnimDatabasePoseStateLibrary::PoseStateRootDirection(const FAnimDatabasePoseState& PoseState, const FVector ForwardVector)
{
	if (!PoseState.CurrPoseData) { return FVector::ForwardVector; }
	if (PoseState.CurrPoseData->IsEmpty()) { return FVector::ForwardVector; }

	return ((FVector)PoseState.CurrPoseData->RootData.RootRotations[PoseState.PoseIdx].RotateVector((FVector3f)ForwardVector)).GetSafeNormal();
}

FVector UAnimDatabasePoseStateLibrary::PoseStateRootLinearVelocity(const FAnimDatabasePoseState& PoseState)
{
	if (!PoseState.CurrPoseData) { return FVector::ZeroVector; }
	if (PoseState.CurrPoseData->IsEmpty()) { return FVector::ZeroVector; }

	return (FVector)PoseState.CurrPoseData->RootData.RootLinearVelocities[PoseState.PoseIdx];
}

FVector UAnimDatabasePoseStateLibrary::PoseStateRootAngularVelocity(const FAnimDatabasePoseState& PoseState)
{
	if (!PoseState.CurrPoseData) { return FVector::ZeroVector; }
	if (PoseState.CurrPoseData->IsEmpty()) { return FVector::ZeroVector; }

	return (FVector)PoseState.CurrPoseData->RootData.RootAngularVelocities[PoseState.PoseIdx];
}

FVector UAnimDatabasePoseStateLibrary::PoseStateRootScalarVelocity(const FAnimDatabasePoseState& PoseState)
{
	if (!PoseState.CurrPoseData) { return FVector::ZeroVector; }
	if (PoseState.CurrPoseData->IsEmpty()) { return FVector::ZeroVector; }

	return (FVector)PoseState.CurrPoseData->RootData.RootScalarVelocities[PoseState.PoseIdx] ;
}

FVector UAnimDatabasePoseStateLibrary::PoseStateBoneWorldLocation(const FAnimDatabasePoseState& PoseState, const int32 BoneIndex)
{
	if (!PoseState.PoseGlobalBoneData) { return FVector::ZeroVector; }
	if (PoseState.PoseGlobalBoneData->IsEmpty()) { return FVector::ZeroVector; }
	if (!(BoneIndex >= 0 && BoneIndex < PoseState.PoseGlobalBoneData->GetBoneNum())) { return FVector::ZeroVector; }

	return (FVector)PoseState.PoseGlobalBoneData->BoneLocations[PoseState.PoseIdx][BoneIndex];
}

FRotator UAnimDatabasePoseStateLibrary::PoseStateBoneWorldRotation(const FAnimDatabasePoseState& PoseState, const int32 BoneIndex)
{
	if (!PoseState.PoseGlobalBoneData) { return FRotator::ZeroRotator; }
	if (PoseState.PoseGlobalBoneData->IsEmpty()) { return FRotator::ZeroRotator; }
	if (!(BoneIndex >= 0 && BoneIndex < PoseState.PoseGlobalBoneData->GetBoneNum())) { return FRotator::ZeroRotator; }

	return (FRotator)PoseState.PoseGlobalBoneData->BoneRotations[PoseState.PoseIdx][BoneIndex].Rotator();
}

FTransform UAnimDatabasePoseStateLibrary::PoseStateBoneWorldTransform(const FAnimDatabasePoseState& PoseState, const int32 BoneIndex)
{
	if (!PoseState.PoseGlobalBoneData) { return FTransform::Identity; }
	if (PoseState.PoseGlobalBoneData->IsEmpty()) { return FTransform::Identity; }
	if (!(BoneIndex >= 0 && BoneIndex < PoseState.PoseGlobalBoneData->GetBoneNum())) { return FTransform::Identity; }

	return FTransform(
		((FQuat)PoseState.PoseGlobalBoneData->BoneRotations[PoseState.PoseIdx][BoneIndex]).GetNormalized(),
		(FVector)PoseState.PoseGlobalBoneData->BoneLocations[PoseState.PoseIdx][BoneIndex],
		(FVector)PoseState.PoseGlobalBoneData->BoneScales[PoseState.PoseIdx][BoneIndex]);
}

FVector UAnimDatabasePoseStateLibrary::PoseStateBoneWorldLinearVelocity(const FAnimDatabasePoseState& PoseState, const int32 BoneIndex)
{
	if (!PoseState.PoseGlobalBoneData) { return FVector::ZeroVector; }
	if (PoseState.PoseGlobalBoneData->IsEmpty()) { return FVector::ZeroVector; }
	if (!(BoneIndex >= 0 && BoneIndex < PoseState.PoseGlobalBoneData->GetBoneNum())) { return FVector::ZeroVector; }

	return (FVector)PoseState.PoseGlobalBoneData->BoneLinearVelocities[PoseState.PoseIdx][BoneIndex];
}

int32 UAnimDatabasePoseStateLibrary::PoseStateAttributeNum(const FAnimDatabasePoseState& PoseState)
{
	if (!PoseState.CurrPoseData) { return 0; }
	if (PoseState.CurrPoseData->IsEmpty()) { return 0; }
	return PoseState.CurrPoseData->GetAttributeNum();
}

bool UAnimDatabasePoseStateLibrary::PoseStateIsAttributeActive(const FAnimDatabasePoseState& PoseState, const int32 AttributeIndex)
{
	if (!PoseState.CurrPoseData) { return false; }
	if (PoseState.CurrPoseData->IsEmpty()) { return false; }
	if (!(AttributeIndex >= 0 && AttributeIndex < PoseState.CurrPoseData->GetAttributeNum())) { return false; }

	return PoseState.CurrPoseData->AttributeData.GetAttributeActive(PoseState.PoseIdx, AttributeIndex);
}

EAnimDatabaseAttributeType UAnimDatabasePoseStateLibrary::PoseStateAttributeType(const FAnimDatabasePoseState& PoseState, const int32 AttributeIndex)
{
	if (!PoseState.CurrPoseData) { return EAnimDatabaseAttributeType::Null; }
	if (PoseState.CurrPoseData->IsEmpty()) { return EAnimDatabaseAttributeType::Null; }
	if (!(AttributeIndex >= 0 && AttributeIndex < PoseState.CurrPoseData->GetAttributeNum())) { return EAnimDatabaseAttributeType::Null; }

	return PoseState.CurrPoseData->AttributeData.GetAttributeType(AttributeIndex);
}

FName UAnimDatabasePoseStateLibrary::PoseStateAttributeName(const FAnimDatabasePoseState& PoseState, const int32 AttributeIndex)
{
	if (!PoseState.CurrPoseData) { return NAME_None; }
	if (PoseState.CurrPoseData->IsEmpty()) { return NAME_None; }
	if (!(AttributeIndex >= 0 && AttributeIndex < PoseState.CurrPoseData->GetAttributeNum())) { return NAME_None; }

	return PoseState.CurrPoseData->AttributeData.GetAttributeName(AttributeIndex);
}

FString UAnimDatabasePoseStateLibrary::PoseStateAttributeString(const FAnimDatabasePoseState& PoseState, const int32 AttributeIndex)
{
	if (!PoseState.CurrPoseData) { return FString(); }
	if (PoseState.CurrPoseData->IsEmpty()) { return FString(); }
	if (!(AttributeIndex >= 0 && AttributeIndex < PoseState.CurrPoseData->GetAttributeNum())) { return FString(); }

	bool bEventTimeKnown = false;
	float EventTimeUntilEvent = 0.0f;

	switch (PoseState.CurrPoseData->AttributeData.GetAttributeType(AttributeIndex))
	{
	case EAnimDatabaseAttributeType::Null: return TEXT("Null");
	case EAnimDatabaseAttributeType::Bool: return PoseState.CurrPoseData->AttributeData.GetBool(PoseState.PoseIdx, AttributeIndex) ? TEXT("true") : TEXT("false");
	case EAnimDatabaseAttributeType::Float: return FString::Printf(TEXT("% 0.2f"), PoseState.CurrPoseData->AttributeData.GetFloat(PoseState.PoseIdx, AttributeIndex));
	case EAnimDatabaseAttributeType::Location: return FString::Printf(TEXT("X=% 0.2f Y=% 0.2f Z=% 0.2f"), 
		PoseState.CurrPoseData->AttributeData.GetLocation(PoseState.PoseIdx, AttributeIndex).X,
		PoseState.CurrPoseData->AttributeData.GetLocation(PoseState.PoseIdx, AttributeIndex).Y,
		PoseState.CurrPoseData->AttributeData.GetLocation(PoseState.PoseIdx, AttributeIndex).Z);

	case EAnimDatabaseAttributeType::Rotation: return FString::Printf(TEXT("Yaw=% 0.2f Pitch=% 0.2f Roll=% 0.2f"),
		PoseState.CurrPoseData->AttributeData.GetRotation(PoseState.PoseIdx, AttributeIndex).Rotator().Yaw,
		PoseState.CurrPoseData->AttributeData.GetRotation(PoseState.PoseIdx, AttributeIndex).Rotator().Pitch,
		PoseState.CurrPoseData->AttributeData.GetRotation(PoseState.PoseIdx, AttributeIndex).Rotator().Roll);

	case EAnimDatabaseAttributeType::Scale: return FString::Printf(TEXT("X=% 0.2f Y=% 0.2f Z=% 0.2f"),
		PoseState.CurrPoseData->AttributeData.GetScale(PoseState.PoseIdx, AttributeIndex).X,
		PoseState.CurrPoseData->AttributeData.GetScale(PoseState.PoseIdx, AttributeIndex).Y,
		PoseState.CurrPoseData->AttributeData.GetScale(PoseState.PoseIdx, AttributeIndex).Z);

	case EAnimDatabaseAttributeType::LinearVelocity: return FString::Printf(TEXT("X=% 0.2f Y=% 0.2f Z=% 0.2f"),
		PoseState.CurrPoseData->AttributeData.GetLinearVelocity(PoseState.PoseIdx, AttributeIndex).X,
		PoseState.CurrPoseData->AttributeData.GetLinearVelocity(PoseState.PoseIdx, AttributeIndex).Y,
		PoseState.CurrPoseData->AttributeData.GetLinearVelocity(PoseState.PoseIdx, AttributeIndex).Z);

	case EAnimDatabaseAttributeType::AngularVelocity: return FString::Printf(TEXT("X=% 0.2f Y=% 0.2f Z=% 0.2f"),
		FMath::RadiansToDegrees(PoseState.CurrPoseData->AttributeData.GetAngularVelocity(PoseState.PoseIdx, AttributeIndex).X),
		FMath::RadiansToDegrees(PoseState.CurrPoseData->AttributeData.GetAngularVelocity(PoseState.PoseIdx, AttributeIndex).Y),
		FMath::RadiansToDegrees(PoseState.CurrPoseData->AttributeData.GetAngularVelocity(PoseState.PoseIdx, AttributeIndex).Z));

	case EAnimDatabaseAttributeType::ScalarVelocity: return FString::Printf(TEXT("X=% 0.2f Y=% 0.2f Z=% 0.2f"),
		PoseState.CurrPoseData->AttributeData.GetScalarVelocity(PoseState.PoseIdx, AttributeIndex).X,
		PoseState.CurrPoseData->AttributeData.GetScalarVelocity(PoseState.PoseIdx, AttributeIndex).Y,
		PoseState.CurrPoseData->AttributeData.GetScalarVelocity(PoseState.PoseIdx, AttributeIndex).Z);

	case EAnimDatabaseAttributeType::Direction: return FString::Printf(TEXT("X=% 0.2f Y=% 0.2f Z=% 0.2f"),
		PoseState.CurrPoseData->AttributeData.GetDirection(PoseState.PoseIdx, AttributeIndex).X,
		PoseState.CurrPoseData->AttributeData.GetDirection(PoseState.PoseIdx, AttributeIndex).Y,
		PoseState.CurrPoseData->AttributeData.GetDirection(PoseState.PoseIdx, AttributeIndex).Z);

	case EAnimDatabaseAttributeType::Transform: return FString::Printf(TEXT("LocX=% 0.2f LocY=% 0.2f LocZ=% 0.2f  Yaw=% 0.2f Pitch=% 0.2f Roll=% 0.2f  ScaleX=% 0.2f ScaleY=% 0.2f ScaleZ=% 0.2f"),
		PoseState.CurrPoseData->AttributeData.GetTransform(PoseState.PoseIdx, AttributeIndex).GetLocation().X,
		PoseState.CurrPoseData->AttributeData.GetTransform(PoseState.PoseIdx, AttributeIndex).GetLocation().Y,
		PoseState.CurrPoseData->AttributeData.GetTransform(PoseState.PoseIdx, AttributeIndex).GetLocation().Z,
		PoseState.CurrPoseData->AttributeData.GetTransform(PoseState.PoseIdx, AttributeIndex).GetRotation().Rotator().Yaw,
		PoseState.CurrPoseData->AttributeData.GetTransform(PoseState.PoseIdx, AttributeIndex).GetRotation().Rotator().Pitch,
		PoseState.CurrPoseData->AttributeData.GetTransform(PoseState.PoseIdx, AttributeIndex).GetRotation().Rotator().Roll,
		PoseState.CurrPoseData->AttributeData.GetTransform(PoseState.PoseIdx, AttributeIndex).GetScale3D().X,
		PoseState.CurrPoseData->AttributeData.GetTransform(PoseState.PoseIdx, AttributeIndex).GetScale3D().Y,
		PoseState.CurrPoseData->AttributeData.GetTransform(PoseState.PoseIdx, AttributeIndex).GetScale3D().Z);

	case EAnimDatabaseAttributeType::Event:
	{
		PoseState.CurrPoseData->AttributeData.GetEvent(bEventTimeKnown, EventTimeUntilEvent, PoseState.PoseIdx, AttributeIndex);
		return bEventTimeKnown ? FString::Printf(TEXT("TimeUntilEvent=% 0.2f"), EventTimeUntilEvent) : TEXT("TimeUntilEventUnknown");
	}

	case EAnimDatabaseAttributeType::Angle: return FString::Printf(TEXT("% 0.2f"), FMath::RadiansToDegrees(PoseState.CurrPoseData->AttributeData.GetAngle(PoseState.PoseIdx, AttributeIndex)));
	default: checkNoEntry(); return FString();
	};
}

bool UAnimDatabasePoseStateLibrary::PoseStateBoolAttribute(const FAnimDatabasePoseState& PoseState, const int32 AttributeIndex)
{
	if (!PoseState.CurrPoseData) { return false; }
	if (PoseState.CurrPoseData->IsEmpty()) { return false; }
	if (!(AttributeIndex >= 0 && AttributeIndex < PoseState.CurrPoseData->GetAttributeNum())) { return false; }
	if (PoseState.CurrPoseData->AttributeData.GetAttributeType(AttributeIndex) != EAnimDatabaseAttributeType::Bool) { return false; }
	if (!PoseState.CurrPoseData->AttributeData.GetAttributeActive(PoseState.PoseIdx, AttributeIndex)) { return false; }

	return PoseState.CurrPoseData->AttributeData.GetBool(PoseState.PoseIdx, AttributeIndex);
}

float UAnimDatabasePoseStateLibrary::PoseStateFloatAttribute(const FAnimDatabasePoseState& PoseState, const int32 AttributeIndex)
{
	if (!PoseState.CurrPoseData) { return 0.0f; }
	if (PoseState.CurrPoseData->IsEmpty()) { return 0.0f; }
	if (!(AttributeIndex >= 0 && AttributeIndex < PoseState.CurrPoseData->GetAttributeNum())) { return 0.0f; }
	if (PoseState.CurrPoseData->AttributeData.GetAttributeType(AttributeIndex) != EAnimDatabaseAttributeType::Float) { return 0.0f; }
	if (!PoseState.CurrPoseData->AttributeData.GetAttributeActive(PoseState.PoseIdx, AttributeIndex)) { return 0.0f; }

	return PoseState.CurrPoseData->AttributeData.GetFloat(PoseState.PoseIdx, AttributeIndex);
}

float UAnimDatabasePoseStateLibrary::PoseStateAngleAttributeRadians(const FAnimDatabasePoseState& PoseState, const int32 AttributeIndex)
{
	if (!PoseState.CurrPoseData) { return 0.0f; }
	if (PoseState.CurrPoseData->IsEmpty()) { return 0.0f; }
	if (!(AttributeIndex >= 0 && AttributeIndex < PoseState.CurrPoseData->GetAttributeNum())) { return 0.0f; }
	if (PoseState.CurrPoseData->AttributeData.GetAttributeType(AttributeIndex) != EAnimDatabaseAttributeType::Angle) { return 0.0f; }
	if (!PoseState.CurrPoseData->AttributeData.GetAttributeActive(PoseState.PoseIdx, AttributeIndex)) { return 0.0f; }

	return PoseState.CurrPoseData->AttributeData.GetAngle(PoseState.PoseIdx, AttributeIndex);
}

float UAnimDatabasePoseStateLibrary::PoseStateAngleAttributeDegrees(const FAnimDatabasePoseState& PoseState, const int32 AttributeIndex)
{
	if (!PoseState.CurrPoseData) { return 0.0f; }
	if (PoseState.CurrPoseData->IsEmpty()) { return 0.0f; }
	if (!(AttributeIndex >= 0 && AttributeIndex < PoseState.CurrPoseData->GetAttributeNum())) { return 0.0f; }
	if (PoseState.CurrPoseData->AttributeData.GetAttributeType(AttributeIndex) != EAnimDatabaseAttributeType::Angle) { return 0.0f; }
	if (!PoseState.CurrPoseData->AttributeData.GetAttributeActive(PoseState.PoseIdx, AttributeIndex)) { return 0.0f; }

	return FMath::RadiansToDegrees(PoseState.CurrPoseData->AttributeData.GetAngle(PoseState.PoseIdx, AttributeIndex));
}

FVector UAnimDatabasePoseStateLibrary::PoseStateLocationAttribute(const FAnimDatabasePoseState& PoseState, const int32 AttributeIndex)
{
	if (!PoseState.CurrPoseData) { return FVector::ZeroVector; }
	if (PoseState.CurrPoseData->IsEmpty()) { return FVector::ZeroVector; }
	if (!(AttributeIndex >= 0 && AttributeIndex < PoseState.CurrPoseData->GetAttributeNum())) { return FVector::ZeroVector; }
	if (PoseState.CurrPoseData->AttributeData.GetAttributeType(AttributeIndex) != EAnimDatabaseAttributeType::Location) { return FVector::ZeroVector; }
	if (!PoseState.CurrPoseData->AttributeData.GetAttributeActive(PoseState.PoseIdx, AttributeIndex)) { return FVector::ZeroVector; }

	return (FVector)PoseState.CurrPoseData->AttributeData.GetLocation(PoseState.PoseIdx, AttributeIndex);
}

FRotator UAnimDatabasePoseStateLibrary::PoseStateRotationAttribute(const FAnimDatabasePoseState& PoseState, const int32 AttributeIndex)
{
	return PoseStateRotationAttributeAsQuat(PoseState, AttributeIndex).Rotator();
}

FQuat UAnimDatabasePoseStateLibrary::PoseStateRotationAttributeAsQuat(const FAnimDatabasePoseState& PoseState, const int32 AttributeIndex)
{
	if (!PoseState.CurrPoseData) { return FQuat::Identity; }
	if (PoseState.CurrPoseData->IsEmpty()) { return FQuat::Identity; }
	if (!(AttributeIndex >= 0 && AttributeIndex < PoseState.CurrPoseData->GetAttributeNum())) { return FQuat::Identity; }
	if (PoseState.CurrPoseData->AttributeData.GetAttributeType(AttributeIndex) != EAnimDatabaseAttributeType::Rotation) { return FQuat::Identity; }
	if (!PoseState.CurrPoseData->AttributeData.GetAttributeActive(PoseState.PoseIdx, AttributeIndex)) { return FQuat::Identity; }

	return ((FQuat)PoseState.CurrPoseData->AttributeData.GetRotation(PoseState.PoseIdx, AttributeIndex)).GetNormalized();
}

FVector UAnimDatabasePoseStateLibrary::PoseStateScaleAttribute(const FAnimDatabasePoseState& PoseState, const int32 AttributeIndex)
{
	if (!PoseState.CurrPoseData) { return FVector::OneVector; }
	if (PoseState.CurrPoseData->IsEmpty()) { return FVector::OneVector; }
	if (!(AttributeIndex >= 0 && AttributeIndex < PoseState.CurrPoseData->GetAttributeNum())) { return FVector::OneVector; }
	if (PoseState.CurrPoseData->AttributeData.GetAttributeType(AttributeIndex) != EAnimDatabaseAttributeType::Scale) { return FVector::OneVector; }
	if (!PoseState.CurrPoseData->AttributeData.GetAttributeActive(PoseState.PoseIdx, AttributeIndex)) { return FVector::OneVector; }

	return (FVector)PoseState.CurrPoseData->AttributeData.GetScale(PoseState.PoseIdx, AttributeIndex);
}

FVector UAnimDatabasePoseStateLibrary::PoseStateLinearVelocityAttribute(const FAnimDatabasePoseState& PoseState, const int32 AttributeIndex)
{
	if (!PoseState.CurrPoseData) { return FVector::ZeroVector; }
	if (PoseState.CurrPoseData->IsEmpty()) { return FVector::ZeroVector; }
	if (!(AttributeIndex >= 0 && AttributeIndex < PoseState.CurrPoseData->GetAttributeNum())) { return FVector::ZeroVector; }
	if (PoseState.CurrPoseData->AttributeData.GetAttributeType(AttributeIndex) != EAnimDatabaseAttributeType::LinearVelocity) { return FVector::ZeroVector; }
	if (!PoseState.CurrPoseData->AttributeData.GetAttributeActive(PoseState.PoseIdx, AttributeIndex)) { return FVector::ZeroVector; }

	return (FVector)PoseState.CurrPoseData->AttributeData.GetLinearVelocity(PoseState.PoseIdx, AttributeIndex);
}

FVector UAnimDatabasePoseStateLibrary::PoseStateAngularVelocityAttribute(const FAnimDatabasePoseState& PoseState, const int32 AttributeIndex)
{
	if (!PoseState.CurrPoseData) { return FVector::ZeroVector; }
	if (PoseState.CurrPoseData->IsEmpty()) { return FVector::ZeroVector; }
	if (!(AttributeIndex >= 0 && AttributeIndex < PoseState.CurrPoseData->GetAttributeNum())) { return FVector::ZeroVector; }
	if (PoseState.CurrPoseData->AttributeData.GetAttributeType(AttributeIndex) != EAnimDatabaseAttributeType::AngularVelocity) { return FVector::ZeroVector; }
	if (!PoseState.CurrPoseData->AttributeData.GetAttributeActive(PoseState.PoseIdx, AttributeIndex)) { return FVector::ZeroVector; }

	return (FVector)PoseState.CurrPoseData->AttributeData.GetAngularVelocity(PoseState.PoseIdx, AttributeIndex);
}

FVector UAnimDatabasePoseStateLibrary::PoseStateScalarVelocityAttribute(const FAnimDatabasePoseState& PoseState, const int32 AttributeIndex)
{
	if (!PoseState.CurrPoseData) { return FVector::ZeroVector; }
	if (PoseState.CurrPoseData->IsEmpty()) { return FVector::ZeroVector; }
	if (!(AttributeIndex >= 0 && AttributeIndex < PoseState.CurrPoseData->GetAttributeNum())) { return FVector::ZeroVector; }
	if (PoseState.CurrPoseData->AttributeData.GetAttributeType(AttributeIndex) != EAnimDatabaseAttributeType::ScalarVelocity) { return FVector::ZeroVector; }
	if (!PoseState.CurrPoseData->AttributeData.GetAttributeActive(PoseState.PoseIdx, AttributeIndex)) { return FVector::ZeroVector; }

	return (FVector)PoseState.CurrPoseData->AttributeData.GetScalarVelocity(PoseState.PoseIdx, AttributeIndex);
}

FVector UAnimDatabasePoseStateLibrary::PoseStateDirectionAttribute(const FAnimDatabasePoseState& PoseState, const int32 AttributeIndex)
{
	if (!PoseState.CurrPoseData) { return FVector::ForwardVector; }
	if (PoseState.CurrPoseData->IsEmpty()) { return FVector::ForwardVector; }
	if (!(AttributeIndex >= 0 && AttributeIndex < PoseState.CurrPoseData->GetAttributeNum())) { return FVector::ForwardVector; }
	if (PoseState.CurrPoseData->AttributeData.GetAttributeType(AttributeIndex) != EAnimDatabaseAttributeType::Direction) { return FVector::ForwardVector; }
	if (!PoseState.CurrPoseData->AttributeData.GetAttributeActive(PoseState.PoseIdx, AttributeIndex)) { return FVector::ForwardVector; }

	return ((FVector)PoseState.CurrPoseData->AttributeData.GetDirection(PoseState.PoseIdx, AttributeIndex)).GetSafeNormal();
}

FTransform UAnimDatabasePoseStateLibrary::PoseStateTransformAttribute(const FAnimDatabasePoseState& PoseState, const int32 AttributeIndex)
{
	if (!PoseState.CurrPoseData) { return FTransform::Identity; }
	if (PoseState.CurrPoseData->IsEmpty()) { return FTransform::Identity; }
	if (!(AttributeIndex >= 0 && AttributeIndex < PoseState.CurrPoseData->GetAttributeNum())) { return FTransform::Identity; }
	if (PoseState.CurrPoseData->AttributeData.GetAttributeType(AttributeIndex) != EAnimDatabaseAttributeType::Transform) { return FTransform::Identity; }
	if (!PoseState.CurrPoseData->AttributeData.GetAttributeActive(PoseState.PoseIdx, AttributeIndex)) { return FTransform::Identity; }

	const FTransform3f Transform = PoseState.CurrPoseData->AttributeData.GetTransform(PoseState.PoseIdx, AttributeIndex);

	return FTransform(
		((FQuat)Transform.GetRotation()).GetNormalized(),
		(FVector)Transform.GetLocation(),
		(FVector)Transform.GetScale3D());
}

void UAnimDatabasePoseStateLibrary::PoseStateEventAttribute(bool& bOutTimeUntilEventKnown, float& OutTimeUntilEvent, const FAnimDatabasePoseState& PoseState, const int32 AttributeIndex)
{
	if (!PoseState.CurrPoseData) { bOutTimeUntilEventKnown = false; OutTimeUntilEvent = UE_MAX_FLT; return; }
	if (PoseState.CurrPoseData->IsEmpty()) { bOutTimeUntilEventKnown = false; OutTimeUntilEvent = UE_MAX_FLT; return; }
	if (!(AttributeIndex >= 0 && AttributeIndex < PoseState.CurrPoseData->GetAttributeNum())) { bOutTimeUntilEventKnown = false; OutTimeUntilEvent = UE_MAX_FLT; return; }
	if (PoseState.CurrPoseData->AttributeData.GetAttributeType(AttributeIndex) != EAnimDatabaseAttributeType::Event) { bOutTimeUntilEventKnown = false; OutTimeUntilEvent = UE_MAX_FLT; return; }
	if (!PoseState.CurrPoseData->AttributeData.GetAttributeActive(PoseState.PoseIdx, AttributeIndex)) { bOutTimeUntilEventKnown = false; OutTimeUntilEvent = UE_MAX_FLT; return; }

	PoseState.CurrPoseData->AttributeData.GetEvent(bOutTimeUntilEventKnown, OutTimeUntilEvent, PoseState.PoseIdx, AttributeIndex);
}

#undef UE_ANIMDATABASE_ISPC