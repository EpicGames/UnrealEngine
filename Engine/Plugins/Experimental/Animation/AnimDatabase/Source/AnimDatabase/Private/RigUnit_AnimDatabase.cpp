// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigUnit_AnimDatabase.h"

#include "Animation/AnimSequence.h"
#include "AnimDatabase.h"
#include "AnimDatabaseIndex.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_AnimDatabase)

FRigVMFunction_FrameAttributeIntersection_Execute()
{
	Result = UAnimDatabaseFrameAttributeLibrary::FrameAttributeIntersection(A, B);
}

FRigVMFunction_FrameAttributeAdd_Execute()
{
	Result = UAnimDatabaseFrameAttributeLibrary::FrameAttributeAdd(A, B);
}

FRigVMFunction_FrameAttributeInertializationMatchingDistance_Execute()
{
	Result = UAnimDatabaseFrameAttributeLibrary::FrameAttributeInertializationMatchingDistance(
		LocationAttribute,
		VelocityAttribute,
		Location,
		Velocity,
		BlendTime,
		LocationScale,
		VelocityScale,
		LocationWeight,
		VelocityWeight);
}

FRigUnit_FindFrameAttributeMinimumSequence_Execute()
{
	UAnimSequence* OutAnimSequence = nullptr;

	UAnimDatabaseFrameAttributeLibrary::FindFrameAttributeMinimumSequence(
		OutAnimSequence,
		SequenceTime,
		bIsMirrored,
		MinimumValue,
		Attribute,
		Database);

	AnimSequence = OutAnimSequence;
}

FRigVMFunction_MakeFloatFrameAttributeFromUniformRandom_Execute()
{
	Result = UAnimDatabaseFrameAttributeLibrary::MakeFloatFrameAttributeFromUniformRandom(FrameRanges, Seed, Min, Max);
}

FRigVMFunction_DatabaseIndexGetDatabase_Execute()
{
	Database = Index ? Index->IndexDatabase : nullptr;
}

FRigVMFunction_FindDatabaseIndexFrames_Execute()
{
	if (Index)
	{
		if (FAnimDatabaseFrames* FoundFrames = Index->IndexFrames.Find(Name))
		{
			bFound = true;
			Frames = *FoundFrames;
			return;
		}
	}

	bFound = false;
	Frames = FAnimDatabaseFrames();
}

FRigVMFunction_FindDatabaseIndexFrameRanges_Execute()
{
	if (Index)
	{
		if (FAnimDatabaseFrameRanges* FoundFrameRanges = Index->IndexFrameRanges.Find(Name))
		{
			bFound = true;
			FrameRanges = *FoundFrameRanges;
			return;
		}
	}

	bFound = false;
	FrameRanges = FAnimDatabaseFrameRanges();
}

FRigVMFunction_FindDatabaseIndexFrameAttribute_Execute()
{
	if (Index)
	{
		if (FAnimDatabaseFrameAttribute* FoundFrameAttribute = Index->IndexFrameAttributes.Find(Name))
		{
			bFound = true;
			FrameAttribute = *FoundFrameAttribute;
			return;
		}
	}

	bFound = false;
	FrameAttribute = FAnimDatabaseFrameAttribute();
}