// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMCore/RigVMStruct.h"

#include "AnimDatabaseFrameRanges.h"
#include "AnimDatabaseFrameAttribute.h"

#include "RigUnit_AnimDatabase.generated.h"

#define UE_API ANIMDATABASE_API

class UAnimSequence;
class UAnimDatabaseIndex;

/*
 * The base class for all Anim Database functions
 */
USTRUCT(meta = (Abstract, Category = "AnimDatabase", NodeColor = "0.83077 0.846873 0.049707", DocumentationPolicy = "Strict"))
struct FRigVMFunction_AnimDatabaseBase : public FRigVMStruct
{
	GENERATED_BODY()
};

/*
 * Base class for all Anim Database rig units
 */
USTRUCT(meta=(Abstract, ExecuteContext = "FRigVMExecuteContext", Category="AnimDatabase", NodeColor = "0.83077 0.846873 0.049707", DocumentationPolicy = "Strict"))
struct FRigUnit_AnimDatabaseBase : public FRigVMStruct
{
	GENERATED_BODY()

	// The execution result
	UPROPERTY(EditAnywhere, DisplayName = "Execute", Category = "BeginExecution", meta = (Input, Output))
	FRigVMExecuteContext ExecuteContext;
};

/** Frame Attribute Intersection */
USTRUCT(meta = (DisplayName = "Intersection"))
struct FRigVMFunction_FrameAttributeIntersection : public FRigVMFunction_AnimDatabaseBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Input Frame Attribute
	UPROPERTY(meta = (Input))
	FAnimDatabaseFrameAttribute A;

	// Input Frame Ranges
	UPROPERTY(meta = (Input))
	FAnimDatabaseFrameRanges B;

	// Output Frame Attribute
	UPROPERTY(meta = (Output))
	FAnimDatabaseFrameAttribute Result;
};

/** Frame Attribute Add */
USTRUCT(meta = (DisplayName = "Add (Frame Attribute)"))
struct FRigVMFunction_FrameAttributeAdd : public FRigVMFunction_AnimDatabaseBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Input Frame Attribute A
	UPROPERTY(meta = (Input))
	FAnimDatabaseFrameAttribute A;

	// Input Frame Attribute B
	UPROPERTY(meta = (Input))
	FAnimDatabaseFrameAttribute B;

	// Result Frame Attribute
	UPROPERTY(meta = (Output))
	FAnimDatabaseFrameAttribute Result;
};

/** Computes a Frame Attribute representing Inertialization Matching Distance */
USTRUCT(meta = (DisplayName = "Inertialization Matching Distance"))
struct FRigVMFunction_FrameAttributeInertializationMatchingDistance : public FRigVMFunction_AnimDatabaseBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Location Attribute
	UPROPERTY(meta = (Input))
	FAnimDatabaseFrameAttribute LocationAttribute;

	// Velocity Attribute
	UPROPERTY(meta = (Input))
	FAnimDatabaseFrameAttribute VelocityAttribute;

	// Input Location
	UPROPERTY(meta = (Input))
	FVector Location = FVector::ZeroVector;

	// Input Velocity
	UPROPERTY(meta = (Input))
	FVector Velocity = FVector::ZeroVector;

	// Inertialization Blend Time
	UPROPERTY(meta = (Input))
	float BlendTime = 0.2f;

	// Location scales
	UPROPERTY(meta = (Input))
	float LocationScale = 100.0f;

	// Velocity scales
	UPROPERTY(meta = (Input))
	float VelocityScale = 200.0f;

	// Location weight
	UPROPERTY(meta = (Input))
	float LocationWeight = 1.0f;

	// Velocity weight
	UPROPERTY(meta = (Input))
	float VelocityWeight = 1.0f;

	// Resulting Distance Attribute
	UPROPERTY(meta = (Output))
	FAnimDatabaseFrameAttribute Result;
};

/** Finds the AnimSequence corresponding to the minimum value in a frame attribute */
USTRUCT(meta = (DisplayName = "Find Frame Attribute Minimum Sequence"))
struct FRigUnit_FindFrameAttributeMinimumSequence : public FRigUnit_AnimDatabaseBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Input Attribute
	UPROPERTY(meta = (Input))
	FAnimDatabaseFrameAttribute Attribute;

	// Input Database
	UPROPERTY(meta = (Input))
	TObjectPtr<UAnimDatabase> Database;

	// Output Anim Sequence
	UPROPERTY(meta = (Output))
	TObjectPtr<UAnimSequence> AnimSequence;

	// Output Anim Sequence Time
	UPROPERTY(meta = (Output))
	float SequenceTime = 0.0f;

	// If the output Anim Sequence should be mirrored
	UPROPERTY(meta = (Output))
	bool bIsMirrored = false;

	// Output minimum value
	UPROPERTY(meta = (Output))
	float MinimumValue = 0.0f;
};

/** Makes a float frame attribute from a uniform distribution.  */
USTRUCT(meta = (DisplayName = "Make Float Frame Attribute from Uniform Random"))
struct FRigVMFunction_MakeFloatFrameAttributeFromUniformRandom : public FRigVMFunction_AnimDatabaseBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Input Frame Ranges
	UPROPERTY(meta = (Input))
	FAnimDatabaseFrameRanges FrameRanges;

	// Random State
	UPROPERTY(meta = (Input, Output))
	int32 Seed = 1234;

	// Minimum Value
	UPROPERTY(meta = (Input))
	float Min = 0.0f;

	// Maximum Value
	UPROPERTY(meta = (Input))
	float Max = 1.0f;

	// Result Float Frame Attribute
	UPROPERTY(meta = (Output))
	FAnimDatabaseFrameAttribute Result;
};

/** Gets the Database from a DatabaseIndex */
USTRUCT(meta = (DisplayName = "Get Database (Database Index)"))
struct FRigVMFunction_DatabaseIndexGetDatabase : public FRigVMFunction_AnimDatabaseBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Database Index
	UPROPERTY(meta = (Input))
	TObjectPtr<UAnimDatabaseIndex> Index;

	// Output Database
	UPROPERTY(meta = (Output))
	TObjectPtr<UAnimDatabase> Database;
};

/** Find frames in a Database Index */
USTRUCT(meta = (DisplayName = "Find Database Index Frames"))
struct FRigVMFunction_FindDatabaseIndexFrames : public FRigVMFunction_AnimDatabaseBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Database Index
	UPROPERTY(meta = (Input))
	TObjectPtr<UAnimDatabaseIndex> Index;

	// Name to look-up
	UPROPERTY(meta = (Input))
	FName Name;

	// Output Found
	UPROPERTY(meta = (Output))
	bool bFound = false;

	// Output Frames
	UPROPERTY(meta = (Output))
	FAnimDatabaseFrames Frames;
};

/** Find frame ranges in a Database Index */
USTRUCT(meta = (DisplayName = "Find Database Index Frame Ranges"))
struct FRigVMFunction_FindDatabaseIndexFrameRanges : public FRigVMFunction_AnimDatabaseBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Database Index
	UPROPERTY(meta = (Input))
	TObjectPtr<UAnimDatabaseIndex> Index;

	// Name to look-up
	UPROPERTY(meta = (Input))
	FName Name;

	// Output Found
	UPROPERTY(meta = (Output))
	bool bFound = false;

	// Output Frame Ranges
	UPROPERTY(meta = (Output))
	FAnimDatabaseFrameRanges FrameRanges;
};

/** Find frame attributes in a Database Index */
USTRUCT(meta = (DisplayName = "Find Database Index Frame Attribute"))
struct FRigVMFunction_FindDatabaseIndexFrameAttribute : public FRigVMFunction_AnimDatabaseBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Database Index
	UPROPERTY(meta = (Input))
	TObjectPtr<UAnimDatabaseIndex> Index;

	// Name to look-up
	UPROPERTY(meta = (Input))
	FName Name;

	// Output Found
	UPROPERTY(meta = (Output))
	bool bFound = false;

	// Output Frame Attribute
	UPROPERTY(meta = (Output))
	FAnimDatabaseFrameAttribute FrameAttribute;
};

#undef UE_API