// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LearningArray.h"

#include "Engine/DataAsset.h"
#include "Misc/FrameRate.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Animation/AnimNotifies/AnimNotify.h"
#include "Animation/AnimNotifies/AnimNotifyState.h"
#include "Logging/StructuredLog.h"
#include "Modules/ModuleManager.h"

#include "AnimDatabaseFrameRanges.h"
#include "AnimDatabaseFrameAttribute.h"

#include "AnimDatabase.generated.h"

#define UE_API ANIMDATABASE_API

class UAnimDatabase;
class UAnimDatabaseDebugDraw;
class UAnimDatabaseFramesFunction;
class UAnimDatabaseFrameRangesFunction;
class UAnimDatabaseFrameAttributeFunction;
struct FAnimDatabaseFrames;
struct FAnimDatabaseFrameRanges;
struct FAnimDatabaseFrameAttribute;
enum class EAnimDatabaseAttributeType : uint8;
class UAnimDatabaseQuery;

namespace UE::AnimDatabase
{
	struct FPoseAttributeDataView;
	struct FPoseRootDataView;
	struct FPoseLocalBoneDataView;
	struct FPoseDataView;
	struct FPoseDataConstView;
	struct FPoseAttributeDataConstView;
}

class USkeleton;
class USkeletalMesh;
class UAnimSequence;
class UMirrorDataTable;
class IAnimationDataController;
class UStaticMesh;

//--------------------------------------------------

class FAnimDatabaseModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
};

UE_API DECLARE_LOG_CATEGORY_EXTERN(LogAnimDatabase, Log, All);

//--------------------------------------------------

/** Rotation order used when exporting data as euler angles */
UENUM(BlueprintType)
enum class EAnimDatabaseRotationOrder : uint8
{
	XYZ,
	XZY,
	YXZ,
	YZX,
	ZXY,
	ZYX
};

/** Transform mode for use with animation database functions */
UENUM(BlueprintType)
enum class EAnimDatabaseTransformMode : uint8
{
	PreMultiply,
	PostMultiply,
	Replace
};

//--------------------------------------------------

/** Class containing all the UAnimDatabase Editor window viewport settings */
UCLASS()
class UAnimDatabaseViewportSettings : public UObject
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	/** Mesh to use for previewing the character */
	UPROPERTY(EditAnywhere, Category = "Mesh")
	TObjectPtr<USkeletalMesh> PreviewMesh = nullptr;

public:

	/** If to draw the skeleton of all characters */
	UPROPERTY(EditAnywhere, Category = "Skeleton")
	bool bDrawSkeleton = false;

	/** Draw a simple skeleton made up of lines connecting bones */
	UPROPERTY(EditAnywhere, Category = "Skeleton", meta = (EditCondition = "bDrawSkeleton", HideEditConditionToggle))
	bool bDrawSimpleSkeleton = false;

	/** Opacity of the drawn skeleton */
	UPROPERTY(EditAnywhere, Category = "Skeleton", meta = (ClampMin = "0", UIMin = "0", ClampMax = "1", UIMax = "1", EditCondition = "bDrawSkeleton", HideEditConditionToggle))
	float DrawSkeletonOpacity = 1.0f;

	/** The scale of the drawn skeleton bones */
	UPROPERTY(EditAnywhere, Category = "Skeleton", meta = (ClampMin = "0", UIMin = "0", EditCondition = "bDrawSkeleton", HideEditConditionToggle))
	float DrawSkeletonScale = 1.0f;

	/** If to only draw the skeleton of the mesh bones used by the PreviewMesh */
	UPROPERTY(EditAnywhere, Category = "Skeleton", meta = (EditCondition = "bDrawSkeleton", HideEditConditionToggle))
	bool bDrawMeshBonesOnly = false;

	/** Mesh bone LOD index to use when only drawing mesh bones */
	UPROPERTY(EditAnywhere, Category = "Skeleton", meta = (ClampMin = "0", UIMin = "0", ClampMax = "5", UIMax = "5", EditCondition = "bDrawSkeleton && bDrawMeshBonesOnly", HideEditConditionToggle))
	int32 DrawMeshBonesLOD = 0;

	/** Draw the bone names and indices */
	UPROPERTY(EditAnywhere, Category = "Skeleton", meta = (EditCondition = "bDrawSkeleton", HideEditConditionToggle))
	bool bDrawBoneNamesAndIndices = false;

	/** If to draw linear velocities for each of the skeleton's bones */
	UPROPERTY(EditAnywhere, Category = "Skeleton", meta = (EditCondition = "bDrawSkeleton", HideEditConditionToggle))
	bool bDrawLinearVelocities = false;

	/** The scale of the linear velocities being drawn */
	UPROPERTY(EditAnywhere, Category = "Skeleton", meta = (ClampMin = "0", UIMin = "0", EditCondition = "bDrawSkeleton && bDrawLinearVelocities", HideEditConditionToggle))
	float DrawLinearVelocitiesScale = 0.05f;

	/** Opacity of the linear velocities being drawn */
	UPROPERTY(EditAnywhere, Category = "Skeleton", meta = (ClampMin = "0", UIMin = "0", ClampMax = "1", UIMax = "1", EditCondition = "bDrawSkeleton && bDrawLinearVelocities", HideEditConditionToggle))
	float DrawLinearVelocitiesOpacity = 0.25f;

	/** If to draw angular velocities for each of the skeleton's bones */
	UPROPERTY(EditAnywhere, Category = "Skeleton", meta = (EditCondition = "bDrawSkeleton", HideEditConditionToggle))
	bool bDrawAngularVelocities = false;

	/** The scale of the angular velocities being drawn */
	UPROPERTY(EditAnywhere, Category = "Skeleton", meta = (ClampMin = "0", UIMin = "0", EditCondition = "bDrawSkeleton && bDrawAngularVelocities", HideEditConditionToggle))
	float DrawAngularVelocitiesScale = 2.5f;

	/** Opacity of the angular velocities being drawn */
	UPROPERTY(EditAnywhere, Category = "Skeleton", meta = (ClampMin = "0", UIMin = "0", ClampMax = "1", UIMax = "1", EditCondition = "bDrawSkeleton && bDrawAngularVelocities", HideEditConditionToggle))
	float DrawAngularVelocitiesOpacity = 0.25f;

	/** If to draw bone transforms */
	UPROPERTY(EditAnywhere, Category = "Skeleton", meta = (EditCondition = "bDrawSkeleton", HideEditConditionToggle))
	bool bDrawBoneTransforms = false;

public:

	/** If to draw the root of the skeleton as a transform */
	UPROPERTY(EditAnywhere, Category = "Root")
	bool bDrawRoot = true;

	/** Opacity of the drawn root */
	UPROPERTY(EditAnywhere, Category = "Root", meta = (ClampMin = "0", UIMin = "0", ClampMax = "1", UIMax = "1", EditCondition = "bDrawRoot", HideEditConditionToggle))
	float DrawRootOpacity = 1.0f;

	/** The scale of the drawn root */
	UPROPERTY(EditAnywhere, Category = "Root", meta = (ClampMin = "0", UIMin = "0", EditCondition = "bDrawRoot", HideEditConditionToggle))
	float DrawRootScale = 10.0f;

public:

	/** If to draw the trajectories of the selected ranges */
	UPROPERTY(EditAnywhere, Category = "Trajectory")
	bool bDrawTrajectories = false;

	/** If to draw the trajectory orientations of the selected ranges */
	UPROPERTY(EditAnywhere, Category = "Trajectory", meta = (EditCondition = "bDrawTrajectories", HideEditConditionToggle))
	bool bDrawTrajectoryOrientations = false;

public:

	/** Re-orients all ranges so that they start at the scene origin rather than wherever they started in the source data */
	UPROPERTY(EditAnywhere, Category = "Ranges")
	bool bRangesStartAtOrigin = true;

	/** Offset different ranges away from origin so they can be viewed together */
	UPROPERTY(EditAnywhere, Category = "Ranges", meta = (EditCondition = "bRangesStartAtOrigin", HideEditConditionToggle))
	bool bOffsetRanges = true;

	/** Spacing to use for offsetting ranges */
	UPROPERTY(EditAnywhere, Category = "Ranges", meta = (ClampMin = "0", UIMin = "0", EditCondition = "bOffsetRanges", HideEditConditionToggle))
	float RangeOffsetSpacing = 100.0f;

	/** If to offset selected ranges as a grid */
	UPROPERTY(EditAnywhere, Category = "Ranges", meta = (EditCondition = "bOffsetRanges", HideEditConditionToggle))
	bool bOffsetRangesInGrid = false;

	/** If to accumulate root motion during playback */
	UPROPERTY(EditAnywhere, Category = "Ranges")
	bool bAccumulateRootMotion = false;

	/** Remove root translation when display ranges */
	UPROPERTY(EditAnywhere, Category = "Ranges", meta = (EditCondition = "bRangesStartAtOrigin", HideEditConditionToggle))
	bool bDisableRootTranslation = false;

	/** Remove root rotation when display ranges */
	UPROPERTY(EditAnywhere, Category = "Ranges", meta = (EditCondition = "bRangesStartAtOrigin && bDisableRootTranslation", HideEditConditionToggle))
	bool bDisableRootRotation = false;

	/** If to draw the colored range identifier over the character to show which range in the query the animation is from */
	UPROPERTY(EditAnywhere, Category = "Ranges")
	bool bDrawRangeIdentifier = true;

	/** Seed used for the colors of the range identifiers */
	UPROPERTY(EditAnywhere, Category = "Ranges")
	int32 RangeIdentifierColorSeed = 1234;

	/** If to color the skeleton bones using the range identifier color  */
	UPROPERTY(EditAnywhere, Category = "Ranges")
	bool bColorSkeletonBonesUsingRangeIdentifier = false;

	/** Thickness of the range identifier */
	UPROPERTY(EditAnywhere, Category = "Ranges", meta = (ClampMin = "0", UIMin = "0", EditCondition = "bDrawRangeIdentifier", HideEditConditionToggle))
	float DrawRangeIdentifierThickness = 1.0f;

	/** Radius of the range identifier */
	UPROPERTY(EditAnywhere, Category = "Ranges", meta = (ClampMin = "0", UIMin = "0", EditCondition = "bDrawRangeIdentifier", HideEditConditionToggle))
	float DrawRangeIdentifierRadius = 2.0f;

	/** Opacity of the range identifier */
	UPROPERTY(EditAnywhere, Category = "Ranges", meta = (ClampMin = "0", UIMin = "0", ClampMax = "1", UIMax = "1", EditCondition = "bDrawRangeIdentifier", HideEditConditionToggle))
	float DrawRangeIdentifierOpacity = 0.5f;

	/** Height offset used to draw the range identifier */
	UPROPERTY(EditAnywhere, Category = "Ranges", meta = (EditCondition = "bDrawRangeIdentifier", HideEditConditionToggle))
	float DrawRangeIdentifierHeight = 200.0f;

	/** If to override the range identifier color */
	UPROPERTY(EditAnywhere, Category = "Ranges")
	bool bOverrideRangeIdentifierColor = false;

	/** Color to use to override the range identifier */
	UPROPERTY(EditAnywhere, Category = "Ranges", meta = (EditCondition = "bOverrideRangeIdentifierColor", HideEditConditionToggle))
	FLinearColor RangeIdentifierColorOverride = FLinearColor(1.0f, 0.0f, 1.0f, 1.0f);

public:

	/** If to draw the scene origin as a transform */
	UPROPERTY(EditAnywhere, Category = "Origin")
	bool bDrawOrigin = true;

	/** Thickness of the drawn scene origin */
	UPROPERTY(EditAnywhere, Category = "Origin", meta = (ClampMin = "0", UIMin = "0", EditCondition = "bDrawOrigin", HideEditConditionToggle))
	float DrawOriginLineThickness = 1.0f;

	/** Opacity of the drawn scene origin */
	UPROPERTY(EditAnywhere, Category = "Origin", meta = (ClampMin = "0", UIMin = "0", ClampMax = "1", UIMax = "1", EditCondition = "bDrawOrigin", HideEditConditionToggle))
	float DrawOriginOpacity = 0.5f;

	/** Scale of the drawn scene origin */
	UPROPERTY(EditAnywhere, Category = "Origin", meta = (ClampMin = "0", UIMin = "0", EditCondition = "bDrawOrigin", HideEditConditionToggle))
	float DrawOriginScale = 25.0f;

#endif
};

//--------------------------------------------------

namespace UE::AnimDatabase::Editor
{
	/** An entry in the list of queries used by UAnimDatabaseQuery. */
	struct FQueryEntry
	{
		const UAnimSequence* Sequence = nullptr;
		FString Name;
		int32 StartFrame = 0;
		int32 StopFrame = 0;
		bool bIsMirrored = false;
		bool bIsSelected = false;
		FLinearColor Color = FLinearColor::Black;
	};
}

/** Which ranges to pass from the query to the database function */
UENUM(BlueprintType)
enum class EAnimDatabaseFunctionQueryRanges : uint8
{
	Database = 0,
	Query = 1,
	Selection = 2,
	Custom = 3,
};

/**
 * This class is used for displaying the query window in the UAnimDatabase UI.
 * The query window is implemented as a details customization of this class.
 */
UCLASS()
class UAnimDatabaseQuery : public UObject
{
	GENERATED_BODY()

public:

#if WITH_EDITOR

	/** Forces a refresh of the query object */
	UFUNCTION(CallInEditor, Category = "Query", DisplayName = "Refresh")
	UE_API void ForceRefresh();

#endif

#if WITH_EDITORONLY_DATA

	/** A pointer to the query database, used when constructing the query ranges */
	UPROPERTY()
	TObjectPtr<UAnimDatabase> Database;

	/** Flag to force refresh the query during update */
	bool bForceRefresh = false;

	/** This hash is used to detect when the database might have changed and re-run the query */
	int32 DatabaseContentHash = 0;

	/** The query class being used to filter the database */
	UPROPERTY(EditAnywhere, Instanced, Category = "Query")
	TObjectPtr<UAnimDatabaseFrameRangesFunction> FrameRanges;

	/** Query Ranges object for the current query */
	FAnimDatabaseFrameRanges QueryRanges;

	/** List of UI entries for the current query */
	TArray<TSharedPtr<UE::AnimDatabase::Editor::FQueryEntry>> QueryEntries;

	/** Temporary array of active ranges used to compare selected ranges */
	TArray<int32> ActiveRanges;

	/** Indices of all the currently selected ranges */
	TArray<int32> SelectedRanges;

	/** The frame ranges of the selected ranges */
	FAnimDatabaseFrameRanges SelectedFrameRanges;

	/** Seed used for the colors of the range identifiers */
	int32 RangeIdentifierColorSeed = 1234;

#endif

#if WITH_EDITOR

	/**
	 * Checks if QueryRanges needs updating, and if so updates it. returns true if QueryRanges was updated (and therefore the UI may need updating),
	 * otherwise false.
	 */
	UE_API bool Update();

	/** Deletes the selected sequences from the database */
	UE_API void DeleteSelectedFromDatabase();

private:

	/** Internal function for updating the QueryEntries array from an updated QueryRanges */
	UE_API void UpdateQueryEntries();

#endif

public:

#if WITH_EDITORONLY_DATA

	/** Additional Frames to display in the timeline */
	UPROPERTY(EditAnywhere, Category = "Timeline")
	TArray<FAnimDatabaseFramesEntry> AdditionalFrames;

	/** Additional Frame Ranges to display in the timeline */
	UPROPERTY(EditAnywhere, Category = "Timeline")
	TArray<FAnimDatabaseFrameRangesEntry> AdditionalFrameRanges;

	/** Additional Frame Attributes to display in the timeline */
	UPROPERTY(EditAnywhere, Category = "Timeline")
	TArray<FAnimDatabaseFrameAttributeEntry> AdditionalFrameAttributes;

public:

	/** Custom debug draw object that can be used to add additional debug draw behavior */
	UPROPERTY(EditAnywhere, Instanced, Category = "Debug Draw")
	TObjectPtr<UAnimDatabaseDebugDraw> DebugDrawer;

public:

	/** Which set of ranges to pass to the database function */
	UPROPERTY(EditAnywhere, Category = "Function")
	EAnimDatabaseFunctionQueryRanges FunctionRanges = EAnimDatabaseFunctionQueryRanges::Database;

	/** Custom frame ranges to use for the database function */
	UPROPERTY(EditAnywhere, Instanced, Category = "Function", meta = (EditCondition = "FunctionRanges == EAnimDatabaseFunctionQueryRanges::Custom", EditConditionHides))
	TObjectPtr<UAnimDatabaseFrameRangesFunction> CustomFrameRanges;

	/** If the function call should transact (be undo-able) */
	UPROPERTY(EditAnywhere, Category = "Function")
	bool bFunctionShouldTransact = true;

	/** Function object to call */
	UPROPERTY(EditAnywhere, Instanced, Category = "Function")
	TObjectPtr<UAnimDatabaseFunction> FunctionObject;

#endif

#if WITH_EDITOR

	/** Evaluate the given function on the database with the chosen ranges */
	UFUNCTION(CallInEditor, Category = "Function", DisplayName = "Run Function")
	void RunFunction();

#endif
};

//--------------------------------------------------

UENUM(BlueprintType)
enum class EAnimDatabaseSampler : uint8
{
	/**
	 * Snaps to the nearest keyframe in the animation sequence.
	 *
	 * This interpolation mode is not recommended for downstream tasks since it does not produce any velocities when you sample the animation data, 
	 * but may be useful for debugging and inspecting the data.
	 */
	Nearest,

	/**
	 * Linearly interpolates animation data.
	 *
	 * This will give the same result as what is seen in the AnimSequence asset viewer. The main problem with this interpolation method is that it 
	 * produces a velocity discontinuity as it switches key-frames. This means velocities tend to change in discrete jumps and sometimes the velocity 
	 * discontinuity is visible when playing the animation back slowly.
	 */
	Linear,

	/** 
	 * Interpolations animation data using a cubic catmull-rom spline.
	 *
	 * This produces a smooth interpolation of the animation data with smoothly varying velocities. The only potential downside of this method is that
	 * it can produce "over-shooting" where the joint rotations and translations can over-extend past the keyframes during the interpolation period.
	 */
	Cubic,

	/**
	 * Interpolates animation data using a monotone cubic spline.
	 * 
	 * This produces a similar result to the Cubic interpolation but uses monotone interpolation to ensure that the animation pose never "over-shoots"
	 * during the interpolation. The cost of this is that CubicMono interpolation can sometimes produce more shaky looking motion at end-effectors.
	 */
	CubicMono,
};

DECLARE_MULTICAST_DELEGATE(FAnimDatabaseOnSkeletonChanged);

/**
 * Animation Database Asset
 * 
 * This asset contains an array of animations, as well as a skeleton, and a mirror table. It is assumed that all added animations should
 * match the provided skeleton. As well as providing a hook for a custom UI and tooling, this asset provides functions for sampling animation data
 * at a single uniform framerate and in the pose representation used by the database.
 * 
 * The number of "Sequences" in this database (as returned by GetSequenceNum()) will be two times the number of entries in "Entries" when the mirror
 * data table is present and valid, otherwise it will be the same as the number of entries.
 */
UCLASS(MinimalAPI, BlueprintType, Category = "Animation", meta = (DisplayName = "Animation Database"))
class UAnimDatabase : public UDataAsset
{
	GENERATED_BODY()

private:

	UE_API UAnimDatabase(const FObjectInitializer& ObjectInitializer);

public:

	/** Skeleton structure for Animation Sequences */
	UPROPERTY(EditAnywhere, Category = "Database")
	TObjectPtr<USkeleton> Skeleton;

	/** Mirror Data Table. Leave as null if you don't want to include mirrored animations in the database. */
	UPROPERTY(EditAnywhere, Category = "Database")
	TObjectPtr<UMirrorDataTable> MirrorDataTable;

	/** FrameRate to use for sampling the database. This should generally set to the target frame rate of your game. */
	UPROPERTY(EditAnywhere, Category = "Database")
	FFrameRate FrameRate = FFrameRate(60, 1);

	/** Interpolation method used for sampling poses in the database.  */
	UPROPERTY(EditAnywhere, Category = "Database")
	EAnimDatabaseSampler PoseSampler = EAnimDatabaseSampler::Cubic;

	/** Interpolation method used for sampling curves in the database.  */
	UPROPERTY(EditAnywhere, Category = "Database")
	EAnimDatabaseSampler CurveSampler = EAnimDatabaseSampler::Linear;

	/** List of Animations in Database */
	UPROPERTY(EditAnywhere, Category = "Database")
	TArray<TObjectPtr<UAnimSequence>> Entries;


public:
	
#if WITH_EDITORONLY_DATA

	/** Callback to fire when the skeleton is changed */
	FAnimDatabaseOnSkeletonChanged OnSkeletonChanged;
#endif

#if WITH_EDITOR
	/** We use this to ensure that entries added have a matching skeleton */
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& Event) override;
#endif

public:

	/**
	 * Gets a hash value which can be used to test if the underlying content in the database, or database properties have changed. This hash should 
	 * not be relied on for detecting changes with certainty, and so should only be used for non-critical purposes (e.g. UI updates).
	 */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	UE_API int32 GetContentHash() const;

public:

	/** Get the database frame rate */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	UE_API FFrameRate GetFrameRate() const;

	/** Get a pointer to the database skeleton */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	UE_API USkeleton* GetSkeleton() const;

	/** Get a pointer to the database mirror table */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	UE_API UMirrorDataTable* GetMirrorDataTable() const;

	/** Gets the number of bones in the database skeleton */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	UE_API int32 GetBoneNum() const;

	/** Gets the bone name for the given bone index in the database skeleton */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	UE_API FName GetBoneName(const int32 BoneIndex) const;

	/** Gets the bone names for all bones on the skeleton */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	UE_API void GetBoneNames(TArray<FName>& OutBoneNames) const;
	UE_API void GetBoneNamesToArrayView(TArrayView<FName> OutBoneNames) const;

	/** Finds the bone index for a given bone name in the database skeleton */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	UE_API int32 FindBoneIndex(const FName BoneName) const;

	/** Finds the bone indices associated with the given bone names in the database skeleton */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	UE_API void FindBoneIndices(TArray<int32>& OutBoneIndices, const TArray<FName>& BoneNames) const;
	UE_API void FindBoneIndicesFromArrayViews(const TArrayView<int32> OutBoneIndices, const TArrayView<const FName> BoneNames) const;

	/** Gets the parent bone index */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	UE_API int32 GetBoneParent(const int32 BoneIndex) const;

	/** Gets the array of bone parent indices for the database skeleton */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	UE_API void GetBoneParents(TArray<int32>& OutParents) const;
	UE_API void GetBoneParentsToArrayView(const TArrayView<int32> OutParents) const;

public:

	/** Gets the number of sequences in the database. This will be two times the number of entries when a mirror table is provided. */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	UE_API int32 GetSequenceNum() const;

	/** Finds the sequence index associated with a given animation */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	UE_API int32 FindSequenceIndex(UAnimSequence* AnimSequence, const bool bIsMirrored) const;

	/** Gets a pointer for the animation corresponding to a particular sequence index */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	UE_API UAnimSequence* GetAnimSequence(const int32 SequenceIdx) const;

	/**
	 * Waits for the animation compression to complete for the given sequence. It is generally best to do this before trying to sample a sequence as
	 * otherwise the evaluation can be incredibly slow.
	 */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase")
	UE_API void WaitForCompressionOnAnimSequence(const int32 SequenceIdx) const;

	/**
	 * Waits for the animation compression to complete for the given sequences. It is generally best to do this before trying to sample a sequence as
	 * otherwise the evaluation can be incredibly slow.
	 */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase")
	UE_API void WaitForCompressionOnAnimSequences(const TArray<int32>& SequenceIndices) const;
	UE_API void WaitForCompressionOnAnimSequencesFromArrayView(const TArrayView<const int32> SequenceIndices) const;
	UE_API void WaitForCompressionOnAll() const;

	/** Returns if a given sequence index is for a mirrored animation or not */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	UE_API bool GetIsMirrored(const int32 SequenceIdx) const;

	/** Returns the duration of a sequence in seconds */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	UE_API float GetSequenceDuration(const int32 SequenceIdx) const;

	/** Gets the number of frames in a sequence */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	UE_API int32 GetSequenceFrameNum(const int32 SequenceIdx) const;

	/** Gets the asset name for a sequence index */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	UE_API FString GetSequenceAssetName(const int32 SequenceIdx) const;

	/** Gets the path string for a sequence index */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	UE_API FString GetSequencePathString(const int32 SequenceIdx) const;

	/** Gets the total duration of all sequences */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	UE_API float GetTotalDuration() const;

	/** Gets the total number of frames across all sequences */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	UE_API int32 GetTotalFrameNum() const;

	/** Gets the sequence time for the start of a given frame */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	UE_API float GetSequenceTimeFromFrame(const int32 SequenceIdx, const int32 FrameIdx) const;

	/** Gets the closest frame associated with a sequence time */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	UE_API int32 GetClosestFrameFromSequenceTime(const int32 SequenceIdx, const float SequenceTime) const;

public:

	/** Gets the bone location in the reference skeleton */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	UE_API FVector GetBoneReferenceLocation(const int32 BoneIdx);

	/** Gets the bone locations in the reference skeleton */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	UE_API void GetBoneReferenceLocations(TArray<FVector>& OutReferenceLocations);
	UE_API void GetBoneReferenceLocationsToArrayView(TArrayView<FVector> OutReferenceLocations);

	/** Gets the bone rotation in the reference skeleton */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	UE_API FQuat GetBoneReferenceRotation(const int32 BoneIdx);

	/** Gets the bone rotations in the reference skeleton */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	UE_API void GetBoneReferenceRotations(TArray<FQuat>& OutReferenceRotations);
	UE_API void GetBoneReferenceRotationsToArrayView(TArrayView<FQuat> OutReferenceRotations);

	/** Gets the bone scale in the reference skeleton */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	UE_API FVector GetBoneReferenceScale(const int32 BoneIdx);

public:

	/** Gets the root transform for a given sequence and start frame */
	UE_API void GetRootTransform(const TLearningArrayView<1, FTransform> OutTransforms, const int32 SequenceIdx, const int32 FrameStart) const;

	/** Gets the root location for a given sequence and start frame */
	UE_API void GetRootLocation(const TLearningArrayView<1, FVector> OutLocations, const int32 SequenceIdx, const int32 FrameStart) const;

	/** Gets the root rotation for a given sequence and start frame */
	UE_API void GetRootRotation(const TLearningArrayView<1, FQuat4f> OutRotations, const int32 SequenceIdx, const int32 FrameStart) const;

	/** Gets the root direction for a given sequence and start frame */
	UE_API void GetRootDirection(const TLearningArrayView<1, FVector3f> OutDirections, const int32 SequenceIdx, const int32 FrameStart, const FVector3f ForwardVector = FVector3f(0.0f, 1.0f, 0.0f)) const;

	/** Gets the root linear velocity for a given sequence and start frame */
	UE_API void GetRootLinearVelocity(const TLearningArrayView<1, FVector3f> OutLinearVelocities, const int32 SequenceIdx, const int32 FrameStart) const;

	/** Gets the root angular velocity for a given sequence and start frame */
	UE_API void GetRootAngularVelocity(const TLearningArrayView<1, FVector3f> OutAngularVelocities, const int32 SequenceIdx, const int32 FrameStart) const;

	/** Get a number of frames of root data for a given sequence and frame range */
	UE_API void GetPoseRootData(
		const UE::AnimDatabase::FPoseRootDataView& OutPoseRootData,
		const int32 SequenceIdx,
		const int32 FrameStart) const;

	/** Gets a number of frames of local bone data for a given sequence and frame range */
	UE_API void GetPoseLocalBoneData(
		const UE::AnimDatabase::FPoseLocalBoneDataView& OutPoseLocalBoneData,
		const int32 SequenceIdx,
		const int32 FrameStart) const;

	/** Gets a number of frames of local bone data for a given sequence and frame range, and a subset of bones */
	UE_API void GetPoseLocalBoneSubsetData(
		const UE::AnimDatabase::FPoseLocalBoneDataView& OutPoseLocalBoneData,
		const int32 SequenceIdx,
		const int32 FrameStart,
		const UE::Learning::FIndexSet BoneIndices) const;

	/** Gets if the given curves are active for a given sequence and frame range */
	UE_API void GetCurveActiveData(
		const TLearningArrayView<2, bool> OutCurveActive,
		const int32 SequenceIdx,
		const int32 FrameStart,
		const TArrayView<const FName> CurveNames) const;

	/** Gets curve data for a given sequence and frame range */
	UE_API void GetCurveData(
		const TLearningArrayView<2, float> OutCurveValues,
		const TLearningArrayView<2, float> OutCurveVelocities,
		const TLearningArrayView<2, bool> OutCurveActive,
		const int32 SequenceIdx,
		const int32 FrameStart,
		const TArrayView<const FName> CurveNames) const;

	/** Gets a number of frames of attribute data for a given sequence and frame range */
	UE_API void GetAttributeData(
		const UE::AnimDatabase::FPoseAttributeDataView& OutAttributeData,
		const int32 SequenceIdx,
		const int32 FrameStart,
		const TArrayView<const FAnimDatabaseFrameAttribute> FrameAttributes) const;

	/** Gets a number of frames of pose data for a given sequence and frame range */
	UE_API void GetPoseData(
		const UE::AnimDatabase::FPoseDataView& OutPoseData,
		const int32 SequenceIdx,
		const int32 FrameStart,
		const TArrayView<const FAnimDatabaseFrameAttribute> FrameAttributes = {}) const;

	/** Gets a number of frames of pose data for a given sequence and frame range, and a subset of bones */
	UE_API void GetPoseSubsetData(
		const UE::AnimDatabase::FPoseDataView& OutPoseData,
		const int32 SequenceIdx,
		const int32 FrameStart,
		const UE::Learning::FIndexSet BoneIndices,
		const TArrayView<const FAnimDatabaseFrameAttribute> FrameAttributes = {}) const;

public:

	/** Samples the root transform for a given sequence and sequence time */
	UE_API FTransform SampleRootTransform(const int32 SequenceIdx, const float SequenceTime) const;

	/** Samples the root location for a given sequence and sequence time */
	UE_API FVector SampleRootLocation(const int32 SequenceIdx, const float SequenceTime) const;

	/** Samples the root rotation for a given sequence and sequence time */
	UE_API FQuat4f SampleRootRotation(const int32 SequenceIdx, const float SequenceTime) const;

	/** Samples the root direction for a given sequence and sequence time */
	UE_API FVector3f SampleRootDirection(const int32 SequenceIdx, const float SequenceTime, const FVector3f ForwardVector = FVector3f(0.0f, 1.0f, 0.0f)) const;

	/** Samples the root linear velocity for a given sequence and sequence time */
	UE_API FVector3f SampleRootLinearVelocity(const int32 SequenceIdx, const float SequenceTime) const;
	
	/** Samples root data for the given sequences and times in those sequences */
	UE_API void SamplePoseRootData(
		const UE::AnimDatabase::FPoseRootDataView& OutPoseRootData,
		const TLearningArrayView<1, const int32> SequenceIndices,
		const TLearningArrayView<1, const float> SequenceTimes) const;

	/** Samples local bone data for the given sequences and times in those sequences */
	UE_API void SamplePoseLocalBoneData(
		const UE::AnimDatabase::FPoseLocalBoneDataView& OutPoseLocalBoneData,
		const TLearningArrayView<1, const int32> SequenceIndices,
		const TLearningArrayView<1, const float> SequenceTimes) const;

	/** Samples local bone data for the given sequences and times in those sequences, and a subset of bones. */
	UE_API void SamplePoseLocalBoneSubsetData(
		const UE::AnimDatabase::FPoseLocalBoneDataView& OutPoseLocalBoneData,
		const TLearningArrayView<1, const int32> SequenceIndices,
		const TLearningArrayView<1, const float> SequenceTimes,
		const UE::Learning::FIndexSet BoneIndices) const;

	/** Samples if curve are active for the given sequences and times in those sequences. */
	UE_API void SampleCurveActiveData(
		const TLearningArrayView<2, bool> OutCurveActive,
		const TLearningArrayView<1, const int32> SequenceIndices,
		const TLearningArrayView<1, const float> SequenceTimes,
		const TArrayView<const FName> CurveNames) const;

	/** Samples curve data for the given sequences and times in those sequences. */
	UE_API void SampleCurveData(
		const TLearningArrayView<2, float> OutCurveValues,
		const TLearningArrayView<2, float> OutCurveVelocities,
		const TLearningArrayView<2, bool> OutCurveActive,
		const TLearningArrayView<1, const int32> SequenceIndices,
		const TLearningArrayView<1, const float> SequenceTimes,
		const TArrayView<const FName> CurveNames) const;

	/** Samples attribute data for the given sequences and times in those sequences. */
	UE_API void SampleAttributeData(
		const UE::AnimDatabase::FPoseAttributeDataView& OutAttributeData,
		const TLearningArrayView<1, const int32> SequenceIndices,
		const TLearningArrayView<1, const float> SequenceTimes,
		const TLearningArrayView<1, const FAnimDatabaseFrameAttribute> FrameAttributes) const;

	/** Samples pose data for the given sequences and times in those sequences. */
	UE_API void SamplePoseData(
		const UE::AnimDatabase::FPoseDataView& OutPoseData,
		const TLearningArrayView<1, const int32> SequenceIndices,
		const TLearningArrayView<1, const float> SequenceTimes,
		const TLearningArrayView<1, const FAnimDatabaseFrameAttribute> FrameAttributes = {}) const;

	/** Samples pose data for the given sequences and times in those sequences, for a subset of bones. */
	UE_API void SamplePoseSubsetData(
		const UE::AnimDatabase::FPoseDataView& OutPoseData,
		const TLearningArrayView<1, const int32> SequenceIndices,
		const TLearningArrayView<1, const float> SequenceTimes,
		const UE::Learning::FIndexSet BoneIndices,
		const TLearningArrayView<1, const FAnimDatabaseFrameAttribute> FrameAttributes = {}) const;

public:

	/** Editor-only object containing the viewport settings for the editor window */
	UPROPERTY(Instanced)
	TObjectPtr<UAnimDatabaseViewportSettings> ViewportSettings;

	/** Editor-only object containing the query state for the editor window */
	UPROPERTY(Instanced)
	TObjectPtr<UAnimDatabaseQuery> Query;
};

//--------------------------------------------------

/** Class that can be implemented to perform an operation on an animation database */
UCLASS(Abstract, HideDropdown, EditInlineNew, DefaultToInstanced, CollapseCategories, BlueprintType, Blueprintable)
class UAnimDatabaseFunction : public UObject
{
	GENERATED_BODY()

#if WITH_EDITOR

public:

	/** Run the database function on the provided database and frame ranges */
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "AnimDatabase", meta = (ForceAsFunction))
	UE_API void Run(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact);

#endif
};

/**
 * A blueprint library providing functions for writing back frame attribute data into an animation database.
 *
 * This library provides something akin to AnimModifiers, where it becomes possible to write an asset action which can extract some frame attributes
 * and then write these back to the animation data either in the form of AnimNotifies, AnimNotifyStates, Curves, or modifications to bone transforms.
 * This method of AnimModifiers is far more efficient as it can be scripted via blueprints but still perform processing in batch.
 */
UCLASS(BlueprintType)
class UAnimDatabaseLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/** Gets all the AnimNotify classes present in a given AnimSequence */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase")
	static UE_API void GetAnimSequenceAnimNotifyClasses(TArray<TSubclassOf<UAnimNotify>>& OutClasses, UAnimSequence* AnimSequence);

	/** Gets all the AnimNotifyState classes present in a given AnimSequence */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase")
	static UE_API void GetAnimSequenceAnimNotifyStateClasses(TArray<TSubclassOf<UAnimNotifyState>>& OutClasses, UAnimSequence* AnimSequence);

	/** Gets the times of each AnimNotify of the given class, in the given sequence */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase")
	static UE_API void GetAnimSequenceAnimNotifyTimes(TArray<float>& OutTimes, const TSubclassOf<UAnimNotify>& Class, UAnimSequence* AnimSequence);

	/** Gets the times and durations of each AnimNotifyState of the given class, in the given sequence */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase")
	static UE_API void GetAnimSequenceAnimNotifyStateTimesAndDurations(TArray<float>& OutTimes, TArray<float>& OutDurations, const TSubclassOf<UAnimNotifyState>& Class, UAnimSequence* AnimSequence);

	/** Sets the pose data on a sequence using the given FPoseDataConstView */
	static UE_API void SetAnimSequencePoseData(
		UAnimSequence* AnimSequence, 
		TScriptInterface<IAnimationDataController> Controller, 
		const int32 StartFrame, 
		const UE::AnimDatabase::FPoseDataConstView& PoseData,
		const UE::Learning::FIndexSet UsedBones,
		const bool bShouldTransact = false);

	/** Adds AnimNotifies of the provided class to a sequence at the given times */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase", meta = (DevelopmentOnly))
	static UE_API void AddAnimNotifiesToSequence(UAnimSequence* AnimSequence, const FName TrackName, const TSubclassOf<UAnimNotify>& Class, const TArray<float>& Times);
	static UE_API void AddAnimNotifiesToSequenceArrayView(UAnimSequence* AnimSequence, const FName TrackName, const TSubclassOf<UAnimNotify>& Class, const TArrayView<const float> Times);

	/** Adds AnimNotifyStates of the provided class to a sequence at the given times with the given durations */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase", meta = (DevelopmentOnly))
	static UE_API void AddAnimNotifyStatesToSequence(UAnimSequence* AnimSequence, const FName TrackName, const TSubclassOf<UAnimNotifyState>& Class, const TArray<float>& Times, const TArray<float>& Durations);
	static UE_API void AddAnimNotifyStatesToSequenceArrayView(UAnimSequence* AnimSequence, const FName TrackName, const TSubclassOf<UAnimNotifyState>& Class, const TArrayView<const float> Times, const TArrayView<const float> Durations);

	/** Removes all AnimNotifies and AnimNotifyStates from a sequence */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase", meta = (DevelopmentOnly))
	static UE_API void RemoveAllAnimNotifiesAndAnimNotifyStatesFromSequence(UAnimSequence* AnimSequence);

	/** Enables root motion and force root lock on the given animation sequences */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase", meta = (DevelopmentOnly))
	static UE_API void EnableRootMotionAndForceRootLockOnSequences(const TArray<UAnimSequence*>& Sequences);
	static UE_API void EnableRootMotionAndForceRootLockOnSequencesArrayView(TConstArrayView<UAnimSequence*> Sequences);

public:

	/** Removes all AnimNotifies and AnimNotifyStates from the given frame ranges */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase", meta = (DevelopmentOnly))
	static UE_API void DatabaseRemoveNotifies(UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges);

	/** Exports all AnimNotifies and AnimNotifyStates as JSON */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase", meta = (DevelopmentOnly, RelativePath, FilePathFilter = "json"))
	static UE_API void DatabaseExportNotifies(UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FFilePath& Path);

	/** Imports AnimNotifies and AnimNotifyStates from a JSON file */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase", meta = (DevelopmentOnly, RelativePath, FilePathFilter = "json"))
	static UE_API void DatabaseImportNotifies(UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FFilePath& Path);

public:

	/** Exports the given frame ranges in the database as BVH files into the provided directory */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase", meta = (DevelopmentOnly, RelativePath))
	static UE_API void DatabaseExportAsBVH(UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FDirectoryPath& Path, const bool bAddRootAxesChangeRotation = true, const EAnimDatabaseRotationOrder RotationOrder = EAnimDatabaseRotationOrder::XYZ);

	/** Imports all the BVH files from the given ImportPath directory into the AssetPath directory and adds them to the database */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase", meta = (DevelopmentOnly, RelativePath))
	static UE_API void DatabaseImportFromBVH(UAnimDatabase* Database, const FDirectoryPath& ImportPath, const FDirectoryPath& AssetPath, const bool bRemoveRootAxesChangeRotation = true, const bool bAddToDatabase = true, const bool bIgnoreMirrored = false);

	/** Exports the given frame ranges in the database as AnimSequences into the provided path */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase", meta = (DevelopmentOnly, RelativePath))
	static UE_API void DatabaseExportAsAnimSequences(UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FDirectoryPath& Path, const FString& AssetNameFormatString, const bool bExportMirrored, const bool bShouldTransact = false);

	/** Exports the root motion of given frame ranges in the database as a JSON file */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase", meta = (DevelopmentOnly, RelativePath, FilePathFilter = "json"))
	static UE_API void DatabaseExportRootMotion(UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FFilePath& Path);

public:

	/** Write the provided float frame attribute back to the animations in the database with the given curve name */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase", meta = (DevelopmentOnly))
	static UE_API void DatabaseAddCurve(UAnimDatabase* Database, const FName CurveName, const FAnimDatabaseFrameAttribute& FloatFrameAttribute, const FLinearColor Color = FLinearColor::Red, const bool bSparseKeys = false, const bool bShouldTransact = false);

	/** Write the provided float frame attributes back to the animations in the database with the given curve names */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase", meta = (DevelopmentOnly, AutoCreateRefTerm = "Colors"))
	static UE_API void DatabaseAddCurves(UAnimDatabase* Database, const TArray<FName>& CurveNames, const TArray<FAnimDatabaseFrameAttribute>& FloatFrameAttributes, const TArray<FLinearColor>& Colors, const bool bSparseKeys = false, const bool bShouldTransact = false);
	static UE_API void DatabaseAddCurvesFromArrayViews(UAnimDatabase* Database, const TArrayView<const FName> CurveNames, const TArrayView<const FAnimDatabaseFrameAttribute> FloatFrameAttributes, const TArrayView<const FLinearColor> Colors, const bool bSparseKeys = false, const bool bShouldTransact = false);

public:

	/** Create AnimNotifies at the provided Frames in the database using the given track name and class */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase", meta = (DevelopmentOnly))
	static UE_API void DatabaseAddAnimNotify(UAnimDatabase* Database, const FName AnimNotifyTrackName, const TSubclassOf<UAnimNotify> Class, const FAnimDatabaseFrames& Frames, const FLinearColor Color = FLinearColor::Red, const bool bShouldTransact = false);

	/** Create AnimNotifies at the provided Frames in the database using the given track names and classes */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase", meta = (DevelopmentOnly, AutoCreateRefTerm = "Colors"))
	static UE_API void DatabaseAddAnimNotifies(UAnimDatabase* Database, const TArray<FName>& AnimNotifyTrackNames, const TArray<TSubclassOf<UAnimNotify>>& Classes, const TArray<FAnimDatabaseFrames>& Frames, const TArray<FLinearColor>& Colors, const bool bShouldTransact = false);
	static UE_API void DatabaseAddAnimNotifiesFromArrayViews(UAnimDatabase* Database, const TArrayView<const FName> AnimNotifyTrackNames, const TArrayView<const TSubclassOf<UAnimNotify>> Classes, const TArrayView<const FAnimDatabaseFrames> Frames, const TArrayView<const FLinearColor> Colors, const bool bShouldTransact = false);

	/** Create AnimNotifies at the provided Frames from a duplicate of the given AnimNotify */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase", meta = (DevelopmentOnly))
	static UE_API void DatabaseAddAnimNotifyObject(UAnimDatabase* Database, const FName& TrackName, UAnimNotify* AnimNotify, const FAnimDatabaseFrames& Frames, const FLinearColor Color = FLinearColor::Red, const bool bShouldTransact = false);

	/** Create AnimNotifies at the provided Frames in the database using the given track names and duplicates of the given AnimNotifies */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase", meta = (DevelopmentOnly, AutoCreateRefTerm="Colors"))
	static UE_API void DatabaseAddAnimNotifyObjects(UAnimDatabase* Database, const TArray<FName>& AnimNotifyTrackNames, const TArray<UAnimNotify*>& AnimNotifies, const TArray<FAnimDatabaseFrames>& Frames, const TArray<FLinearColor>& Colors, const bool bShouldTransact = false);
	static UE_API void DatabaseAddAnimNotifyObjectsFromArrayViews(UAnimDatabase* Database, const TArrayView<const FName> AnimNotifyTrackNames, const TArrayView<UAnimNotify* const> AnimNotifies, const TArrayView<const FAnimDatabaseFrames> Frames, const TArrayView<const FLinearColor> Colors, const bool bShouldTransact = false);

	/** Remove all the anim notifies on a given track */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase", meta = (DevelopmentOnly))
	static UE_API void DatabaseRemoveAnimNotifiesFromTrack(UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName AnimNotifyTrackName, const bool bShouldTransact = false);

	/** Remove all the anim notifies from the given tracks */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase", meta = (DevelopmentOnly))
	static UE_API void DatabaseRemoveAnimNotifiesFromTracks(UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArray<FName>& AnimNotifyTrackNames, const bool bShouldTransact = false);
	static UE_API void DatabaseRemoveAnimNotifiesFromTracksArrayView(UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const FName> AnimNotifyTrackNames, const bool bShouldTransact = false);

public:

	/** Create AnimNotifyStates at the provided FrameRanges in the database using the given track name and class */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase", meta = (DevelopmentOnly))
	static UE_API void DatabaseAddAnimNotifyState(UAnimDatabase* Database, const FName AnimNotifyStateTrackName, const TSubclassOf<UAnimNotifyState> Class, const FAnimDatabaseFrameRanges& FrameRanges, const FLinearColor Color = FLinearColor::Red, const bool bShouldTransact = false);

	/** Create AnimNotifyStates at the provided FrameRanges in the database using the given track names and classes */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase", meta = (DevelopmentOnly, AutoCreateRefTerm = "Colors"))
	static UE_API void DatabaseAddAnimNotifyStates(UAnimDatabase* Database, const TArray<FName>& AnimNotifyStateTrackNames, const TArray<TSubclassOf<UAnimNotifyState>>& Classes, const TArray<FAnimDatabaseFrameRanges>& FrameRanges, const TArray<FLinearColor>& Colors, const bool bShouldTransact = false);
	static UE_API void DatabaseAddAnimNotifyStatesFromArrayViews(UAnimDatabase* Database, const TArrayView<const FName> AnimNotifyStateTrackNames, const TArrayView<const TSubclassOf<UAnimNotifyState>> Classes, const TArrayView<const FAnimDatabaseFrameRanges> FrameRanges, const TArrayView<const FLinearColor> Colors, const bool bShouldTransact = false);

	/** Create AnimNotifyStates at the provided FrameRanges from a duplicate of the given AnimNotifyState */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase", meta = (DevelopmentOnly))
	static UE_API void DatabaseAddAnimNotifyStateObject(UAnimDatabase* Database, const FName TrackName, UAnimNotifyState* AnimNotifyState, const FAnimDatabaseFrameRanges& FrameRanges, const FLinearColor Color = FLinearColor::Red, const bool bShouldTransact = false);

	/** Create AnimNotifyStates at the provided FrameRanges in the database using the given track names and duplicates of the given AnimNotifyStates */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase", meta = (DevelopmentOnly, AutoCreateRefTerm = "Colors"))
	static UE_API void DatabaseAddAnimNotifyStateObjects(UAnimDatabase* Database, const TArray<FName>& AnimNotifyStateTrackNames, const TArray<UAnimNotifyState*>& AnimNotifyStates, const TArray<FAnimDatabaseFrameRanges>& FrameRanges, const TArray<FLinearColor>& Colors, const bool bShouldTransact = false);
	static UE_API void DatabaseAddAnimNotifyStateObjectsFromArrayViews(UAnimDatabase* Database, const TArrayView<const FName> AnimNotifyStateTrackNames, const TArrayView<UAnimNotifyState* const> AnimNotifyStates, const TArrayView<const FAnimDatabaseFrameRanges> FrameRanges, const TArrayView<const FLinearColor> Colors, const bool bShouldTransact = false);

public:

	/** Create SyncMarkers at the provided Frames in the database using the given track name and sync marker name */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase", meta = (DevelopmentOnly))
	static UE_API void DatabaseAddSyncMarker(UAnimDatabase* Database, const FName SyncMarkerTrackName, const FName SyncMarkerName, const FAnimDatabaseFrames& Frames, const FLinearColor Color = FLinearColor::Red, const bool bShouldTransact = false);

	/** Create SyncMarkers at the provided Frames in the database using the given track names and sync marker names */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase", meta = (DevelopmentOnly, AutoCreateRefTerm = "Colors"))
	static UE_API void DatabaseAddSyncMarkers(UAnimDatabase* Database, const TArray<FName>& SyncMarkerTrackNames, const TArray<FName>& SyncMarkerNames, const TArray<FAnimDatabaseFrames>& Frames, const TArray<FLinearColor>& Colors, const bool bShouldTransact = false);
	static UE_API void DatabaseAddSyncMarkersFromArrayViews(UAnimDatabase* Database, const TArrayView<const FName> SyncMarkerTrackNames, const TArrayView<const  FName> SyncMarkerNames, const TArrayView<const FAnimDatabaseFrames> Frames, const TArrayView<const FLinearColor> Colors, const bool bShouldTransact = false);

	/** Remove all the sync markers on a given track */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase", meta = (DevelopmentOnly))
	static UE_API void DatabaseRemoveSyncMarkerFromTrack(UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName SyncMarkerTrackName, const bool bShouldTransact = false);

	/** Remove all the sync markers from the given tracks */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase", meta = (DevelopmentOnly))
	static UE_API void DatabaseRemoveSyncMarkerFromTracks(UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArray<FName>& SyncMarkerTrackNames, const bool bShouldTransact = false);
	static UE_API void DatabaseRemoveSyncMarkerFromTracksArrayView(UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const FName> SyncMarkerTrackNames, const bool bShouldTransact = false);

public:

	/** Set the given bone's local transform in the database at the given transform frame attribute */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase", meta = (DevelopmentOnly))
	static UE_API void DatabaseSetLocalBoneTransform(UAnimDatabase* Database, const FName BoneName, const FAnimDatabaseFrameAttribute& TransformFrameAttribute, const bool bShouldTransact = false);

	/** Set the given bones' local transforms in the database at the given transform frame attributes */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase", meta = (DevelopmentOnly))
	static UE_API void DatabaseSetLocalBoneTransforms(UAnimDatabase* Database, const TArray<FName>& BoneNames, const TArray<FAnimDatabaseFrameAttribute>& TransformFrameAttributes, const bool bShouldTransact = false);
	static UE_API void DatabaseSetLocalBoneTransformsFromArrayViews(UAnimDatabase* Database, const TArrayView<const FName> BoneNames, const TArrayView<const FAnimDatabaseFrameAttribute> TransformFrameAttributes, const bool bShouldTransact = false);

	/** Set the given bone's global transform in the database at the given transform frame attribute */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase", meta = (DevelopmentOnly))
	static UE_API void DatabaseSetGlobalBoneTransform(UAnimDatabase* Database, const FName BoneName, const FAnimDatabaseFrameAttribute& TransformFrameAttribute, const bool bShouldTransact = false);

	/** Set the given bones' global transforms in the database at the given transform frame attributes */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase", meta = (DevelopmentOnly))
	static UE_API void DatabaseSetGlobalBoneTransforms(UAnimDatabase* Database, const TArray<FName>& BoneNames, const TArray<FAnimDatabaseFrameAttribute>& TransformFrameAttributes, const bool bShouldTransact = false);
	static UE_API void DatabaseSetGlobalBoneTransformsFromArrayViews(UAnimDatabase* Database, const TArrayView<const FName> BoneNames, const TArrayView<const FAnimDatabaseFrameAttribute> TransformFrameAttributes, const bool bShouldTransact = false);

	/** Set the given bone's local transforms in the database to the reference pose */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase", meta = (DevelopmentOnly))
	static UE_API void DatabaseResetBoneTransform(UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName BoneName, const bool bShouldTransact = false);

	/** Set the given bones' local transforms in the database to the reference pose */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase", meta = (DevelopmentOnly))
	static UE_API void DatabaseResetBoneTransforms(UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArray<FName>& BoneNames, const bool bShouldTransact = false);
	static UE_API void DatabaseResetBoneTransformsFromArrayViews(UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const FName> BoneNames, const bool bShouldTransact = false);

public:

	/** Strips unused bone tracks from the data to reduce the file size on disk */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase", meta = (DevelopmentOnly))
	static UE_API void DatabaseStripUnusedBoneTracks(UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const bool bShouldTransact = false);

	/** Uses the provided function to modify the database pose data */
	static UE_API void DatabaseModifyPoseData(UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TFunctionRef<void(const UE::AnimDatabase::FPoseDataView& InOutPoseData)> Function, const bool bShouldTransact = false);

	/** Makes the given ranges looped in the database */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase", meta = (DevelopmentOnly))
	static UE_API void DatabaseMakeLooped(UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const float StartEndRatio, const float BlendInTime, const float BlendOutTime, const bool bShouldTransact = false);

	/** Patches the given ranges for the provided bones with a simple interpolation */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase", meta = (DevelopmentOnly))
	static UE_API void DatabasePatchRanges(UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const bool bApplyToRoot, const TArray<int32>& BoneIndices, const bool bShouldTransact = false);

	/** Sets the given ranges to the reference pose, with the exception of the provided IgnoreBoneIndices */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase", meta = (DevelopmentOnly))
	static UE_API void DatabaseSetToReferencePose(UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const bool bApplyToRoot, const TArray<int32>& IgnoreBoneIndices, const bool bShouldTransact = false);

	/** Removes foot ground penetration in the database */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase", meta = (DevelopmentOnly))
	static UE_API void DatabaseRemoveFootGroundPenetration(UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName LeftToeBone, const FName RightToeBone, const float ToeBoneLength, const float PelvisHeightAdjustment = -1.0f, const FVector LeftKneeSideVector = FVector(0.0f, 0.0f, +1.0f), const FVector RightKneeSideVector = FVector(0.0f, 0.0f, +1.0f), const FVector LeftToeForwardVector = FVector(-1.0f, 0.0f, 0.0f), const FVector RightToeForwardVector = FVector(-1.0f, 0.0f, 0.0f), const bool bShouldTransact = false);

private:

	/** Loads into memory all the blueprint assets of a given class. */
	static void LoadAllBlueprintAssetsOfClass(UClass* Class);
};

/** Function for stripping unused bone-tracks from the AnimSequence to reduce the file size on desk  */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Strip Unused Bone Tracks Function"))
class UAnimDatabaseFunction_StripUnusedBoneTracks : public UAnimDatabaseFunction
{
	GENERATED_BODY()

public:

#if WITH_EDITOR

public:

	UE_API virtual void Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact) override;

#endif
};

/** Function for exporting the given ranges as BVH  */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Export to BVH Function"))
class UAnimDatabaseFunction_ExportBVH : public UAnimDatabaseFunction
{
	GENERATED_BODY()

public:

	/** Directory to export BVH files to. */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (RelativePath))
	FDirectoryPath ExportDirectory;

	/** If to add the appropriate root rotation to account for the axes switch. */
	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bAddRootAxesChangeRotation = true;

	/** Rotation order to use for conversion to euler angles. */
	UPROPERTY(EditAnywhere, Category = "Settings")
	EAnimDatabaseRotationOrder RotationOrder = EAnimDatabaseRotationOrder::XYZ;

#if WITH_EDITOR

public:

	UE_API virtual void Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact) override;

#endif
};

/** Function for importing from BVH  */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Import from BVH Function"))
class UAnimDatabaseFunction_ImportBVH : public UAnimDatabaseFunction
{
	GENERATED_BODY()

public:

	/** Directory to import BVH files from. */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (RelativePath))
	FDirectoryPath ImportDirectory;

	/** Directory to write AnimSequences to. */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (RelativePath))
	FDirectoryPath AssetDirectory;

	/** If to remove the appropriate root rotation to account for the axes switch. */
	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bRemoveRootAxesChangeRotation = true;

	/** Ignores BVH files with the _mirrored suffix */
	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bIgnoreMirrored = false;

	/** If to add the imported assets to the database. */
	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bAddToDatabase = true;

#if WITH_EDITOR

public:

	UE_API virtual void Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact) override;

#endif
};

/** Function for exporting all AnimNotifies and AnimNotifyStates from the given animations  */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Export Notifies Function"))
class UAnimDatabaseFunction_ExportNotifies : public UAnimDatabaseFunction
{
	GENERATED_BODY()

public:

	/** Directory to export the notifies to. */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (RelativePath))
	FDirectoryPath ExportDirectory;

	/** File to export notifies as. */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FString FileName = TEXT("Notifies.json");

#if WITH_EDITOR

public:

	UE_API virtual void Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact) override;

#endif
};

/** Function for importing AnimNotifies and AnimNotifyStates onto the given animations  */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Import Notifies Function"))
class UAnimDatabaseFunction_ImportNotifies : public UAnimDatabaseFunction
{
	GENERATED_BODY()

public:

	/** File to import Notifies from. */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (RelativePath))
	FFilePath ImportPath;

#if WITH_EDITOR

public:

	UE_API virtual void Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact) override;

#endif
};

/** Function for removing AnimNotifies and AnimNotifyStates from the given animations  */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Remove Notifies Function"))
class UAnimDatabaseFunction_RemoveNotifies : public UAnimDatabaseFunction
{
	GENERATED_BODY()

#if WITH_EDITOR

public:

	UE_API virtual void Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact) override;

#endif
};

/** Function for placing IK bones at the correct hand and foot locations  */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Update IK Bones Function"))
class UAnimDatabaseFunction_UpdateIKBones : public UAnimDatabaseFunction
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	UPROPERTY(EditAnywhere, Category = "Bones")
	FName RootBone = TEXT("root");

	UPROPERTY(EditAnywhere, Category = "Bones")
	FName FootBoneL = TEXT("foot_l");

	UPROPERTY(EditAnywhere, Category = "Bones")
	FName FootBoneR = TEXT("foot_r");

	UPROPERTY(EditAnywhere, Category = "Bones")
	FName HandBoneL = TEXT("hand_l");

	UPROPERTY(EditAnywhere, Category = "Bones")
	FName HandBoneR = TEXT("hand_r");

public:

	UPROPERTY(EditAnywhere, Category = "Bones")
	FName IKFootBoneL = TEXT("ik_foot_l");

	UPROPERTY(EditAnywhere, Category = "Bones")
	FName IKFootBoneR = TEXT("ik_foot_r");

	UPROPERTY(EditAnywhere, Category = "Bones")
	FName IKHandBoneGun = TEXT("ik_hand_gun");

	UPROPERTY(EditAnywhere, Category = "Bones")
	FName IKHandBoneL = TEXT("ik_hand_l");

	UPROPERTY(EditAnywhere, Category = "Bones")
	FName IKHandBoneR = TEXT("ik_hand_r");

#endif

#if WITH_EDITOR

public:

	UE_API virtual void Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact) override;

#endif
};

/** Function for computing the root transform using the look-at direction  */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Set Root from Look-At Function"))
class UAnimDatabaseFunction_SetRootFromLookAt : public UAnimDatabaseFunction
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	/** Root bone name */
	UPROPERTY(EditAnywhere, Category = "Bones")
	FName RootBone = TEXT("root");

	/** Pelvis bone name */
	UPROPERTY(EditAnywhere, Category = "Bones")
	FName PelvisBone = TEXT("pelvis");

	/** Bone to project onto the floor to use as the root location */
	UPROPERTY(EditAnywhere, Category = "Bones")
	FName RootLocationBone = TEXT("pelvis");

	/** Bone to use for the root facing direction */
	UPROPERTY(EditAnywhere, Category = "Bones")
	FName RootDirectionBone = TEXT("head");

	/** Local forward direction of the above bone to use as the facing direction */
	UPROPERTY(EditAnywhere, Category = "Bones")
	FVector RootDirectionBoneDirection = FVector::RightVector;

	/** Root bone forward vector */
	UPROPERTY(EditAnywhere, Category = "Bones")
	FVector RootDirection = FVector::RightVector;

public:

	/** If to apply smoothing to the root */
	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bApplySmoothing = true;

	/** If to use a SavGol filter for smoothing. A SavGol filter can work better when applying large amounts of smoothing */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (EditCondition = "bApplySmoothing", HideEditConditionToggle))
	bool bUseSavGolFilter = true;

	/** Filter width for the location savgol filter. Larger values create smoother trajectories. */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ForceUnits = "s", ClampMin = "1", UIMin = "1", EditCondition = "bApplySmoothing && bUseSavGolFilter", HideEditConditionToggle))
	float LocationSavGolFilterWidth = 1.5;

	/** Degree of polynomial to use for the location savgol filter. Larger degrees allow for more bendy trajectories. */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ClampMin = "1", ClampMax = "9", UIMin = "1", UIMax = "9", EditCondition = "bApplySmoothing && bUseSavGolFilter", HideEditConditionToggle))
	int32 LocationSavGolPolynomialDegree = 3;

	/** Filter width for the direction savgol filter. Larger values create smoother trajectories. */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ForceUnits = "s", ClampMin = "1", UIMin = "1", EditCondition = "bApplySmoothing && bUseSavGolFilter", HideEditConditionToggle))
	float DirectionSavGolFilterWidth = 1.5f;

	/** Degree of polynomial to use for the direction savgol filter. Larger degrees allow for more bendy trajectories. */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ClampMin = "1", ClampMax = "9", UIMin = "1", UIMax = "9", EditCondition = "bApplySmoothing && bUseSavGolFilter", HideEditConditionToggle))
	int32 DirectionSavGolPolynomialDegree = 3;

	/** If to use a gaussian window on the SavGol filter weights */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (EditCondition = "bApplySmoothing && bUseSavGolFilter", HideEditConditionToggle))
	bool bUseGaussianWindowedSavGolFilter = true;

	/** Amount of smoothing to apply to the projected bone location */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ForceUnits = "s", EditCondition = "bApplySmoothing && !bUseSavGolFilter", HideEditConditionToggle))
	float LocationSmoothingTime = 0.15f;

	/** Amount of smoothing to apply to the facing direction */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ForceUnits = "s", EditCondition = "bApplySmoothing && !bUseSavGolFilter", HideEditConditionToggle))
	float DirectionSmoothingTime = 0.25f;

#endif

#if WITH_EDITOR

public:

	UE_API virtual void Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact) override;

#endif
};

/** Function for computing the root transform using a fixed offset from the pelvis  */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Set Root from Pelvis"))
class UAnimDatabaseFunction_SetRootFromPelvis : public UAnimDatabaseFunction
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	/** Root bone name */
	UPROPERTY(EditAnywhere, Category = "Bones")
	FName RootBone = TEXT("root");

	/** Pelvis bone name */
	UPROPERTY(EditAnywhere, Category = "Bones")
	FName PelvisBone = TEXT("pelvis");

	/** Fixed Offset to apply to the pelvis */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FTransform Offset = FTransform(FRotator(-90.0f, 0.0f, 0.0f), FVector(-100.0f, 0.0f, 0.0f), FVector::OneVector);

	/** If to apply the offset in the local space of the pelvis or the world space */
	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bApplyInLocalSpace = true;

#endif

#if WITH_EDITOR

public:

	UE_API virtual void Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact) override;

#endif
};

/** Function for baking root motion into the pelvis  */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Bake Root Motion into Pelvis Function"))
class UAnimDatabaseFunction_BakeRootMotionIntoPelvis : public UAnimDatabaseFunction
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	UPROPERTY(EditAnywhere, Category = "Bones")
	FName RootBone = TEXT("root");

	UPROPERTY(EditAnywhere, Category = "Bones")
	FName PelvisBone = TEXT("pelvis");

#endif

#if WITH_EDITOR

public:

	UE_API virtual void Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact) override;

#endif
};

/** Function for making the root motion start at the origin for all the provided ranges  */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Make Root Start at Origin Function"))
class UAnimDatabaseFunction_MakeRootStartAtOrigin : public UAnimDatabaseFunction
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	UPROPERTY(EditAnywhere, Category = "Bones")
	FName RootBone = TEXT("root");

#endif

#if WITH_EDITOR

public:

	UE_API virtual void Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact) override;

#endif
};

/** Function for exporting ranges to AnimSequences  */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Export to Anim Sequences Function"))
class UAnimDatabaseFunction_ExportToAnimSequences : public UAnimDatabaseFunction
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	UPROPERTY(EditAnywhere, Category = "Settings")
	FDirectoryPath ExportDirectory;

	UPROPERTY(EditAnywhere, Category = "Settings")
	FString AssetNameFormatString = TEXT("{SequenceName}_{StartFrame}_{EndFrame}");

	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bExportMirrored = false;

#endif

#if WITH_EDITOR

public:

	UE_API virtual void Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact) override;

#endif
};

/** Function for exporting root motion from the given animations  */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Export Root Motion Function"))
class UAnimDatabaseFunction_ExportRootMotion : public UAnimDatabaseFunction
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	/** Directory to export the root motion to. */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (RelativePath))
	FDirectoryPath ExportDirectory;

	/** File to export root motion as. */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FString FileName = TEXT("RootMotion.json");

#endif

#if WITH_EDITOR

public:

	UE_API virtual void Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact) override;

#endif
};

/** Function for making ranges looping  */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Make Looped Function"))
class UAnimDatabaseFunction_MakeLooped : public UAnimDatabaseFunction
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float StartEndRatio = 0.5f;

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ForceUnits = "s"))
	float BlendInTime = 1.0f;

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ForceUnits = "s"))
	float BlendOutTime = 1.0f;

#endif

#if WITH_EDITOR

public:

	UE_API virtual void Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact) override;

#endif
};

/** Function for patching ranges  */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Patch Bones Function"))
class UAnimDatabaseFunction_PatchBones : public UAnimDatabaseFunction
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bApplyToRoot = false;

	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bEntirePose = false;

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (EditCondition = "!bEntirePose", HideEditConditionToggle))
	TArray<FName> BoneNames;

#endif

#if WITH_EDITOR

public:

	UE_API virtual void Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact) override;

#endif
};

/** Function for setting to the reference pose  */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Set to Reference Pose Function"))
class UAnimDatabaseFunction_SetToReferencePose : public UAnimDatabaseFunction
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bApplyToRoot = false;

	UPROPERTY(EditAnywhere, Category = "Settings")
	TArray<FName> IgnoreBoneNames;

#endif

#if WITH_EDITOR

public:

	UE_API virtual void Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact) override;

#endif
};

/** Function for applying a transform to a bone  */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Bone Apply Transform Function"))
class UAnimDatabaseFunction_BoneApplyTransform : public UAnimDatabaseFunction
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	UPROPERTY(EditAnywhere, Category = "Settings")
	FName BoneName = NAME_None;

	UPROPERTY(EditAnywhere, Category = "Settings")
	EAnimDatabaseTransformMode Mode = EAnimDatabaseTransformMode::Replace;

	UPROPERTY(EditAnywhere, Category = "Settings")
	FTransform Transform = FTransform::Identity;

#endif

#if WITH_EDITOR

public:

	UE_API virtual void Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact) override;

#endif
};

/** Function for adding AnimNotifyStates from a reference object  */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Add Anim Notify State Function"))
class UAnimDatabaseFunction_AddAnimNotifyState : public UAnimDatabaseFunction
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	UPROPERTY(EditAnywhere, Category = "Settings")
	FName TrackName = NAME_None;

	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TObjectPtr<UAnimNotifyState> AnimNotifyState;

	UPROPERTY(EditAnywhere, Category = "Settings")
	FLinearColor Color = FLinearColor::Red;

#endif

#if WITH_EDITOR

public:

	UE_API virtual void Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact) override;

#endif
};

/** Function that can execute a sequence of sub-functions  */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Sequence Function"))
class UAnimDatabaseFunction_Sequence : public UAnimDatabaseFunction
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TArray<TObjectPtr<UAnimDatabaseFunction>> Functions;

#endif

#if WITH_EDITOR

public:

	UE_API virtual void Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact) override;

#endif
};

/** Function for adding an AnimNotify from a frame function  */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Add Anim Notify at Frames Function"))
class UAnimDatabaseFunction_AddAnimNotifyAtFrames : public UAnimDatabaseFunction
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	UPROPERTY(EditAnywhere, Category = "Settings")
	FName TrackName = NAME_None;

	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TObjectPtr<UAnimNotify> AnimNotify;

	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TObjectPtr<UAnimDatabaseFramesFunction> Frames;

	UPROPERTY(EditAnywhere, Category = "Settings")
	FLinearColor Color = FLinearColor::Red;

#endif

#if WITH_EDITOR

public:

	UE_API virtual void Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact) override;

#endif
};

/** Function for adding an AnimNotifyState from a frame ranges function  */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Add Anim Notify State at Frame Ranges Function"))
class UAnimDatabaseFunction_AddAnimNotifyStateAtFrameRanges : public UAnimDatabaseFunction
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	UPROPERTY(EditAnywhere, Category = "Settings")
	FName TrackName = NAME_None;

	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TObjectPtr<UAnimNotifyState> AnimNotifyState;

	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TObjectPtr<UAnimDatabaseFrameRangesFunction> FrameRanges;

	UPROPERTY(EditAnywhere, Category = "Settings")
	FLinearColor Color = FLinearColor::Red;

#endif

#if WITH_EDITOR

public:

	UE_API virtual void Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact) override;

#endif
};

/** Function for adding a curve from a frame attribute function  */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Add Curve from Frame Attribute Function"))
class UAnimDatabaseFunction_AddCurveFromFrameAttribute : public UAnimDatabaseFunction
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	UPROPERTY(EditAnywhere, Category = "Settings")
	FName CurveName = NAME_None;

	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TObjectPtr<UAnimDatabaseFrameAttributeFunction> FrameAttribute;

	UPROPERTY(EditAnywhere, Category = "Settings")
	FLinearColor Color = FLinearColor::Red;

	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bSparseKeys = true;

#endif

#if WITH_EDITOR

public:

	UE_API virtual void Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact) override;

#endif
};

/** Function for processing contacts */
UCLASS(BlueprintType, Blueprintable, DontCollapseCategories, meta = (DisplayName = "Process Contacts Function"))
class UAnimDatabaseFunction_ProcessContacts : public UAnimDatabaseFunction
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	/** If to add foot speed curves */
	UPROPERTY(EditAnywhere, Category = "Foot Speed Curves")
	bool bAddFootSpeedCurves = false;

	/** Name of the foot speed curve for the left foot */
	UPROPERTY(EditAnywhere, Category = "Foot Speed Curves", meta = (EditCondition = "bAddFootSpeedCurves", HideEditConditionToggle))
	FName LeftFootSpeedCurveName = TEXT("footspeed_l");

	/** Name of the foot speed curve for the right foot */
	UPROPERTY(EditAnywhere, Category = "Foot Speed Curves", meta = (EditCondition = "bAddFootSpeedCurves", HideEditConditionToggle))
	FName RightFootSpeedCurveName = TEXT("footspeed_r");

	/** Name of the left bone to generate the foot speed curve for */
	UPROPERTY(EditAnywhere, Category = "Foot Speed Curves", meta = (EditCondition = "bAddFootSpeedCurves", HideEditConditionToggle))
	FName LeftFootSpeedBoneName = TEXT("ball_l");

	/** Name of the right bone to generate the foot speed curve for */
	UPROPERTY(EditAnywhere, Category = "Foot Speed Curves", meta = (EditCondition = "bAddFootSpeedCurves", HideEditConditionToggle))
	FName RightFootSpeedBoneName = TEXT("ball_r");

	/** Color of the left foot speed curve */
	UPROPERTY(EditAnywhere, Category = "Foot Speed Curves", meta = (EditCondition = "bAddFootSpeedCurves", HideEditConditionToggle))
	FLinearColor LeftFootSpeedCurveColor = FLinearColor::Red;

	/** Color of the right foot speed curve */
	UPROPERTY(EditAnywhere, Category = "Foot Speed Curves", meta = (EditCondition = "bAddFootSpeedCurves", HideEditConditionToggle))
	FLinearColor RightFootSpeedCurveColor = FLinearColor::Blue;

public:

	/** If to add contact curves */
	UPROPERTY(EditAnywhere, Category = "Contact Curves")
	bool bAddContactCurves = false;

	/** Name of the contact curve for the left foot */
	UPROPERTY(EditAnywhere, Category = "Contact Curves", meta = (EditCondition = "bAddContactCurves", HideEditConditionToggle))
	FName LeftContactCurveName = TEXT("contact_l");

	/** Name of the contact curve for the right foot */
	UPROPERTY(EditAnywhere, Category = "Contact Curves", meta = (EditCondition = "bAddContactCurves", HideEditConditionToggle))
	FName RightContactCurveName = TEXT("contact_r");

	/** Name of the left bone to generate the contact curve for */
	UPROPERTY(EditAnywhere, Category = "Contact Curves", meta = (EditCondition = "bAddContactCurves", HideEditConditionToggle))
	FName LeftContactBoneName = TEXT("ball_l");

	/** Name of the right bone to generate the contact curve for */
	UPROPERTY(EditAnywhere, Category = "Contact Curves", meta = (EditCondition = "bAddContactCurves", HideEditConditionToggle))
	FName RightContactBoneName = TEXT("ball_r");

	/** Color of the left contact curve */
	UPROPERTY(EditAnywhere, Category = "Contact Curves", meta = (EditCondition = "bAddContactCurves", HideEditConditionToggle))
	FLinearColor LeftContactCurveColor = FLinearColor::Red;

	/** Color of the right contact curve */
	UPROPERTY(EditAnywhere, Category = "Contact Curves", meta = (EditCondition = "bAddContactCurves", HideEditConditionToggle))
	FLinearColor RightContactCurveColor = FLinearColor::Blue;

public:

	/** If to add anim notifies */
	UPROPERTY(EditAnywhere, Category = "Anim Notifies")
	bool bAddAnimNotifies = false;

	/** Clears all Anim Notifies on the given tracks before adding new ones */
	UPROPERTY(EditAnywhere, Category = "Anim Notifies", meta = (EditCondition = "bAddAnimNotifies", HideEditConditionToggle))
	bool bClearAnimNotifyTracks = true;

	/** The track name for the left anim notifies */
	UPROPERTY(EditAnywhere, Category = "Anim Notifies", meta = (EditCondition = "bAddAnimNotifies", HideEditConditionToggle))
	FName LeftContactAnimNotifyTrackName = TEXT("Footstep Left");

	/** The track name for the right anim notifies */
	UPROPERTY(EditAnywhere, Category = "Anim Notifies", meta = (EditCondition = "bAddAnimNotifies", HideEditConditionToggle))
	FName RightContactAnimNotifyTrackName = TEXT("Footstep Right");

	/** The object to use for the left contact anim notify */
	UPROPERTY(EditAnywhere, Instanced, Category = "Anim Notifies", meta = (EditCondition = "bAddAnimNotifies", HideEditConditionToggle))
	TObjectPtr<UAnimNotify> LeftContactAnimNotify;

	/** The object to use for the right contact anim notify */
	UPROPERTY(EditAnywhere, Instanced, Category = "Anim Notifies", meta = (EditCondition = "bAddAnimNotifies", HideEditConditionToggle))
	TObjectPtr<UAnimNotify> RightContactAnimNotify;

	/** Color of the left anim notify track */
	UPROPERTY(EditAnywhere, Category = "Anim Notifies", meta = (EditCondition = "bAddAnimNotifies", HideEditConditionToggle))
	FLinearColor LeftContactAnimNotifyColor = FLinearColor::Red;

	/** Color of the right anim notify track */
	UPROPERTY(EditAnywhere, Category = "Anim Notifies", meta = (EditCondition = "bAddAnimNotifies", HideEditConditionToggle))
	FLinearColor RightContactAnimNotifyColor = FLinearColor::Blue;

	/** The amount of time before the contact (in seconds) to place the location of the Anim Notifies */
	UPROPERTY(EditAnywhere, Category = "Anim Notifies", meta = (EditCondition = "bAddAnimNotifies", HideEditConditionToggle, ForceUnits = "s"))
	float AnimNotifyOffset = 0.0f;

public:

	/** If to add sync markers */
	UPROPERTY(EditAnywhere, Category = "Sync Markers")
	bool bAddSyncMarkers = false;

	/** Clears all Sync Markers on the given tracks before adding new ones */
	UPROPERTY(EditAnywhere, Category = "Sync Markers", meta = (EditCondition = "bAddSyncMarkers", HideEditConditionToggle))
	bool bClearSyncMarkerTracks = true;

	/** The track name for the left sync markers */
	UPROPERTY(EditAnywhere, Category = "Sync Markers", meta = (EditCondition = "bAddSyncMarkers", HideEditConditionToggle))
	FName LeftContactSyncMarkerTrackName = TEXT("FootSyncMarkers");

	/** The track name for the right sync markers */
	UPROPERTY(EditAnywhere, Category = "Sync Markers", meta = (EditCondition = "bAddSyncMarkers", HideEditConditionToggle))
	FName RightContactSyncMarkerTrackName = TEXT("FootSyncMarkers");

	/** The name of the left sync marker */
	UPROPERTY(EditAnywhere, Category = "Sync Markers", meta = (EditCondition = "bAddSyncMarkers", HideEditConditionToggle))
	FName LeftContactSyncMarker = TEXT("L");

	/** The name of the right sync marker */
	UPROPERTY(EditAnywhere, Category = "Sync Markers", meta = (EditCondition = "bAddSyncMarkers", HideEditConditionToggle))
	FName RightContactSyncMarker = TEXT("R");

	/** Color of the left sync marker track */
	UPROPERTY(EditAnywhere, Category = "Sync Markers", meta = (EditCondition = "bAddSyncMarkers", HideEditConditionToggle))
	FLinearColor LeftContactSyncMarkerColor = FLinearColor(0.5f, 0.0, 0.5f, 1.0f);

	/** Color of the right sync marker track */
	UPROPERTY(EditAnywhere, Category = "Sync Markers", meta = (EditCondition = "bAddSyncMarkers", HideEditConditionToggle))
	FLinearColor RightContactSyncMarkerColor = FLinearColor(0.5f, 0.0, 0.5f, 1.0f);

	/** The amount of time before the contact (in seconds) to place the location of the sync markers */
	UPROPERTY(EditAnywhere, Category = "Sync Markers", meta = (EditCondition = "bAddSyncMarkers", HideEditConditionToggle, ForceUnits = "s"))
	float SyncMarkerOffset = 0.0f;

public:

	/** Maximum height for which the bone can be considered in contact */
	UPROPERTY(EditAnywhere, Category = "Filtering", meta = (EditCondition = "bAddContactCurves || bAddAnimNotifies || bAddSyncMarkers", HideEditConditionToggle, ForceUnits = "cm"))
	float HeightThreshold = 25.0f;

	/** Maximum velocity for which the bone can be considered in contact */
	UPROPERTY(EditAnywhere, Category = "Filtering", meta = (EditCondition = "bAddContactCurves || bAddAnimNotifies || bAddSyncMarkers", HideEditConditionToggle, ForceUnits = "cm/s"))
	float VelocityThreshold = 50.0f;

	/** If to filter the contact curves */
	UPROPERTY(EditAnywhere, Category = "Filtering", meta = (EditCondition = "bAddContactCurves || bAddAnimNotifies || bAddSyncMarkers", HideEditConditionToggle))
	bool bFilterContactCurves = true;

	/** Roughly corresponds to the minimum contact duration (in seconds) that will be allowed. Reduce if very brief contacts are being missed. Increase if you are getting lots of very short contacts or lifts. */
	UPROPERTY(EditAnywhere, Category = "Filtering", meta = (EditCondition = "(bAddContactCurves || bAddAnimNotifies || bAddSyncMarkers) && bFilterContactCurves", HideEditConditionToggle, ForceUnits = "s", UIMin="0.0", ClampMin="0.0"))
	float ContactFilterTime = 0.025f;

	/** If there is a contact event on the first frame then it is removed */
	UPROPERTY(EditAnywhere, Category = "Filtering", meta = (EditCondition = "bAddAnimNotifies || bAddSyncMarkers", HideEditConditionToggle))
	bool bRemoveFirstFrameContactEvent = true;

public:

	/** If to smooth the contact curves */
	UPROPERTY(EditAnywhere, Category = "Smoothing", meta = (EditCondition = "bAddContactCurves", HideEditConditionToggle))
	bool bSmoothContactCurves = true;

	/** Amount of smoothing to apply to the resulting contact curve (in frames) */
	UPROPERTY(EditAnywhere, Category = "Smoothing", meta = (EditCondition = "bAddContactCurves && bSmoothContactCurves", HideEditConditionToggle))
	float SmoothingAmount = 3.0f;

	/** If to write curves using sparse keys */
	UPROPERTY(EditAnywhere, Category = "Smoothing", meta = (EditCondition = "bAddFootSpeedCurves || bAddContactCurves", HideEditConditionToggle))
	bool bSparseKeys = true;

#endif

#if WITH_EDITOR

public:

	UE_API virtual void Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact) override;

#endif
};


/** Function for generating phase curves from another source */
UCLASS(BlueprintType, Blueprintable, DontCollapseCategories, meta = (DisplayName = "Process Phase Curves Function"))
class UAnimDatabaseFunction_ProcessPhaseCurves : public UAnimDatabaseFunction
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	/** The frames used to label a phase time of zero */
	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TObjectPtr<UAnimDatabaseFramesFunction> ZeroPhaseFrames;

	/** The frames used to label that half the phase time has elapsed */
	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TObjectPtr<UAnimDatabaseFramesFunction> HalfPhaseFrames;

	/** Phase extrapolation mode for the ends of the animation */
	UPROPERTY(EditAnywhere, Category = "Settings")
	EAnimDatabasePhaseExtrapolationMode ExtrapolationMode = EAnimDatabasePhaseExtrapolationMode::Extrapolate;

	/** If to write curves using sparse keys */
	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bSparseKeys = true;

public:

	/** If to add the phase angle as a curve */
	UPROPERTY(EditAnywhere, Category = "Phase Angle")
	bool bAddPhaseAngle = false;

	/** Name of the angular phase curve */
	UPROPERTY(EditAnywhere, Category = "Phase Angle", meta = (EditCondition = "bAddPhaseAngle", HideEditConditionToggle))
	FName PhaseAngleCurveName = TEXT("phase_angle");

	/** Color of the angular phase curve */
	UPROPERTY(EditAnywhere, Category = "Phase Angle", meta = (EditCondition = "bAddPhaseAngle", HideEditConditionToggle))
	FLinearColor PhaseAngleCurveColor = FLinearColor(0.5f, 0.25, 0.5f, 1.0f);

	/** If to re-scale the phase angle to be between 0 and 1 */
	UPROPERTY(EditAnywhere, Category = "Phase Angle", meta = (EditCondition = "bAddPhaseAngle", HideEditConditionToggle))
	bool bRescalePhaseAngle = false;

public:

	/** If to add two phase direction as two curves */
	UPROPERTY(EditAnywhere, Category = "Phase Direction")
	bool bAddPhaseDirection = false;

	/** Name of the x phase curve */
	UPROPERTY(EditAnywhere, Category = "Phase Direction", meta = (EditCondition = "bAddPhaseDirection", HideEditConditionToggle))
	FName PhaseDirectionXCurveName = TEXT("phase_x");

	/** Name of the y phase curve */
	UPROPERTY(EditAnywhere, Category = "Phase Direction", meta = (EditCondition = "bAddPhaseDirection", HideEditConditionToggle))
	FName PhaseDirectionYCurveName = TEXT("phase_y");

	/** Color of the x phase curve */
	UPROPERTY(EditAnywhere, Category = "Phase Direction", meta = (EditCondition = "bAddPhaseDirection", HideEditConditionToggle))
	FLinearColor PhaseDirectionXCurveColor = FLinearColor(0.5f, 0.5f, 0.25, 1.0f);

	/** Color of the y phase curve */
	UPROPERTY(EditAnywhere, Category = "Phase Direction", meta = (EditCondition = "bAddPhaseDirection", HideEditConditionToggle))
	FLinearColor PhaseDirectionYCurveColor = FLinearColor(0.5f, 0.5f, 0.75, 1.0f);

public:

	/** If to add the phase angle as a ramp that goes up and down between -1 and 1 */
	UPROPERTY(EditAnywhere, Category = "Phase Ramp")
	bool bAddPhaseRamp = false;

	/** Name of the phase ramp curve */
	UPROPERTY(EditAnywhere, Category = "Phase Ramp", meta = (EditCondition = "bAddPhaseRamp", HideEditConditionToggle))
	FName PhaseRampCurveName = TEXT("phase");

	/** Color of the phase ramp curve */
	UPROPERTY(EditAnywhere, Category = "Phase Ramp", meta = (EditCondition = "bAddPhaseRamp", HideEditConditionToggle))
	FLinearColor PhaseRampCurveColor = FLinearColor(0.5f, 0.75, 0.5f, 1.0f);
	
#endif

#if WITH_EDITOR

public:

	UE_API virtual void Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact) override;

#endif
};


/** Function for setting a bone local transform from frame attribute function  */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Set Bone Local Transform from Frame Attribute Function"))
class UAnimDatabaseFunction_SetBoneLocalTransformFromFrameAttribute : public UAnimDatabaseFunction
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	UPROPERTY(EditAnywhere, Category = "Settings")
	FName BoneName = NAME_None;

	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TObjectPtr<UAnimDatabaseFrameAttributeFunction> FrameAttribute;

#endif

#if WITH_EDITOR

public:

	UE_API virtual void Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact) override;

#endif
};

/** Function for setting a bone global transform from frame attribute function  */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Set Bone Global Transform from Frame Attribute Function"))
class UAnimDatabaseFunction_SetBoneGlobalTransformFromFrameAttribute : public UAnimDatabaseFunction
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	UPROPERTY(EditAnywhere, Category = "Settings")
	FName BoneName = NAME_None;

	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TObjectPtr<UAnimDatabaseFrameAttributeFunction> FrameAttribute;

#endif

#if WITH_EDITOR

public:

	UE_API virtual void Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact) override;

#endif
};

UENUM(BlueprintType)
enum class EAnimDatabaseBoneSpace : uint8
{
	// Local (parent) space 
	Local,
	// Global (world) space
	Global
};

/** Copy bone transforms from certain bones to others  */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Copy Bones Function"))
class UAnimDatabaseFunction_CopyBones : public UAnimDatabaseFunction
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	/** Bone Mapping */
	UPROPERTY(EditAnywhere, Category = "Settings")
	TMap<FName, FName> BoneMap;

	/** Space to perform copying in */
	UPROPERTY(EditAnywhere, Category = "Settings")
	EAnimDatabaseBoneSpace Space = EAnimDatabaseBoneSpace::Global;

#endif

#if WITH_EDITOR

public:

	UE_API virtual void Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact) override;

#endif
};

/** Reset bone transforms to the reference pose  */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Reset Bones Function"))
class UAnimDatabaseFunction_ResetBones : public UAnimDatabaseFunction
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	UPROPERTY(EditAnywhere, Category = "Settings")
	TArray<FName> Bones;

#endif

#if WITH_EDITOR

public:

	UE_API virtual void Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact) override;

#endif
};

/** Function to set bone to root location at frame  */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Set Bone to Root Transform at Frames Function"))
class UAnimDatabaseFunction_SetBoneToRootTransformAtFrames : public UAnimDatabaseFunction
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	UPROPERTY(EditAnywhere, Category = "Settings")
	FName BoneName = TEXT("attach");

	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TObjectPtr<UAnimDatabaseFramesFunction> Frames;

#endif

#if WITH_EDITOR

public:

	UE_API virtual void Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact) override;

#endif
};

/** Function to adjust neck rotation to remove potential artifacts after retargeting */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Adjust Neck Rotation Function"))
class UAnimDatabaseFunction_AdjustNeckRotation : public UAnimDatabaseFunction
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	/** Name of the neck0 bone */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FName Neck0Bone = TEXT("neck_01");

	/** Name of the neck1 bone */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FName Neck1Bone = TEXT("neck_02");

	/** Name of the head bone */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FName HeadBone = TEXT("head");

	/** Rotation to apply to the neck0 bone */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ForceUnits = "deg"))
	float Neck0Rotation = -18.0f;

	/** Rotation to apply to the neck1 bone */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ForceUnits = "deg"))
	float Neck1Rotation = 0.0f;

	/** Rotation to apply to the head bone */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ForceUnits = "deg"))
	float HeadRotation = 18.0f;

	/** Local axis to apply rotation on */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FVector LocalAxis = FVector(0.0, 0.0, 1.0);

#endif

#if WITH_EDITOR

public:

	UE_API virtual void Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact) override;

#endif
};

/** Function to remove foot ground penetration */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Remove Foot Ground Penetration Function"))
class UAnimDatabaseFunction_RemoveFootGroundPenetration : public UAnimDatabaseFunction
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	/** Name of the left toe bone */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FName LeftToeBone = TEXT("ball_l");

	/** Name of the right toe bone */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FName RightToeBone = TEXT("ball_r");

	/** Length of the toe joint */
	UPROPERTY(EditAnywhere, Category = "Settings", meta=(ForceUnits="cm"))
	float ToeBoneLength = 7.0f;

	/** Side Vector of the Left Knee */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FVector LeftKneeSideVector = FVector(0.0f, 0.0f, +1.0f);

	/** Side Vector of the Right Knee */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FVector RightKneeSideVector = FVector(0.0f, 0.0f, +1.0f);

	/** Forward Vector of the Left Toe */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FVector LeftToeForwardVector = FVector(-1.0f, 0.0f, 0.0f);

	/** Forward Vector of the Right Toe */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FVector RightToeForwardVector = FVector(-1.0f, 0.0f, 0.0f);

	/** Adjustment to apply to the pelvis height */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ForceUnits = "cm"))
	float PelvisHeightAdjustment = 0.0f;

#endif

#if WITH_EDITOR

public:

	UE_API virtual void Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact) override;

#endif
};

/** Computes various statistics about the provided ranges */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Statistics Function"))
class UAnimDatabaseFunction_Statistics : public UAnimDatabaseFunction
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	/** Statistics for the given ranges */
	UPROPERTY(VisibleAnywhere, Category = "Settings")
	FAnimDatabaseFrameRangesStatistics Statistics;

#endif

#if WITH_EDITOR

public:

	UE_API virtual void Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact) override;

#endif
};

/** Computes various stats about root speeds for the provide frame ranges */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Speed Statistics Function"))
class UAnimDatabaseFunction_SpeedStatistics : public UAnimDatabaseFunction
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	/** Frame ranges over which to compute the statistics */
	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TObjectPtr<UAnimDatabaseFrameRangesFunction> FrameRanges;

public:

	/** Average linear velocity across the frame ranges */
	UPROPERTY(VisibleAnywhere, Category = "Settings", meta = (ForceUnits = "cm/s"))
	float AverageLinearVelocity = 0.0f;

	/** Smallest linear velocity across the frame ranges */
	UPROPERTY(VisibleAnywhere, Category = "Settings", meta = (ForceUnits = "cm/s"))
	float MinimumLinearVelocity = 0.0f;

	/** Largest linear velocity across the frame ranges */
	UPROPERTY(VisibleAnywhere, Category = "Settings", meta = (ForceUnits = "cm/s"))
	float MaximumLinearVelocity = 0.0f;

	/** Average angular velocity across the frame ranges */
	UPROPERTY(VisibleAnywhere, Category = "Settings", meta = (ForceUnits = "deg/s"))
	float AverageAngularVelocity = 0.0f;

	/** Smallest angular velocity across the frame ranges */
	UPROPERTY(VisibleAnywhere, Category = "Settings", meta = (ForceUnits = "deg/s"))
	float MinimumAngularVelocity = 0.0f;

	/** Largest angular velocity across the frame ranges */
	UPROPERTY(VisibleAnywhere, Category = "Settings", meta = (ForceUnits = "deg/s"))
	float MaximumAngularVelocity = 0.0f;

#endif

#if WITH_EDITOR

public:

	UE_API virtual void Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact) override;

#endif
};

/** Computes various statistics regarding transitions given by the provided FrameRanges  */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Transition Statistics Function"))
class UAnimDatabaseFunction_TransitionStatistics : public UAnimDatabaseFunction
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	/** Frame ranges over which to compute the statistics */
	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TObjectPtr<UAnimDatabaseFrameRangesFunction> FrameRanges;

public:

	/** Average duration of the given frame ranges */
	UPROPERTY(VisibleAnywhere, Category = "Settings", meta = (ForceUnits = "s"))
	float AverageTime = 0.0f;

	/** Shortest duration of the given frame ranges */
	UPROPERTY(VisibleAnywhere, Category = "Settings", meta = (ForceUnits = "s"))
	float MinimumTime = 0.0f;

	/** Longest duration of the given frame ranges */
	UPROPERTY(VisibleAnywhere, Category = "Settings", meta = (ForceUnits = "s"))
	float MaximumTime = 0.0f;

	/** Average distance traveled during the given frame ranges */
	UPROPERTY(VisibleAnywhere, Category = "Settings", meta = (ForceUnits = "cm"))
	float AverageDistance = 0.0f;

	/** Minimum distance traveled during the given frame ranges */
	UPROPERTY(VisibleAnywhere, Category = "Settings", meta = (ForceUnits = "cm"))
	float MinimumDistance = 0.0f;

	/** Maximum distance traveled during the given frame ranges */
	UPROPERTY(VisibleAnywhere, Category = "Settings", meta = (ForceUnits = "cm"))
	float MaximumDistance = 0.0f;

	/** Average angle traveled during the given frame ranges */
	UPROPERTY(VisibleAnywhere, Category = "Settings", meta = (ForceUnits = "deg"))
	float AverageAngle = 0.0f;

	/** Minimum angle traveled during the given frame ranges */
	UPROPERTY(VisibleAnywhere, Category = "Settings", meta = (ForceUnits = "deg"))
	float MinimumAngle = 0.0f;

	/** Maximum angle traveled during the given frame ranges */
	UPROPERTY(VisibleAnywhere, Category = "Settings", meta = (ForceUnits = "deg"))
	float MaximumAngle = 0.0f;

#endif

#if WITH_EDITOR

public:

	UE_API virtual void Run_Implementation(UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges, const bool bShouldTransact) override;

#endif
};

//--------------------------------------------------

struct FDebugDrawer;

/** Class that can be implemented to perform custom debug draw functionality */
UCLASS(Abstract, HideDropdown, EditInlineNew, DefaultToInstanced, CollapseCategories, BlueprintType, Blueprintable)
class UAnimDatabaseDebugDraw : public UObject
{
	GENERATED_BODY()

#if WITH_EDITOR

public:

	/** Make any required frame attributes or other data. This function is called once when the class is initialized */
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "AnimDatabase", meta = (ForceAsFunction))
	UE_API void InitializeDrawDebug(const UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges);

public:

	/** Draw Debug function which is called each frame with the current pose state, database, and attribute information */
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "AnimDatabase", meta = (ForceAsFunction))
	UE_API void DrawDebug(
		const FDebugDrawer& Drawer,
		const FDebugDrawer& CanvasDrawer,
		const FAnimDatabasePoseState& InPoseState,
		const UAnimDatabase* InDatabase,
		const FAnimDatabaseFrameRanges& InFrameRanges,
		const FTransform& RangeViewportTransform,
		const int32 CharacterIdx,
		const int32 SequenceIdx,
		const float SequenceTime,
		const int32 RangeStart, 
		const int32 RangeLength,
		const FLinearColor& IdentifierColor);

#endif
};

/** Debug Draw class for drawing attributes */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Attributes"))
class UAnimDatabaseDebugDraw_Attributes : public UAnimDatabaseDebugDraw
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	/** Spacing for text lines vertically */
	UPROPERTY(EditAnywhere, Category = "Settings")
	float VerticalSpacing = 16.0f;

	/** Pixel offset to display the properties */
	UPROPERTY(EditAnywhere, Category = "Settings")
	float PropertyHorizontalOffset = 250.0f;

#endif

#if WITH_EDITOR

public:

	virtual void DrawDebug_Implementation(
		const FDebugDrawer& Drawer,
		const FDebugDrawer& CanvasDrawer,
		const FAnimDatabasePoseState& InPoseState,
		const UAnimDatabase* InDatabase,
		const FAnimDatabaseFrameRanges& InFrameRanges,
		const FTransform& RangeViewportTransform,
		const int32 CharacterIdx,
		const int32 SequenceIdx,
		const float SequenceTime,
		const int32 RangeStart,
		const int32 RangeLength,
		const FLinearColor& IdentifierColor) override;

#endif
};

/** Debug Draw class for drawing contact curves */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Contact Curves Debug Draw"))
class UAnimDatabaseDebugDraw_ContactCurves : public UAnimDatabaseDebugDraw
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	/** Name of the left toe bone */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FName LeftToeBoneName = TEXT("ball_l");

	/** Name of the right toe bone */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FName RightToeBoneName = TEXT("ball_r");

	/** Name of the left contact curve */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FName LeftContactCurveName = TEXT("contact_l");

	/** Name of the right contact curve */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FName RightContactCurveName = TEXT("contact_r");

	/** Radius for the circle to display to show contacts */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ForceUnits = "cm"))
	float ContactCircleRadius = 15.0f;

protected:

	/** Left contact curve attribute */
	FAnimDatabaseFrameAttribute LeftContactCurve;

	/** Right contact curve attribute */
	FAnimDatabaseFrameAttribute RightContactCurve;

#endif

#if WITH_EDITOR

public:

	virtual void InitializeDrawDebug_Implementation(const UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges) override;

	virtual void DrawDebug_Implementation(
		const FDebugDrawer& Drawer,
		const FDebugDrawer& CanvasDrawer,
		const FAnimDatabasePoseState& InPoseState,
		const UAnimDatabase* InDatabase,
		const FAnimDatabaseFrameRanges& InFrameRanges,
		const FTransform& RangeViewportTransform,
		const int32 CharacterIdx,
		const int32 SequenceIdx,
		const float SequenceTime,
		const int32 RangeStart,
		const int32 RangeLength,
		const FLinearColor& IdentifierColor) override;

#endif
};


/** Debug Draw class for working out correct contact thresholds */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Contact Thresholds Debug Draw"))
class UAnimDatabaseDebugDraw_ContactThresholds : public UAnimDatabaseDebugDraw
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	/** Name of the left toe bone */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FName LeftToeBoneName = TEXT("ball_l");

	/** Name of the right toe bone */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FName RightToeBoneName = TEXT("ball_r");

	/** Velocity Threshold for contacts */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ForceUnits = "cm/s"))
	float VelocityThreshold = 50.0f;

	/** Height Threshold for contacts */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ForceUnits = "cm"))
	float HeightThreshold = 25.0f;

	/** Radius for the circle to display to show contacts */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ForceUnits = "cm"))
	float ContactCircleRadius = 15.0f;

	/** Thickness of the circle to display to show contacts */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ForceUnits = "cm"))
	float ContactCircleThickness = 1.0f;

	/** Scale of the line drawn to show the velocity */
	UPROPERTY(EditAnywhere, Category = "Settings")
	float VelocityLineScale = 1.0f;

	/** Thickness of the line drawn to show the velocity */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ForceUnits = "cm"))
	float VelocityLineThickness = 1.0f;

	/** Graph Scale in Pixels */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ClampMin = "0", UIMin = "0"))
	float GraphHeight = 150.0f;

	/** Graph Scale in Pixels */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ClampMin = "0", UIMin = "0"))
	float GraphWidth = 200.0f;

	/** Number of frames to plot in the graphs */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ClampMin = "1", UIMin = "1"))
	int32 GraphFrameNum = 60;

	/** Maximum Velocity to display on graph */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ForceUnits = "cm/s"))
	float GraphMaxVelocity = 100.0f;

	/** Maximum Height to display on graph */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ForceUnits = "cm"))
	float GraphMaxHeight = 50.0f;

protected:

	FAnimDatabaseFrameAttribute LeftContactVelocity, LeftContactHeight;
	FAnimDatabaseFrameAttribute RightContactVelocity, RightContactHeight;

	TArray<float> TimeValues;
	TArray<float> LeftVelocityValues, LeftHeightValues;
	TArray<float> RightVelocityValues, RightHeightValues;

#endif

#if WITH_EDITOR

public:

	virtual void InitializeDrawDebug_Implementation(const UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges) override;

	virtual void DrawDebug_Implementation(
		const FDebugDrawer& Drawer,
		const FDebugDrawer& CanvasDrawer,
		const FAnimDatabasePoseState& InPoseState,
		const UAnimDatabase* InDatabase,
		const FAnimDatabaseFrameRanges& InFrameRanges,
		const FTransform& RangeViewportTransform,
		const int32 CharacterIdx,
		const int32 SequenceIdx,
		const float SequenceTime,
		const int32 RangeStart,
		const int32 RangeLength,
		const FLinearColor& IdentifierColor) override;

#endif
};


/** Debug Draw class for multiple other debug drawers */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Multi Debug Draw"))
class UAnimDatabaseDebugDraw_Multi : public UAnimDatabaseDebugDraw
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	/** Widget Radius */
	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TArray<TObjectPtr<UAnimDatabaseDebugDraw>> Drawers;

#endif

#if WITH_EDITOR

public:

	virtual void InitializeDrawDebug_Implementation(const UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges) override;

	virtual void DrawDebug_Implementation(
		const FDebugDrawer& Drawer,
		const FDebugDrawer& CanvasDrawer,
		const FAnimDatabasePoseState& InPoseState,
		const UAnimDatabase* InDatabase,
		const FAnimDatabaseFrameRanges& InFrameRanges,
		const FTransform& RangeViewportTransform,
		const int32 CharacterIdx,
		const int32 SequenceIdx,
		const float SequenceTime,
		const int32 RangeStart,
		const int32 RangeLength,
		const FLinearColor& IdentifierColor) override;

#endif
};

/** Debug Draw class for drawing root orientation */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Root Orientation Debug Draw"))
class UAnimDatabaseDebugDraw_RootOrientation : public UAnimDatabaseDebugDraw
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	/** Vertical Offset from Floor */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ForceUnits = "cm"))
	float FloorOffset = 1.0f;

	/** Widget Radius */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ForceUnits = "cm"))
	float Radius = 50.0f;

	/** Local root forward direction */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FVector ForwardVector = FVector::RightVector;

#endif

#if WITH_EDITOR

public:

	virtual void DrawDebug_Implementation(
		const FDebugDrawer& Drawer,
		const FDebugDrawer& CanvasDrawer,
		const FAnimDatabasePoseState& InPoseState,
		const UAnimDatabase* InDatabase,
		const FAnimDatabaseFrameRanges& InFrameRanges,
		const FTransform& RangeViewportTransform,
		const int32 CharacterIdx,
		const int32 SequenceIdx,
		const float SequenceTime,
		const int32 RangeStart,
		const int32 RangeLength,
		const FLinearColor& IdentifierColor) override;

#endif
};

/** Debug Draw class for drawing the current movement direction */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Movement Direction Debug Draw"))
class UAnimDatabaseDebugDraw_MovementDirection : public UAnimDatabaseDebugDraw
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	/** Vertical Offset from Floor */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ForceUnits = "cm"))
	float FloorOffset = 1.0f;

	/** Local root forward direction */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FVector Direction = FVector::RightVector;

	/** Character is only considered moving when the velocity is above this threshold */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm/s"))
	float VelocityThreshold = 25.0f;

	/** Direction Arrow Length */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ForceUnits = "cm"))
	float DirectionArrowLength = 50.0f;

	/** Velocity Scale */
	UPROPERTY(EditAnywhere, Category = "Settings")
	float VelocityScale = 0.5f;

	/** Widget Thickness */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ForceUnits = "cm"))
	float Thickness = 1.0f;

#endif

#if WITH_EDITOR

public:

	virtual void DrawDebug_Implementation(
		const FDebugDrawer& Drawer,
		const FDebugDrawer& CanvasDrawer,
		const FAnimDatabasePoseState& InPoseState,
		const UAnimDatabase* InDatabase,
		const FAnimDatabaseFrameRanges& InFrameRanges,
		const FTransform& RangeViewportTransform,
		const int32 CharacterIdx,
		const int32 SequenceIdx,
		const float SequenceTime,
		const int32 RangeStart,
		const int32 RangeLength,
		const FLinearColor& IdentifierColor) override;

#endif
};

/** Debug Draw the transform of the given bone */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Bone Transform Debug Draw"))
class UAnimDatabaseDebugDraw_BoneTransform : public UAnimDatabaseDebugDraw
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	/** Bone name to draw widget for */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FName BoneName = NAME_None;

	/** Widget Radius */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ForceUnits = "cm"))
	float Radius = 25.0f;

	/** Widget Thickness */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ForceUnits = "cm"))
	float Thickness = 1.0f;

	/** If to depth test while drawing */
	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bDepthTest = true;

#endif

#if WITH_EDITOR

public:

	virtual void DrawDebug_Implementation(
		const FDebugDrawer& Drawer,
		const FDebugDrawer& CanvasDrawer,
		const FAnimDatabasePoseState& InPoseState,
		const UAnimDatabase* InDatabase,
		const FAnimDatabaseFrameRanges& InFrameRanges,
		const FTransform& RangeViewportTransform,
		const int32 CharacterIdx,
		const int32 SequenceIdx,
		const float SequenceTime,
		const int32 RangeStart,
		const int32 RangeLength,
		const FLinearColor& IdentifierColor) override;

#endif
};


/** Debug Draw a bone attachment such as a prop */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Bone Attachment Debug Draw"))
class UAnimDatabaseDebugDraw_BoneAttachment : public UAnimDatabaseDebugDraw
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	/** Bone attached to */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FName BoneName = NAME_None;

	/** Relative transform of the attachment to the bone */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FTransform RelativeTransform = FTransform::Identity;

	/** If to draw the attachment transform */
	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bDrawTransform = true;

	/** Transform Radius */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ForceUnits = "cm"))
	float TransformRadius = 25.0f;

	/** Transform Thickness */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ForceUnits = "cm"))
	float TransformThickness = 1.0f;

	/** Static mesh to use for attachment (will draw bounding box only) */
	UPROPERTY(EditAnywhere, Category = "Settings")
	TObjectPtr<UStaticMesh> StaticMesh;

	/** Bounding box Thickness */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ForceUnits = "cm"))
	float BoundingBoxThickness = 1.0f;

	/** If to depth test while drawing */
	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bDepthTest = true;

#endif

#if WITH_EDITOR

public:

	virtual void DrawDebug_Implementation(
		const FDebugDrawer& Drawer,
		const FDebugDrawer& CanvasDrawer,
		const FAnimDatabasePoseState& InPoseState,
		const UAnimDatabase* InDatabase,
		const FAnimDatabaseFrameRanges& InFrameRanges,
		const FTransform& RangeViewportTransform,
		const int32 CharacterIdx,
		const int32 SequenceIdx,
		const float SequenceTime,
		const int32 RangeStart,
		const int32 RangeLength,
		const FLinearColor& IdentifierColor) override;

#endif
};

/** Debug Draw class for drawing root velocities */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Root Velocity Debug Draw"))
class UAnimDatabaseDebugDraw_RootVelocity : public UAnimDatabaseDebugDraw
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	/** Graph axes maximum for linear velocity */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ForceUnits = "cm/s"))
	float MaxLinearVelocity = 500.0f;

	/** Graph axes maximum for angular velocity */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ForceUnits = "deg/s"))
	float MaxAngularVelocity = 180.0f;

protected:

	/** Root Linear Velocity attribute */
	FAnimDatabaseFrameAttribute RootLinearVelocity;

	/** Root Angular Velocity attribute */
	FAnimDatabaseFrameAttribute RootAngularVelocity;

#endif

#if WITH_EDITOR

public:

	virtual void InitializeDrawDebug_Implementation(const UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges) override;

	virtual void DrawDebug_Implementation(
		const FDebugDrawer& Drawer,
		const FDebugDrawer& CanvasDrawer,
		const FAnimDatabasePoseState& InPoseState,
		const UAnimDatabase* InDatabase,
		const FAnimDatabaseFrameRanges& InFrameRanges,
		const FTransform& RangeViewportTransform,
		const int32 CharacterIdx,
		const int32 SequenceIdx,
		const float SequenceTime,
		const int32 RangeStart,
		const int32 RangeLength,
		const FLinearColor& IdentifierColor) override;

#endif
};


/** Debug Draw class for drawing a chair at the attach bone */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Chair on Attach Debug Draw"))
class UAnimDatabaseDebugDraw_ChairOnAttach : public UAnimDatabaseDebugDraw
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

public:

	/** Name of the attach bone */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FName AttachBoneName = TEXT("attach");

	/** Ranges where the attach bone is valid */
	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TObjectPtr<UAnimDatabaseFrameRangesFunction> AttachRanges;

protected:

	/** Valid ranges for attachment */
	FAnimDatabaseFrameRanges ValidAttachRanges;

#endif

#if WITH_EDITOR

public:

	virtual void InitializeDrawDebug_Implementation(const UAnimDatabase* InDatabase, const FAnimDatabaseFrameRanges& InFrameRanges) override;

	virtual void DrawDebug_Implementation(
		const FDebugDrawer& Drawer,
		const FDebugDrawer& CanvasDrawer,
		const FAnimDatabasePoseState& InPoseState,
		const UAnimDatabase* InDatabase,
		const FAnimDatabaseFrameRanges& InFrameRanges,
		const FTransform& RangeViewportTransform,
		const int32 CharacterIdx,
		const int32 SequenceIdx,
		const float SequenceTime,
		const int32 RangeStart,
		const int32 RangeLength,
		const FLinearColor& IdentifierColor) override;

#endif
};

#undef UE_API