// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimDatabaseFrameRanges.h"

#include "Kismet/BlueprintFunctionLibrary.h"
#include "Misc/FrameRate.h"

#include "AnimDatabaseFrameAttribute.generated.h"

#define UE_API ANIMDATABASE_API

class UAnimSequence;
class UAnimNotify;
class UAnimNotifyState;
class UAnimDatabaseFrameRangesFunction;
class UAnimDatabaseFrameAttributeFunction;

namespace UE::Learning
{
	struct FFrameAttribute;
}

/** The type of an attribute */
UENUM(BlueprintType)
enum class EAnimDatabaseAttributeType : uint8
{
	Null = 0,
	Bool = 1,
	Float = 2,
	Location = 3,
	Rotation = 4,
	Scale = 5,
	LinearVelocity = 6,
	AngularVelocity = 7,
	ScalarVelocity = 8,
	Direction = 9,
	Transform = 10,
	Event = 11,
	Angle = 12,
};

/* Mode for handling edges in phase extraction */
UENUM(BlueprintType)
enum class EAnimDatabasePhaseExtrapolationMode : uint8
{
	// Repeats the phase value at the start and end frames 
	Repeat = 0,

	// Extrapolates the phase value using the nearest pair of frames
	Extrapolate = 1,
};

/**
 * Represents a property associated with a set of frame ranges within a UAnimDatabase.
 *
 * Internally, this is effectively a thin blueprint wrapper around the UE::Learning::FFrameAttribute data structure plus an additional type tag which
 * allows for dynamic type checking in the blueprint interface. Below we provide a Blueprint Function Library that allows for the construction, 
 * modification and scripting of these objects in blueprints, but if you want to handle them in C++ at a low level if you can also access the internal 
 * FrameAttribute object directly.
 *
 * It is assumed that all FrameAttribute objects used by this wrapper are stored at the FrameRate of the database they were created from.
 */
USTRUCT(BlueprintType)
struct FAnimDatabaseFrameAttribute
{
	GENERATED_BODY()

public:

	/** Checks if the given FrameAttribute is valid (i.e. the shared pointer is not null) */
	UE_API bool IsValid() const;

	/** Returns the attribute data at the given frame as a Bool. Assumes the Attribute Type is correct. */
	UE_API bool GetAsBool(const int32 FrameIdx) const;

	/** Returns the attribute data at the given frame as a float. Assumes the Attribute Type is correct. */
	UE_API float GetAsFloat(const int32 FrameIdx) const;

	/** Returns the attribute data at the given frame as an event. Assumes the Attribute Type is correct. */
	UE_API void GetAsEvent(bool& bOutTimeUntilEventKnown, float& OutTimeUntilEvent, const int32 FrameIdx) const;

	/** Returns the attribute data at the given frame as a location. Assumes the Attribute Type is correct. */
	UE_API FVector3f GetAsLocation(const int32 FrameIdx) const;

	/** Returns the attribute data at the given frame as a scale. Assumes the Attribute Type is correct. */
	UE_API FVector3f GetAsScale(const int32 FrameIdx) const;

	/** Returns the attribute data at the given frame as a velocity. Assumes the Attribute Type is correct. */
	UE_API FVector3f GetAsVelocity(const int32 FrameIdx) const;

	/** Returns the attribute data at the given frame as a direction. Assumes the Attribute Type is correct. */
	UE_API FVector3f GetAsDirection(const int32 FrameIdx) const;

	/** Returns the attribute data at the given frame as a rotation. Assumes the Attribute Type is correct. */
	UE_API FQuat4f GetAsRotation(const int32 FrameIdx) const;

	/** Returns the attribute data at the given frame as a transform. Assumes the Attribute Type is correct. */
	UE_API FTransform3f GetAsTransform(const int32 FrameIdx) const;

	/** Returns the attribute data at the given frame as a double-precision location. Assumes the Attribute Type is correct. */
	UE_API FVector GetAsLocationDouble(const int32 FrameIdx) const;

	/** Returns the attribute data at the given frame as a double-precision scale. Assumes the Attribute Type is correct. */
	UE_API FVector GetAsScaleDouble(const int32 FrameIdx) const;

	/** Returns the attribute data at the given frame as a double-precision velocity. Assumes the Attribute Type is correct. */
	UE_API FVector GetAsVelocityDouble(const int32 FrameIdx) const;

	/** Returns the attribute data at the given frame as a double-precision direction. Assumes the Attribute Type is correct. */
	UE_API FVector GetAsDirectionDouble(const int32 FrameIdx) const;

	/** Returns the attribute data at the given frame as a double-precision rotation. Assumes the Attribute Type is correct. */
	UE_API FQuat GetAsRotationDouble(const int32 FrameIdx) const;

	/** Returns the attribute data at the given frame as a double-precision transform. Assumes the Attribute Type is correct. */
	UE_API FTransform GetAsTransformDouble(const int32 FrameIdx) const;

	/** Returns the attribute data at the given frame as an angle in degrees. Assumes the Attribute Type is correct. */
	UE_API float GetAsAngleDegrees(const int32 FrameIdx) const;

	/** Returns the attribute data at the given frame as an angle in radians. Assumes the Attribute Type is correct. */
	UE_API float GetAsAngleRadians(const int32 FrameIdx) const;

	/** Custom Serialization */
	UE_API bool Serialize(FArchive& Ar);

public:

	/** The type of this Frame Attribute */
	EAnimDatabaseAttributeType Type = EAnimDatabaseAttributeType::Null;

	/** Shared pointer to the actual FrameAttribute data structure */
	TSharedPtr<UE::Learning::FFrameAttribute, ESPMode::ThreadSafe> FrameAttribute;
};

UE_API bool operator==(const FAnimDatabaseFrameAttribute& Lhs, const FAnimDatabaseFrameAttribute& Rhs);

template<>
struct TStructOpsTypeTraits<FAnimDatabaseFrameAttribute> : TStructOpsTypeTraitsBase2<FAnimDatabaseFrameAttribute>
{
	enum
	{
		WithIdenticalViaEquality = true,
		WithSerializer = true
	};
};


UCLASS(BlueprintType, meta = (BlueprintThreadSafe))
class UAnimDatabaseFrameAttributeLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/** Gets the size (number of channels in the FrameAttribute) for a given type */ 
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API int32 AttributeTypeSize(const EAnimDatabaseAttributeType Type);

	/** Gets the name for a given type */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API FString AttributeTypeName(const EAnimDatabaseAttributeType Type);
	static UE_API const TCHAR* AttributeTypeNameInternal(const EAnimDatabaseAttributeType Type);

	/** Gets the Frame Ranges associated with a given Frame Attribute */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (BlueprintAutocast, CompactNodeTitle = "->"))
	static UE_API FAnimDatabaseFrameRanges FrameAttributeFrameRanges(const FAnimDatabaseFrameAttribute& FrameAttribute);

	/** Gets the Frame Ranges associated with a given Frame Attribute */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API void FrameAttributeAnimSequences(TArray<UAnimSequence*>& OutAnimSequences, const UAnimDatabase* Database, const FAnimDatabaseFrameAttribute& FrameAttribute);

public:

	/** Make an empty Frame Attribute */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute MakeEmptyFrameAttribute();

	/** Make a null Frame Attribute for the given ranges */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute MakeNullFrameAttribute(const FAnimDatabaseFrameRanges& FrameRanges);

	/** Make a Frame Attribute using the given UAnimDatabaseFrameAttributeFunction */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API FAnimDatabaseFrameAttribute MakeFrameAttributeFromFunction(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const UAnimDatabaseFrameAttributeFunction* Function);

	/** Makes Frame Attributes using the given UAnimDatabaseFrameAttributeFunctions */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API void MakeFrameAttributesFromFunctions(
		TArray<FAnimDatabaseFrameAttribute>& OutFrameAttributes,
		const UAnimDatabase* Database, 
		const FAnimDatabaseFrameRanges& FrameRanges, 
		const TArray<UAnimDatabaseFrameAttributeFunction*>& Functions);

	static UE_API void MakeFrameAttributesFromFunctionsArrayView(
		const TArrayView<FAnimDatabaseFrameAttribute> OutFrameAttributes,
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const TArrayView<UAnimDatabaseFrameAttributeFunction* const> Functions);


	/** Make a Frame Attribute using the given UAnimDatabaseFrameAttributeFunction */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API FAnimDatabaseFrameAttribute MakeFrameAttributeFromClass(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TSubclassOf<UAnimDatabaseFrameAttributeFunction> Class);

	/** Make Frame Attributes using the given array of UAnimDatabaseFrameAttributeFunctions */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API void MakeFrameAttributesFromClasses(
		TArray<FAnimDatabaseFrameAttribute>& OutFrameAttributes, 
		const UAnimDatabase* Database, 
		const FAnimDatabaseFrameRanges& FrameRanges,
		const TArray<TSubclassOf<UAnimDatabaseFrameAttributeFunction>>& Classes);

	static UE_API void MakeFrameAttributesFromClassesArrayView(
		const TArrayView<FAnimDatabaseFrameAttribute> OutFrameAttributes,
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const TArrayView<const TSubclassOf<UAnimDatabaseFrameAttributeFunction>> Classes);

	/** Make Frame Attributes using the given map of UAnimDatabaseFrameAttributeFunctions */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API void MakeFrameAttributesFromClassesMap(
		TMap<FName, FAnimDatabaseFrameAttribute>& OutFrameAttributes,
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const TMap<FName, TSubclassOf<UAnimDatabaseFrameAttributeFunction>>& Classes);

public:

	/** Make a bool frame range attribute from the given ranges and constant value */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute MakeBoolFrameAttributeFromConstant(const bool bActive, const FAnimDatabaseFrameRanges& FrameRanges);

	/** Make a bool frame range attribute from the given ranges with all frames set to true */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute MakeBoolFrameAttributeFromTrue(const FAnimDatabaseFrameRanges& FrameRanges);

	/** Make a bool frame range attribute from the given ranges with all frames set to false */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute MakeBoolFrameAttributeFromFalse(const FAnimDatabaseFrameRanges& FrameRanges);

	/** Makes a bool frame attribute for the given ranges where each truth value is randomly sampled */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta=(NotBlueprintThreadSafe))
	static UE_API FAnimDatabaseFrameAttribute MakeRandomBoolFrameAttribute(UPARAM(ref) int32& State, const FAnimDatabaseFrameRanges& FrameRanges);

	/** Makes multiple random bool frame attributes from the given range set */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API void MakeRandomBoolFrameAttributes(TArray<FAnimDatabaseFrameAttribute>& OutFrameAttributes, UPARAM(ref) int32& State, const FAnimDatabaseFrameRanges& FrameRanges, const int32 FrameAttributeNum);

	/** Makes a bool frame attribute which is true where the frame is contained in given Active FrameRangeSet, and false elsewhere */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute MakeBoolFrameAttributeFromActiveRanges(const FAnimDatabaseFrameRanges& Active, const FAnimDatabaseFrameRanges& FrameRanges);

	/** Makes multiple bool frame attributes which are true where the frame is contained in given Active FrameRangeSets, and false elsewhere */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API void MakeBoolFrameAttributesFromActiveRanges(TArray<FAnimDatabaseFrameAttribute>& OutFrameAttributes, const TArray<FAnimDatabaseFrameRanges>& Actives, const FAnimDatabaseFrameRanges& FrameRanges);
	static UE_API void MakeBoolFrameAttributesFromActiveRangesArrayViews(TArrayView<FAnimDatabaseFrameAttribute> OutFrameAttributes, const TArrayView<const FAnimDatabaseFrameRanges> Actives, const FAnimDatabaseFrameRanges& FrameRanges);

	/** Makes a bool frame attribute from the given ranges, which is true when the provided NotifyState is active, and false everywhere else */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API FAnimDatabaseFrameAttribute MakeBoolFrameAttributeFromAnimNotifyState(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TSubclassOf<UAnimNotifyState> NotifyState);

	/** Makes a bool frame attribute from the given ranges, which is true when the provided curve is present in the data, and false everywhere else */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API FAnimDatabaseFrameAttribute MakeBoolFrameAttributeFromCurveActive(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName CurveName);

	/** Makes a set of bool frame attributes from the given ranges, which are true when the provided curves are present in the data and false everywhere else */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API void MakeBoolFrameAttributesFromCurvesActive(TArray<FAnimDatabaseFrameAttribute>& OutBoolFrameAttributes, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArray<FName>& CurveNames);
	static UE_API void MakeBoolFrameAttributesFromCurvesActiveArrayView(TArrayView<FAnimDatabaseFrameAttribute> OutBoolFrameAttributes, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const FName> CurveNames);

	/** Makes a bool frame attribute which is true where the frame is mirrored, and false elsewhere */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API FAnimDatabaseFrameAttribute MakeBoolFrameAttributeFromMirrored(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges);

	/** Makes a bool frame attribute which is true where the frame is NOT mirrored, and false elsewhere */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API FAnimDatabaseFrameAttribute MakeBoolFrameAttributeFromNotMirrored(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges);

public:

	/** Makes a float frame attribute from the given ranges, filled with the provided value. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute MakeFloatFrameAttributeFromConstant(const FAnimDatabaseFrameRanges& FrameRanges, const float Value = 0.0f);

	/** Makes a float frame attribute from the given ranges, filled with zero. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute MakeFloatFrameAttributeFromZero(const FAnimDatabaseFrameRanges& FrameRanges);

	/** Makes a float frame attribute from the given ranges, filled with one. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute MakeFloatFrameAttributeFromOne(const FAnimDatabaseFrameRanges& FrameRanges);

	/** Makes a float frame attribute from the given ranges, using the given curve name. For when the curve is not present in the animations a default value of 0 will be used. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API FAnimDatabaseFrameAttribute MakeFloatFrameAttributeFromCurve(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName CurveName);

	/** Makes multiple float frame attributes from the given ranges and curve names. For when the curve is not present in the animations a default value of 0 will be used. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API void MakeFloatFrameAttributesFromCurves(TArray<FAnimDatabaseFrameAttribute>& OutFloatFrameAttributes, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArray<FName>& CurveNames);
	static UE_API void MakeFloatFrameAttributesFromCurvesArrayView(const TArrayView<FAnimDatabaseFrameAttribute> OutFloatFrameAttributes, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const FName> CurveNames);

	/** Makes a float frame attribute using the given curve name. For when the curve is not present in the database the attribute will be inactive. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API FAnimDatabaseFrameAttribute MakeFloatFrameAttributeFromCurveWhenActive(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName CurveName);

	/** Makes multiple float frame attributes from the given ranges and curve names. For when the curve is not present in the animations the attribute will be inactive. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API void MakeFloatFrameAttributesFromCurvesWhenActive(TArray<FAnimDatabaseFrameAttribute>& OutFloatFrameAttributes, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArray<FName>& CurveNames);
	static UE_API void MakeFloatFrameAttributesFromCurvesWhenActiveArrayView(const TArrayView<FAnimDatabaseFrameAttribute> OutFloatFrameAttributes, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const FName> CurveNames);

	/** Makes a float frame attribute where the value of the attribute is given by the time since the start of the range for each frame in each range in the set. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute MakeFloatFrameAttributeFromRangeTimes(const FAnimDatabaseFrameRanges& FrameRanges, const FFrameRate& FrameRate);

	/** Makes a float frame attribute where the value of the attribute is given by the sequence time of each frame in each range in the set. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute MakeFloatFrameAttributeFromSequenceTimes(const FAnimDatabaseFrameRanges& FrameRanges, const FFrameRate& FrameRate);

	/** Makes a float frame attribute where the value of the attribute is given by the sequence time of the nearest frame in the Frames frames set. If the range does not contain a frame from Frames, then the time is 0.0 */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute MakeFloatFrameAttributeFromNearestFrameSequenceTimes(const FAnimDatabaseFrameRanges& FrameRanges, const FAnimDatabaseFrames& Frames, const FFrameRate& FrameRate);

	/** Makes a float frame attribute from a uniform distribution. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API FAnimDatabaseFrameAttribute MakeFloatFrameAttributeFromUniformRandom(const FAnimDatabaseFrameRanges& FrameRanges, UPARAM(ref) int32& State, const float Min = 0.0f, const float Max = 1.0f);

public:

	/** Makes a location frame attribute from X, Y and Z attributes. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute MakeLocationFrameAttribute(const FAnimDatabaseFrameAttribute& X, const FAnimDatabaseFrameAttribute& Y, const FAnimDatabaseFrameAttribute& Z);

	/** Makes a location frame attribute from the given ranges, filled with the provided value. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute MakeLocationFrameAttributeFromConstant(const FAnimDatabaseFrameRanges& FrameRanges, const FVector Location = FVector::ZeroVector);

public:

	/** Makes a rotation frame attribute from Yaw, Pitch and Roll. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute MakeRotationFrameAttribute(const FAnimDatabaseFrameAttribute& Pitch, const FAnimDatabaseFrameAttribute& Yaw, const FAnimDatabaseFrameAttribute& Roll);

	/** Makes a rotation frame attribute from the given ranges, filled with the provided value. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute MakeRotationFrameAttributeFromConstant(const FAnimDatabaseFrameRanges& FrameRanges, const FRotator Rotation = FRotator::ZeroRotator);

	/** Makes a rotation frame attribute from the given ranges, filled with the provided value. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute MakeRotationFrameAttributeFromConstantQuat(const FAnimDatabaseFrameRanges& FrameRanges, const FQuat Rotation);

	/** Makes a rotation frame attribute from a set of basis directions. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute MakeRotationFrameAttributeFromBasisDirections(const FAnimDatabaseFrameAttribute& ForwardDirection, const FAnimDatabaseFrameAttribute& RightDirection, const FAnimDatabaseFrameAttribute& UpDirection);

public:

	/** Makes a direction frame attribute from X, Y and Z attributes. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute MakeDirectionFrameAttribute(const FAnimDatabaseFrameAttribute& X, const FAnimDatabaseFrameAttribute& Y, const FAnimDatabaseFrameAttribute& Z);

	/** Makes a direction frame attribute from the given ranges, filled with the provided value. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute MakeDirectionFrameAttributeFromConstant(const FAnimDatabaseFrameRanges& FrameRanges, const FVector Direction = FVector::ForwardVector);

public:

	/** Makes a scale frame attribute from X, Y and Z attributes. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute MakeScaleFrameAttribute(const FAnimDatabaseFrameAttribute& X, const FAnimDatabaseFrameAttribute& Y, const FAnimDatabaseFrameAttribute& Z);

	/** Makes a scale frame attribute from the given ranges, filled with the provided value. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute MakeScaleFrameAttributeFromConstant(const FAnimDatabaseFrameRanges& FrameRanges, const FVector Scale = FVector(1, 1, 1));

public:

	/** Makes an angle frame attribute from a float frame attribute with values in degrees */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute MakeAngleFrameAttributeFromFloatDegrees(const FAnimDatabaseFrameAttribute& A);

	/** Makes an angle frame attribute from a float frame attribute with values in radians */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute MakeAngleFrameAttributeFromFloatRadians(const FAnimDatabaseFrameAttribute& A);

	/** Makes an angle frame attribute from the given ranges, filled with the provided value in degrees. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute MakeAngleFrameAttributeFromConstantDegrees(const FAnimDatabaseFrameRanges& FrameRanges, const float Value = 0.0f);

	/** Makes an angle frame attribute from the given ranges, filled with the provided value in radians. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute MakeAngleFrameAttributeFromConstantRadians(const FAnimDatabaseFrameRanges& FrameRanges, const float Value = 0.0f);

public:

	/** Makes a location frame attribute from the root location of the character at the given frame ranges */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 2))
	static UE_API FAnimDatabaseFrameAttribute MakeRootLocationFrameAttribute(
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const float RelativeTime = 0.0f);

	/** Makes a rotation frame attribute from the root rotation of the character at the given frame ranges */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 2))
	static UE_API FAnimDatabaseFrameAttribute MakeRootRotationFrameAttribute(
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const float RelativeTime = 0.0f);

	/** Makes a linear velocity frame attribute from the root linear velocity of the character at the given frame ranges */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 2))
	static UE_API FAnimDatabaseFrameAttribute MakeRootLinearVelocityFrameAttribute(
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const float RelativeTime = 0.0f);

	/** Makes a angular velocity frame attribute from the root angular velocity of the character at the given frame ranges */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 2))
	static UE_API FAnimDatabaseFrameAttribute MakeRootAngularVelocityFrameAttribute(
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const float RelativeTime = 0.0f);

	/** Makes a direction frame attribute from the root direction of the character at the given frame ranges */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 2))
	static UE_API FAnimDatabaseFrameAttribute MakeRootDirectionFrameAttribute(
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const float RelativeTime = 0.0f,
		const FVector ForwardVector = FVector(0.0f, 1.0f, 0.0f));

	/** Makes a transform frame attribute from the root transform of the character at the given frame ranges */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 2))
	static UE_API FAnimDatabaseFrameAttribute MakeRootTransformFrameAttribute(
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const float RelativeTime = 0.0f);

	/** Makes a transform frame attribute from the root transform of the character at the start of each sequence */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 2))
	static UE_API FAnimDatabaseFrameAttribute MakeRootTransformAtSequenceStartFrameAttribute(
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges);

	/** Makes a location and rotation frame attribute from the root location and rotation of the character at the given frame ranges */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 4))
	static UE_API void MakeRootLocationAndRotationFrameAttribute(
		FAnimDatabaseFrameAttribute& OutRootLocationFrameAttribute,
		FAnimDatabaseFrameAttribute& OutRootRotationFrameAttribute,
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const float RelativeTime = 0.0f);

	/** Makes a location and direction frame attribute from the root location and direction of the character at the given frame ranges */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 4))
	static UE_API void MakeRootLocationAndDirectionFrameAttribute(
		FAnimDatabaseFrameAttribute& OutRootLocationFrameAttribute,
		FAnimDatabaseFrameAttribute& OutRootDirectionFrameAttribute,
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const float RelativeTime = 0.0f,
		const FVector ForwardVector = FVector(0.0f, 1.0f, 0.0f));

	/** Makes a location frame attribute from the character root location at the start of the range for every range in the given frame ranges */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API FAnimDatabaseFrameAttribute MakeRootLocationAtRangeStartFrameAttribute(
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges);

	/** Makes a direction frame attribute from the character root direction at the start of the range for every range in the given frame ranges */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API FAnimDatabaseFrameAttribute MakeRootDirectionAtRangeStartFrameAttribute(
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const FVector ForwardVector = FVector(0.0f, 1.0f, 0.0f));

	/** Makes a location frame attribute from the character root location at the end of the range for every range in the given frame ranges */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API FAnimDatabaseFrameAttribute MakeRootLocationAtRangeEndFrameAttribute(
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges);

	/** Makes a direction frame attribute from the character root direction at the end of the range for every range in the given frame ranges */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API FAnimDatabaseFrameAttribute MakeRootDirectionAtRangeEndFrameAttribute(
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const FVector ForwardVector = FVector(0.0f, 1.0f, 0.0f));

	/** 
	 * Makes a location and direction frame attribute from the character root location and direction at the end of the range for every range in the 
	 * given frame ranges
	 */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API void MakeRootLocationAndDirectionAtRangeEndFrameAttribute(
		FAnimDatabaseFrameAttribute& OutLocationFrameAttribute,
		FAnimDatabaseFrameAttribute& OutDirectionFrameAttribute,
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const FVector ForwardVector = FVector(0.0f, 1.0f, 0.0f));

public:
	
	/** Makes an array of location and direction frame attributes for the root at the given sample times into the future */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 8))
	static UE_API void MakeTrajectoryFrameAttribute(
		TArray<FAnimDatabaseFrameAttribute>& OutLocationFrameAttributes,
		TArray<FAnimDatabaseFrameAttribute>& OutDirectionFrameAttributes,
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const int32 SampleNum = 4,
		const float FutureTime = 1.0f,
		const float PastTime = 0.0f,
		const FVector ForwardVector = FVector(0.0f, 1.0f, 0.0f));

	/** Makes an array of location frame attributes for the root at the given sample times into the future */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 6))
	static UE_API void MakeLocationTrajectoryFrameAttribute(
		TArray<FAnimDatabaseFrameAttribute>& OutLocationFrameAttributes,
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const int32 SampleNum = 4,
		const float FutureTime = 1.0f,
		const float PastTime = 0.0f);

	/**
	 * Makes an array of location and direction frame attributes for the root location and look-at-direction of the provided bone at the given 
	 * sample times into the future
	 */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 10))
	static UE_API void MakeLookAtTrajectoryFrameAttribute(
		TArray<FAnimDatabaseFrameAttribute>& OutLocationFrameAttributes,
		TArray<FAnimDatabaseFrameAttribute>& OutDirectionFrameAttributes,
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const int32 LookAtBoneIndex,
		const int32 SampleNum = 4,
		const float FutureTime = 1.0f,
		const float PastTime = 0.0f,
		const FVector LocalDirection = FVector(0.0f, 1.0f, 0.0f),
		const float DirectionSmoothingAmount = 15.0f);

	/**
	 * Makes trajectory frame attributes transformed local to the character's root location and rotation. Computes the overall scale of locations -
	 * which is useful for scaling distance costs for use with matching.
	 */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 9))
	static UE_API void MakeRootLocalTrajectoryFrameAttributeAndScale(
		TArray<FAnimDatabaseFrameAttribute>& OutLocationFrameAttributes,
		TArray<FAnimDatabaseFrameAttribute>& OutDirectionFrameAttributes,
		float& OutLocationScale,
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const int32 SampleNum = 4,
		const float FutureTime = 1.0f,
		const float PastTime = 0.0f,
		const FVector ForwardVector = FVector(0.0f, 1.0f, 0.0f));

	/**
	 * Makes look-at trajectory frame attributes transformed local to the character's root location and rotation. Computes the overall scale of 
	 * locations - which is useful for scaling distance costs for use with matching.
	 */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 11))
	static UE_API void MakeRootLocalLookAtTrajectoryFrameAttributeAndScale(
		TArray<FAnimDatabaseFrameAttribute>& OutLocationFrameAttributes,
		TArray<FAnimDatabaseFrameAttribute>& OutDirectionFrameAttributes,
		float& OutLocationScale,
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const int32 LookAtBoneIndex,
		const int32 SampleNum = 4,
		const float FutureTime = 1.0f,
		const float PastTime = 0.0f,
		const FVector LocalDirection = FVector(0.0f, 1.0f, 0.0f),
		const float DirectionSmoothingAmount = 15.0f);

public:

	/**
	 * General function for computing bone attributes in the local space. Outputs various arrays of frame attributes based on the bone indices passed
	 * in. This is the most efficient way of computing multiple local bone attributes as it combines all required bone indices into one large array,
	 * evaluates the pose data in the animation sequences once extracting information for these bones.
	 *
	 * Given that evaluating the animation data for a whole animation database can be relatively slow it is encouraged to use this function wherever
	 * possible rather than the more convenient individual functions for computing bone attributes.
	 */
	UFUNCTION(BlueprintPure = false, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 14, AutoCreateRefTerm = "BoneLocationIndices, BoneRotationIndices, BoneScaleIndices, BoneLinearVelocityIndices, BoneAngularVelocityIndices, BoneScalarVelocityIndices"))
	static UE_API void MakeBoneLocalFrameAttributes(
		TArray<FAnimDatabaseFrameAttribute>& OutLocationFrameAttributes,
		TArray<FAnimDatabaseFrameAttribute>& OutRotationFrameAttributes,
		TArray<FAnimDatabaseFrameAttribute>& OutScaleFrameAttributes,
		TArray<FAnimDatabaseFrameAttribute>& OutLinearVelocityFrameAttributes,
		TArray<FAnimDatabaseFrameAttribute>& OutAngularVelocityFrameAttributes,
		TArray<FAnimDatabaseFrameAttribute>& OutScalarVelocityFrameAttributes,
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const TArray<int32>& BoneLocationIndices,
		const TArray<int32>& BoneRotationIndices,
		const TArray<int32>& BoneScaleIndices,
		const TArray<int32>& BoneLinearVelocityIndices,
		const TArray<int32>& BoneAngularVelocityIndices,
		const TArray<int32>& BoneScalarVelocityIndices,
		const float RelativeTime = 0.0f);

	static UE_API void MakeBoneLocalFrameAttributesFromArrayViews(
		const TArrayView<FAnimDatabaseFrameAttribute> OutLocationFrameAttributes,
		const TArrayView<FAnimDatabaseFrameAttribute> OutRotationFrameAttributes,
		const TArrayView<FAnimDatabaseFrameAttribute> OutScaleFrameAttributes,
		const TArrayView<FAnimDatabaseFrameAttribute> OutLinearVelocityFrameAttributes,
		const TArrayView<FAnimDatabaseFrameAttribute> OutAngularVelocityFrameAttributes,
		const TArrayView<FAnimDatabaseFrameAttribute> OutScalarVelocityFrameAttributes,
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const TArrayView<const int32> BoneLocationIndices = {},
		const TArrayView<const int32> BoneRotationIndices = {},
		const TArrayView<const int32> BoneScaleIndices = {},
		const TArrayView<const int32> BoneLinearVelocityIndices = {},
		const TArrayView<const int32> BoneAngularVelocityIndices = {},
		const TArrayView<const int32> BoneScalarVelocityIndices = {},
		const float RelativeTime = 0.0f);

	/**
	 * General function for computing bone transforms in the local space. Outputs frame attributes based on the bone indices passed in. This is the
	 * most efficient way of computing multiple local bone transform attributes as it combines all required bone indices into one large array,
	 * evaluates the pose data in the animation sequences once extracting information for these bones.
	 *
	 * Given that evaluating the animation data for a whole animation database can be relatively slow it is encouraged to use this function wherever
	 * possible rather than the more convenient individual functions for computing bone transform attributes.
	 */
	UFUNCTION(BlueprintPure = false, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AutoCreateRefTerm = "BoneIndices"))
	static UE_API void MakeBoneLocalTransformFrameAttributes(
		TArray<FAnimDatabaseFrameAttribute>& OutTransformFrameAttributes,
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const TArray<int32>& BoneIndices,
		const float RelativeTime = 0.0f);

	static UE_API void MakeBoneLocalTransformFrameAttributesFromArrayViews(
		const TArrayView<FAnimDatabaseFrameAttribute> OutTransformFrameAttributes,
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const TArrayView<const int32> BoneIndices = {},
		const float RelativeTime = 0.0f);

	UFUNCTION(BlueprintPure = false, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AutoCreateRefTerm = "BoneNames"))
	static UE_API void MakeBoneLocalTransformFrameAttributesFromNames(
		TArray<FAnimDatabaseFrameAttribute>& OutTransformFrameAttributes,
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const TArray<FName>& BoneNames,
		const float RelativeTime = 0.0f);

	static UE_API void MakeBoneLocalTransformFrameAttributesFromNamesArrayView(
		const TArrayView<FAnimDatabaseFrameAttribute> OutTransformFrameAttributes,
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const TArrayView<const FName> BoneNames = {},
		const float RelativeTime = 0.0f);

	/** Makes a location frame attribute from the local transform of a bone */
	UFUNCTION(BlueprintPure = false, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 3))
	static UE_API FAnimDatabaseFrameAttribute MakeBoneLocalTransformFrameAttribute(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const int32 BoneIndex, const float RelativeTime = 0.0f);

	/** Makes a location frame attribute from the local transform of a bone, provided by name */
	UFUNCTION(BlueprintPure = false, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 3))
	static UE_API FAnimDatabaseFrameAttribute MakeBoneLocalTransformFrameAttributeFromName(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName BoneName, const float RelativeTime = 0.0f);

	/** Makes a location frame attribute from the local location of a bone */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 3))
	static UE_API FAnimDatabaseFrameAttribute MakeBoneLocalLocationFrameAttribute(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const int32 BoneIndex, const float RelativeTime = 0.0f);

	/** Makes a location frame attribute from the local location of a bone, provided by name */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 3))
	static UE_API FAnimDatabaseFrameAttribute MakeBoneLocalLocationFrameAttributeFromName(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName BoneName, const float RelativeTime = 0.0f);

	/** Makes a rotation frame attribute from the local rotation of a bone */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 3))
	static UE_API FAnimDatabaseFrameAttribute MakeBoneLocalRotationFrameAttribute(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const int32 BoneIndex, const float RelativeTime = 0.0f);

	/** Makes a rotation frame attribute from the local rotation of a bone, provided by name */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 3))
	static UE_API FAnimDatabaseFrameAttribute MakeBoneLocalRotationFrameAttributeFromName(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName BoneName, const float RelativeTime = 0.0f);

	/** Makes a scale frame attribute from the local scale of a bone */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 3))
	static UE_API FAnimDatabaseFrameAttribute MakeBoneLocalScaleFrameAttribute(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const int32 BoneIndex, const float RelativeTime = 0.0f);

	/** Makes a scale frame attribute from the local scale of a bone, provided by name */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 3))
	static UE_API FAnimDatabaseFrameAttribute MakeBoneLocalScaleFrameAttributeFromName(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName BoneName, const float RelativeTime = 0.0f);


	/**
	 * General function for computing bone attributes in the world space. Outputs various arrays of frame attributes based on the bone indices passed
	 * in. This is the most efficient way of computing multiple global bone attributes as it combines all required bone indices into one large array, 
	 * evaluates the pose data in the animation sequences once extracting information for these bones, and then performs just the subset of forward 
	 * kinematics required to extract the desired properties.
	 * 
	 * Given that evaluating the animation data for a whole animation database can be relatively slow it is encouraged to use this function wherever
	 * possible rather than the more convenient individual functions for computing bone attributes.
	 */
	UFUNCTION(BlueprintPure = false, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 14, AutoCreateRefTerm = "BoneLocationIndices, BoneRotationIndices, BoneScaleIndices, BoneLinearVelocityIndices, BoneAngularVelocityIndices, BoneScalarVelocityIndices"))
	static UE_API void MakeBoneGlobalFrameAttributes(
		TArray<FAnimDatabaseFrameAttribute>& OutLocationFrameAttributes,
		TArray<FAnimDatabaseFrameAttribute>& OutRotationFrameAttributes,
		TArray<FAnimDatabaseFrameAttribute>& OutScaleFrameAttributes,
		TArray<FAnimDatabaseFrameAttribute>& OutLinearVelocityFrameAttributes,
		TArray<FAnimDatabaseFrameAttribute>& OutAngularVelocityFrameAttributes,
		TArray<FAnimDatabaseFrameAttribute>& OutScalarVelocityFrameAttributes,
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const TArray<int32>& BoneLocationIndices,
		const TArray<int32>& BoneRotationIndices,
		const TArray<int32>& BoneScaleIndices,
		const TArray<int32>& BoneLinearVelocityIndices,
		const TArray<int32>& BoneAngularVelocityIndices,
		const TArray<int32>& BoneScalarVelocityIndices,
		const float RelativeTime = 0.0f);

	static UE_API void MakeBoneGlobalFrameAttributesFromArrayViews(
		const TArrayView<FAnimDatabaseFrameAttribute> OutLocationFrameAttributes,
		const TArrayView<FAnimDatabaseFrameAttribute> OutRotationFrameAttributes,
		const TArrayView<FAnimDatabaseFrameAttribute> OutScaleFrameAttributes,
		const TArrayView<FAnimDatabaseFrameAttribute> OutLinearVelocityFrameAttributes,
		const TArrayView<FAnimDatabaseFrameAttribute> OutAngularVelocityFrameAttributes,
		const TArrayView<FAnimDatabaseFrameAttribute> OutScalarVelocityFrameAttributes,
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const TArrayView<const int32> BoneLocationIndices = {},
		const TArrayView<const int32> BoneRotationIndices = {},
		const TArrayView<const int32> BoneScaleIndices = {},
		const TArrayView<const int32> BoneLinearVelocityIndices = {},
		const TArrayView<const int32> BoneAngularVelocityIndices = {},
		const TArrayView<const int32> BoneScalarVelocityIndices = {},
		const float RelativeTime = 0.0f);

	/**
	 * General function for computing bone transforms in the world space. Outputs frame attributes based on the bone indices passed in. This is the 
	 * most efficient way of computing multiple global bone transform attributes as it combines all required bone indices into one large array,
	 * evaluates the pose data in the animation sequences once extracting information for these bones, and then performs just the subset of forward
	 * kinematics required to extract the desired properties.
	 *
	 * Given that evaluating the animation data for a whole animation database can be relatively slow it is encouraged to use this function wherever
	 * possible rather than the more convenient individual functions for computing bone transform attributes.
	 */
	UFUNCTION(BlueprintPure = false, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 4, AutoCreateRefTerm = "BoneIndices"))
	static UE_API void MakeBoneGlobalTransformFrameAttributes(
		TArray<FAnimDatabaseFrameAttribute>& OutTransformFrameAttributes,
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const TArray<int32>& BoneIndices,
		const float RelativeTime = 0.0f);

	static UE_API void MakeBoneGlobalFrameAttributesFromArrayViews(
		const TArrayView<FAnimDatabaseFrameAttribute> OutTransformFrameAttributes,
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const TArrayView<const int32> BoneIndices = {},
		const float RelativeTime = 0.0f);

	/**
	 * General function for computing bone transforms in the world space. Outputs frame attributes based on the bone names passed in. This is the
	 * most efficient way of computing multiple global bone transform attributes as it combines all required bone names into one large array,
	 * evaluates the pose data in the animation sequences once extracting information for these bones, and then performs just the subset of forward
	 * kinematics required to extract the desired properties.
	 *
	 * Given that evaluating the animation data for a whole animation database can be relatively slow it is encouraged to use this function wherever
	 * possible rather than the more convenient individual functions for computing bone transform attributes.
	 */
	UFUNCTION(BlueprintPure = false, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 4, AutoCreateRefTerm = "BoneNames"))
	static UE_API void MakeBoneGlobalTransformFrameAttributesFromNames(
		TArray<FAnimDatabaseFrameAttribute>& OutTransformFrameAttributes,
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const TArray<FName>& BoneNames,
		const float RelativeTime = 0.0f);

	static UE_API void MakeBoneGlobalTransformFrameAttributesFromNamesArrayView(
		const TArrayView<FAnimDatabaseFrameAttribute> OutTransformFrameAttributes,
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const TArrayView<const FName> BoneNames,
		const float RelativeTime = 0.0f);

	/** Makes a location frame attribute from the global location of a bone */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 3))
	static UE_API FAnimDatabaseFrameAttribute MakeBoneGlobalLocationFrameAttribute(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const int32 BoneIndex, const float RelativeTime = 0.0f);

	/** Makes a location frame attribute from the global location of a bone, provided by name */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 3))
	static UE_API FAnimDatabaseFrameAttribute MakeBoneGlobalLocationFrameAttributeFromName(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName BoneName, const float RelativeTime = 0.0f);

	/** Makes location frame attributes from the global location of bones */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 4))
	static UE_API void MakeBoneGlobalLocationFrameAttributes(TArray<FAnimDatabaseFrameAttribute>& OutLocationFrameAttributes, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArray<int32>& BoneIndices, const float RelativeTime = 0.0f);
	static UE_API void MakeBoneGlobalLocationFrameAttributesFromArrayViews(const TArrayView<FAnimDatabaseFrameAttribute> OutLocationFrameAttributes, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const int32> BoneIndices, const float RelativeTime = 0.0f);

	/** Makes location frame attributes from the global location of bones, provided by name */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 4))
	static UE_API void MakeBoneGlobalLocationFrameAttributesFromNames(TArray<FAnimDatabaseFrameAttribute>& OutLocationFrameAttributes, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArray<FName>& BoneNames, const float RelativeTime = 0.0f);
	static UE_API void MakeBoneGlobalLocationFrameAttributesFromNamesArrayView(const TArrayView<FAnimDatabaseFrameAttribute> OutLocationFrameAttributes, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const FName> BoneNames, const float RelativeTime = 0.0f);

	/** Makes a rotation frame attribute from the global rotation of a bone */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 3))
	static UE_API FAnimDatabaseFrameAttribute MakeBoneGlobalRotationFrameAttribute(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const int32 BoneIndex, const float RelativeTime = 0.0f);

	/** Makes a rotation frame attribute from the global rotation of a bone, provided by name */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 3))
	static UE_API FAnimDatabaseFrameAttribute MakeBoneGlobalRotationFrameAttributeFromName(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName BoneName, const float RelativeTime = 0.0f);

	/** Makes a direction frame attribute from the global direction of a bone */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 4))
	static UE_API FAnimDatabaseFrameAttribute MakeBoneGlobalDirectionFrameAttribute(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const int32 BoneIndex, const FVector LocalDirection = FVector::ForwardVector, const float RelativeTime = 0.0f);

	/** Makes a direction frame attribute from the global direction of a bone, provided by name */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 4))
	static UE_API FAnimDatabaseFrameAttribute MakeBoneGlobalDirectionFrameAttributeFromName(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName BoneName, const FVector LocalDirection = FVector::ForwardVector, const float RelativeTime = 0.0f);

	/** Makes direction frame attributes from the global direction of bones */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 5))
	static UE_API void MakeBoneGlobalDirectionFrameAttributes(TArray<FAnimDatabaseFrameAttribute>& OutDirectionFrameAttributes, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArray<int32>& BoneIndices, const FVector LocalDirection = FVector::ForwardVector, const float RelativeTime = 0.0f);
	static UE_API void MakeBoneGlobalDirectionFrameAttributesFromArrayViews(const TArrayView<FAnimDatabaseFrameAttribute> OutDirectionFrameAttributes, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const int32> BoneIndices, const FVector LocalDirection = FVector::ForwardVector, const float RelativeTime = 0.0f);

	/** Makes direction frame attributes from the global direction of bones, provided by name */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 5))
	static UE_API void MakeBoneGlobalDirectionFrameAttributesFromNames(TArray<FAnimDatabaseFrameAttribute>& OutDirectionFrameAttributes, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArray<FName>& BoneNames, const FVector LocalDirection = FVector::ForwardVector, const float RelativeTime = 0.0f);
	static UE_API void MakeBoneGlobalDirectionFrameAttributesFromNamesArrayView(const TArrayView<FAnimDatabaseFrameAttribute> OutDirectionFrameAttributes, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const FName> BoneNames, const FVector LocalDirection = FVector::ForwardVector, const float RelativeTime = 0.0f);

	/** Makes a linear velocity frame attribute from the global linear velocity of a bone */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 3))
	static UE_API FAnimDatabaseFrameAttribute MakeBoneGlobalLinearVelocityFrameAttribute(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const int32 BoneIndex, const float RelativeTime = 0.0f);

	/** Makes a linear velocity frame attribute from the global linear velocity of a bone, provided by name */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 3))
	static UE_API FAnimDatabaseFrameAttribute MakeBoneGlobalLinearVelocityFrameAttributeFromName(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName BoneName, const float RelativeTime = 0.0f);

	/** Makes a angular velocity frame attribute from the global angular velocity of a bone */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 3))
	static UE_API FAnimDatabaseFrameAttribute MakeBoneGlobalAngularVelocityFrameAttribute(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const int32 BoneIndex, const float RelativeTime = 0.0f);

	/** Makes a angular velocity frame attribute from the global angular velocity of a bone, provided by name */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 3))
	static UE_API FAnimDatabaseFrameAttribute MakeBoneGlobalAngularVelocityFrameAttributeFromName(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName BoneName, const float RelativeTime = 0.0f);

	/** Makes a transform frame attribute from the global transform of a bone */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 3))
	static UE_API FAnimDatabaseFrameAttribute MakeBoneGlobalTransformFrameAttribute(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const int32 BoneIndex, const float RelativeTime = 0.0f);

	/** Makes a transform frame attribute from the global transform of a bone, provided by name */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 3))
	static UE_API FAnimDatabaseFrameAttribute MakeBoneGlobalTransformFrameAttributeFromName(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName BoneName, const float RelativeTime = 0.0f);

	/** Makes a location and linear velocity frame attribute from the global location and linear velocity of a bone */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (AdvancedDisplay = 5))
	static UE_API void MakeBoneGlobalLocationAndLinearVelocityFrameAttribute(FAnimDatabaseFrameAttribute& OutLocationFrameAttribute, FAnimDatabaseFrameAttribute& OutLinearVelocityFrameAttribute, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const int32 BoneIndex, const float RelativeTime = 0.0f);

	/** Makes a location and linear velocity frame attribute from the global location and linear velocity of a bone, provided by name */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (AdvancedDisplay = 5))
	static UE_API void MakeBoneGlobalLocationAndLinearVelocityFrameAttributeFromName(FAnimDatabaseFrameAttribute& OutLocationFrameAttribute, FAnimDatabaseFrameAttribute& OutLinearVelocityFrameAttribute, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName BoneName, const float RelativeTime = 0.0f);

	/** Makes a direction frame attribute from the global vector between bones */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 4))
	static UE_API FAnimDatabaseFrameAttribute MakeGlobalDirectionBetweenBonesFrameAttribute(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const int32 BoneIndexFrom, const int32 BoneIndexTo);

	/** Makes a direction frame attribute from the global vector between bones, provided by name */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 4))
	static UE_API FAnimDatabaseFrameAttribute MakeGlobalDirectionBetweenBonesFrameAttributeFromNames(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName BoneNameFrom, const FName BoneNameTo);

	/** Makes arrays of frame attributes for a pose from the provided bone indices */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AdvancedDisplay = 5))
	static UE_API void MakePoseFrameAttribute(
		TArray<FAnimDatabaseFrameAttribute>& OutLocationFrameAttributes,
		TArray<FAnimDatabaseFrameAttribute>& OutLinearVelocityFrameAttributes,
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const TArray<int32>& BoneIndices,
		const float RelativeTime = 0.0f);

	/**
	 * Makes arrays of frame attributes for a pose from the provided bone indices, transformed to be local to the root location and rotation, and 
	 * then computes the overall scale of the locations and velocities.
	 */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API void MakeRootLocalPoseFrameAttributeAndScale(
		TArray<FAnimDatabaseFrameAttribute>& OutLocationFrameAttributes,
		TArray<FAnimDatabaseFrameAttribute>& OutLinearVelocityFrameAttributes,
		float& OutLocationScale,
		float& OutLinearVelocityScale,
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const TArray<int32>& BoneIndices);

	/** Computes the global bone location of a bone at the nearest frame in the set of Frames */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API FAnimDatabaseFrameAttribute MakeBoneGlobalLocationAtNearestFramesFrameAttribute(
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const int32 BoneIndex,
		const FAnimDatabaseFrames& Frames);

	/** Computes the global bone direction of a bone at the nearest frame in the set of Frames */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API FAnimDatabaseFrameAttribute MakeBoneGlobalDirectionAtNearestFramesFrameAttribute(
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const int32 BoneIndex,
		const FAnimDatabaseFrames& Frames,
		const FVector LocalDirection = FVector::ForwardVector);

public:

	/** Makes frame attributes for curve values, curve velocities, and if a curve is active from a set of given curves */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API void MakeFrameAttributesFromCurves(
		TArray<FAnimDatabaseFrameAttribute>& OutFloatCurveValueFrameAttributes,
		TArray<FAnimDatabaseFrameAttribute>& OutFloatCurveVelocityFrameAttributes,
		TArray<FAnimDatabaseFrameAttribute>& OutBoolCurveActiveFrameAttributes,
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const TArray<FName>& CurveNames);

	static UE_API void MakeFrameAttributesFromCurvesArrayViews(
		const TArrayView<FAnimDatabaseFrameAttribute> OutFloatCurveValueFrameAttributes,
		const TArrayView<FAnimDatabaseFrameAttribute> OutFloatCurveVelocityFrameAttributes,
		const TArrayView<FAnimDatabaseFrameAttribute> OutBoolCurveActiveFrameAttributes,
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const TArrayView<const FName> CurveNames);

public:

	/** Makes a transform frame attribute from the given frame ranges filled with a constant value */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute MakeTransformFrameAttributeFromConstant(const FAnimDatabaseFrameRanges& FrameRanges, const FTransform Transform = FTransform());

	/** Makes a transform frame attribute from the given frame ranges filled with the identity transform */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute MakeTransformFrameAttributeFromIdentity(const FAnimDatabaseFrameRanges& FrameRanges);

	/** Makes a transform frame attribute from the given Location, Rotation, and Scale frame attributes */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute MakeTransformFrameAttribute(const FAnimDatabaseFrameAttribute& Location, const FAnimDatabaseFrameAttribute& Rotation, const FAnimDatabaseFrameAttribute& Scale);

	/** Makes a transform frame attribute from the given Location and Rotation frame attributes */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute MakeTransformFrameAttributeNoScale(const FAnimDatabaseFrameAttribute& Location, const FAnimDatabaseFrameAttribute& Rotation);

public:

	/** Makes an event frame attribute from the given ranges, at the times of the given frames */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute MakeEventFrameAttribute(const FAnimDatabaseFrames& EventFrames, const FAnimDatabaseFrameRanges& FrameRanges, const FFrameRate& FrameRate);

	/** Makes an event frame attribute from the given ranges, at the start times of the given EventFrameRanges */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute MakeEventFrameAttributeAtRangeStarts(const FAnimDatabaseFrameRanges& EventFrameRanges, const FAnimDatabaseFrameRanges& FrameRanges, const FFrameRate& FrameRate);

	/** Makes an event frame attribute from the given ranges, at the end times of the given EventFrameRanges */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute MakeEventFrameAttributeAtRangeEnds(const FAnimDatabaseFrameRanges& EventFrameRanges, const FAnimDatabaseFrameRanges& FrameRanges, const FFrameRate& FrameRate);

	/** Makes an event frame attribute from the given ranges, at the times of the given anim notifies */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API FAnimDatabaseFrameAttribute MakeEventFrameAttributeFromAnimNotify(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TSubclassOf<UAnimNotify> Notify);

	/** Makes an event frame attribute from the given ranges, at the times of the given anim notifies */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API FAnimDatabaseFrameAttribute MakeEventFrameAttributeFromAnimNotifyUnion(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArray<TSubclassOf<UAnimNotify>>& Notifies);
	static UE_API FAnimDatabaseFrameAttribute MakeEventFrameAttributeFromAnimNotifyUnionArrayView(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const TSubclassOf<UAnimNotify>> Notifies);

	/** Makes an event frame attribute from the given ranges, at the times of the given anim notify state starts */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API FAnimDatabaseFrameAttribute MakeEventFrameAttributeFromAnimNotifyStateStarts(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TSubclassOf<UAnimNotifyState> NotifyState);

	/** Makes an event frame attribute from the given ranges, at the times of the given anim notify state ends */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API FAnimDatabaseFrameAttribute MakeEventFrameAttributeFromAnimNotifyStateEnds(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TSubclassOf<UAnimNotifyState> NotifyState);

public:

	/** Sample the given FrameAttribute at the provide Sequence and Frame as a bool. Returns false if sequence and frame is not within Frame Attribute Ranges. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API bool FrameAttributeBoolAtFrame(bool& OutBool, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx);

	/** Sample the given FrameAttribute at the provide Sequence and Frame as a float. Returns false if sequence and frame is not within Frame Attribute Ranges. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API bool FrameAttributeFloatAtFrame(float& OutValue, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx);

	/** Sample the given FrameAttribute at the provide Sequence and Frame as a location. Returns false if sequence and frame is not within Frame Attribute Ranges. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API bool FrameAttributeLocationAtFrame(FVector& OutLocation, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx);
	static UE_API bool FrameAttributeLocationAtFrameFloat(FVector3f& OutLocation, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx);

	/** Sample the given FrameAttribute at the provide Sequence and Frame as a rotation. Returns false if sequence and frame is not within Frame Attribute Ranges. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API bool FrameAttributeRotationAtFrame(FRotator& OutRotation, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx);
	static UE_API bool FrameAttributeRotationAtFrameAsQuat(FQuat& OutRotation, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx);
	static UE_API bool FrameAttributeRotationAtFrameAsQuatFloat(FQuat4f& OutRotation, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx);

	/** Sample the given FrameAttribute at the provide Sequence and Frame as a scale. Returns false if sequence and frame is not within Frame Attribute Ranges. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API bool FrameAttributeScaleAtFrame(FVector& OutScale, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx);
	static UE_API bool FrameAttributeScaleAtFrameFloat(FVector3f& OutScale, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx);

	/** Sample the given FrameAttribute at the provide Sequence and Frame as a linear velocity. Returns false if sequence and frame is not within Frame Attribute Ranges. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API bool FrameAttributeLinearVelocityAtFrame(FVector& OutLinearVelocity, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx);
	static UE_API bool FrameAttributeLinearVelocityAtFrameFloat(FVector3f& OutLinearVelocity, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx);

	/** Sample the given FrameAttribute at the provide Sequence and Frame as an angular velocity. Returns false if sequence and frame is not within Frame Attribute Ranges. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API bool FrameAttributeAngularVelocityAtFrame(FVector& OutAngularVelocity, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx);
	static UE_API bool FrameAttributeAngularVelocityAtFrameFloat(FVector3f& OutAngularVelocity, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx);

	/** Sample the given FrameAttribute at the provide Sequence and Frame as scalar velocity. Returns false if sequence and frame is not within Frame Attribute Ranges. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API bool FrameAttributeScalarVelocityAtFrame(FVector& OutScalarVelocity, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx);
	static UE_API bool FrameAttributeScalarVelocityAtFrameFloat(FVector3f& OutScalarVelocity, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx);

	/** Sample the given FrameAttribute at the provide Sequence and Frame as a direction. Returns false if sequence and frame is not within Frame Attribute Ranges. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API bool FrameAttributeDirectionAtFrame(FVector& OutDirection, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx);
	static UE_API bool FrameAttributeDirectionAtFrameFloat(FVector3f& OutDirection, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx);

	/** Sample the given FrameAttribute at the provide Sequence and Frame as a transform. Returns false if sequence and frame is not within Frame Attribute Ranges. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API bool FrameAttributeTransformAtFrame(FTransform& OutTransform, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx);
	static UE_API bool FrameAttributeTransformAtFrameFloat(FTransform3f& OutTransform, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx);

	/** Sample the given FrameAttribute at the provide Sequence and Frame as an event. Returns false if sequence and frame is not within Frame Attribute Ranges. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API bool FrameAttributeEventAtFrame(bool& OutTimeUntilEventKnown, float& OutTimeUntilEvent, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx);

	/** Sample the given FrameAttributes at the provide Sequence and Frame as an array of locations. Returns false if sequence and frame is not within Frame Attribute Ranges. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API bool FrameAttributeLocationsAtFrame(TArray<FVector>& OutLocations, const TArray<FAnimDatabaseFrameAttribute>& FrameAttributes, const int32 SequenceIdx, const int32 FrameIdx);
	static UE_API bool FrameAttributeLocationsAtFrameToArrayView(const TArrayView<FVector> OutLocations, const TArrayView<const FAnimDatabaseFrameAttribute> FrameAttributes, const int32 SequenceIdx, const int32 FrameIdx);

	/** Sample the given FrameAttributes at the provide Sequence and Frame as an array of directions. Returns false if sequence and frame is not within Frame Attribute Ranges. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API bool FrameAttributeDirectionsAtFrame(TArray<FVector>& OutDirections, const TArray<FAnimDatabaseFrameAttribute>& FrameAttributes, const int32 SequenceIdx, const int32 FrameIdx);
	static UE_API bool FrameAttributeDirectionsAtFrameToArrayView(const TArrayView<FVector> OutDirections, const TArrayView<const FAnimDatabaseFrameAttribute> FrameAttributes, const int32 SequenceIdx, const int32 FrameIdx);

	/** Sample the given FrameAttributes at the provide Sequence and Frame as an array of linear velocities. Returns false if sequence and frame is not within Frame Attribute Ranges. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API bool FrameAttributeLinearVelocitiesAtFrame(TArray<FVector>& OutLinearVelocities, const TArray<FAnimDatabaseFrameAttribute>& FrameAttributes, const int32 SequenceIdx, const int32 FrameIdx);
	static UE_API bool FrameAttributeLinearVelocitiesAtFrameToArrayView(const TArrayView<FVector> OutLinearVelocities, const TArrayView<const FAnimDatabaseFrameAttribute> FrameAttributes, const int32 SequenceIdx, const int32 FrameIdx);

	/** Sample the given FrameAttribute at the provide Sequence and Frame as an angle in degrees. Returns false if sequence and frame is not within Frame Attribute Ranges. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API bool FrameAttributeAngleAtFrameDegrees(float& OutValue, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx);

	/** Sample the given FrameAttribute at the provide Sequence and Frame as an angle in radians. Returns false if sequence and frame is not within Frame Attribute Ranges. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API bool FrameAttributeAngleAtFrameRadians(float& OutValue, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const int32 FrameIdx);


public:

	/** Sample the given FrameAttribute at the provide Sequence and Time as a bool. Returns false if sequence and time is not within Frame Attribute Ranges. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API bool FrameAttributeSampleBool(bool& OutBool, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate);

	/** Sample the given FrameAttribute at the provide Sequence and Time as a float. Returns false if sequence and time is not within Frame Attribute Ranges. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API bool FrameAttributeSampleFloat(float& OutValue, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate);

	/** Sample the given FrameAttribute at the provide Sequence and Time Range as a float. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API void FrameAttributeSampleFloatRange(TArray<float>& OutValues, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float StartTime, const float EndTime, const int32 Num, const FFrameRate& FrameRate);
	static UE_API void FrameAttributeSampleFloatRangeToArrayView(TArrayView<float> OutValues, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float StartTime, const float EndTime, const int32 Num, const FFrameRate& FrameRate);

	/** Sample the given FrameAttribute at the provide Sequence and Time as a location. Returns false if sequence and time is not within Frame Attribute Ranges. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API bool FrameAttributeSampleLocation(FVector& OutLocation, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate);
	static UE_API bool FrameAttributeSampleLocationFloat(FVector3f& OutLocation, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate);

	/** Sample the given FrameAttribute at the provide Sequence and Time as a rotation. Returns false if sequence and time is not within Frame Attribute Ranges. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API bool FrameAttributeSampleRotation(FRotator& OutRotation, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate);
	static UE_API bool FrameAttributeSampleRotationAsQuat(FQuat& OutRotation, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate);
	static UE_API bool FrameAttributeSampleRotationAsQuatFloat(FQuat4f& OutRotation, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate);

	/** Sample the given FrameAttribute at the provide Sequence and Time as a scale. Returns false if sequence and time is not within Frame Attribute Ranges. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API bool FrameAttributeSampleScale(FVector& OutScale, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate);
	static UE_API bool FrameAttributeSampleScaleFloat(FVector3f& OutScale, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate);

	/** Sample the given FrameAttribute at the provide Sequence and Time as a linear velocity. Returns false if sequence and time is not within Frame Attribute Ranges. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API bool FrameAttributeSampleLinearVelocity(FVector& OutLinearVelocity, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate);
	static UE_API bool FrameAttributeSampleLinearVelocityFloat(FVector3f& OutLinearVelocity, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate);

	/** Sample the given FrameAttribute at the provide Sequence and Time as an angular velocity. Returns false if sequence and time is not within Frame Attribute Ranges. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API bool FrameAttributeSampleAngularVelocity(FVector& OutAngularVelocity, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate);
	static UE_API bool FrameAttributeSampleAngularVelocityFloat(FVector3f& OutAngularVelocity, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate);

	/** Sample the given FrameAttribute at the provide Sequence and Time as scalar velocity. Returns false if sequence and time is not within Frame Attribute Ranges. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API bool FrameAttributeSampleScalarVelocity(FVector& OutScalarVelocity, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate);
	static UE_API bool FrameAttributeSampleScalarVelocityFloat(FVector3f& OutScalarVelocity, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate);

	/** Sample the given FrameAttribute at the provide Sequence and Time as a direction. Returns false if sequence and time is not within Frame Attribute Ranges. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API bool FrameAttributeSampleDirection(FVector& OutDirection, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate);
	static UE_API bool FrameAttributeSampleDirectionFloat(FVector3f& OutDirection, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate);

	/** Sample the given FrameAttribute at the provide Sequence and Time as a transform. Returns false if sequence and time is not within Frame Attribute Ranges. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API bool FrameAttributeSampleTransform(FTransform& OutTransform, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate);
	static UE_API bool FrameAttributeSampleTransformFloat(FTransform3f& OutTransform, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate);

	/** Sample the given FrameAttribute at the provide Sequence and Time as an event. Returns false if sequence and time is not within Frame Attribute Ranges. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API bool FrameAttributeSampleEvent(bool& OutTimeUntilEventKnown, float& OutTimeUntilEvent, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate);

	/** Sample the given FrameAttributes at the provide Sequence and Time as an array of locations. Returns false if sequence and time is not within Frame Attribute Ranges. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API bool FrameAttributeSampleLocations(TArray<FVector>& OutLocations, const TArray<FAnimDatabaseFrameAttribute>& FrameAttributes, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate);
	static UE_API bool FrameAttributeSampleLocationsToArrayView(const TArrayView<FVector> OutLocations, const TArrayView<const FAnimDatabaseFrameAttribute> FrameAttributes, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate);

	/** Sample the given FrameAttributes at the provide Sequence and Time as an array of directions. Returns false if sequence and time is not within Frame Attribute Ranges. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API bool FrameAttributeSampleDirections(TArray<FVector>& OutDirections, const TArray<FAnimDatabaseFrameAttribute>& FrameAttributes, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate);
	static UE_API bool FrameAttributeSampleDirectionsToArrayView(const TArrayView<FVector> OutDirections, const TArrayView<const FAnimDatabaseFrameAttribute> FrameAttributes, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate);

	/** Sample the given FrameAttributes at the provide Sequence and Time as an array of linear velocities. Returns false if sequence and time is not within Frame Attribute Ranges. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API bool FrameAttributeSampleLinearVelocities(TArray<FVector>& OutLinearVelocities, const TArray<FAnimDatabaseFrameAttribute>& FrameAttributes, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate);
	static UE_API bool FrameAttributeSampleLinearVelocitiesToArrayView(const TArrayView<FVector> OutLinearVelocities, const TArrayView<const FAnimDatabaseFrameAttribute> FrameAttributes, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate);

	/** Sample the given FrameAttribute at the provide Sequence and Time as an angle in degrees. Returns false if sequence and time is not within Frame Attribute Ranges. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API bool FrameAttributeSampleAngleDegrees(float& OutValue, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate);

	/** Sample the given FrameAttribute at the provide Sequence and Time as an angle in radians. Returns false if sequence and time is not within Frame Attribute Ranges. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API bool FrameAttributeSampleAngleRadians(float& OutValue, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate);

public:

	/* Gets the total number of ranges in a frame attribute */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API int32 FrameAttributeTotalRangeNum(const FAnimDatabaseFrameAttribute& FrameAttribute);

	/* Gets the total number of frames in a frame attribute */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API int32 FrameAttributeTotalFrameNum(const FAnimDatabaseFrameAttribute& FrameAttribute);

	/** Converts a frame attribute to an array of bools */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API void FrameAttributeToBoolArray(TArray<bool>& OutArray, const FAnimDatabaseFrameAttribute& FrameAttribute);
	static UE_API void FrameAttributeToBoolArrayView(const TArrayView<bool> OutArray, const FAnimDatabaseFrameAttribute& FrameAttribute);

	/** Converts a frame attribute to an array of floats */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API void FrameAttributeToFloatArray(TArray<float>& OutArray, const FAnimDatabaseFrameAttribute& FrameAttribute);
	static UE_API void FrameAttributeToFloatArrayView(const TArrayView<float> OutArray, const FAnimDatabaseFrameAttribute& FrameAttribute);

	/** Converts a frame attribute to an array of vectors */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API void FrameAttributeToVectorArray(TArray<FVector>& OutArray, const FAnimDatabaseFrameAttribute& FrameAttribute);
	static UE_API void FrameAttributeToVectorArrayView(TArray<FVector>& OutArray, const FAnimDatabaseFrameAttribute& FrameAttribute);

	/** Converts a frame attribute to an array of quats */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API void FrameAttributeToQuatArray(TArray<FQuat>& OutArray, const FAnimDatabaseFrameAttribute& FrameAttribute);
	static UE_API void FrameAttributeToQuatArrayView(TArray<FQuat>& OutArray, const FAnimDatabaseFrameAttribute& FrameAttribute);

	/** Converts a frame attribute to an array of rotators */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API void FrameAttributeToRotatorArray(TArray<FRotator>& OutArray, const FAnimDatabaseFrameAttribute& FrameAttribute);
	static UE_API void FrameAttributeToRotatorArrayView(TArray<FRotator>& OutArray, const FAnimDatabaseFrameAttribute& FrameAttribute);

	/** Converts a frame attribute to an array of transforms */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API void FrameAttributeToTransformArray(TArray<FTransform>& OutArray, const FAnimDatabaseFrameAttribute& FrameAttribute);
	static UE_API void FrameAttributeToTransformArrayView(TArray<FTransform>& OutArray, const FAnimDatabaseFrameAttribute& FrameAttribute);

	/** Converts a frame attribute to an array of floats in degrees */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API void FrameAttributeToAngleArrayDegrees(TArray<float>& OutArray, const FAnimDatabaseFrameAttribute& FrameAttribute);
	static UE_API void FrameAttributeToAngleArrayViewDegrees(const TArrayView<float> OutArray, const FAnimDatabaseFrameAttribute& FrameAttribute);

	/** Converts a frame attribute to an array of floats in radians */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API void FrameAttributeToAngleArrayRadians(TArray<float>& OutArray, const FAnimDatabaseFrameAttribute& FrameAttribute);
	static UE_API void FrameAttributeToAngleArrayViewRadians(const TArrayView<float> OutArray, const FAnimDatabaseFrameAttribute& FrameAttribute);

public:

	/** Computes a float frame attribute representing the matching distance between some location frame attribute and a constant location value. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeLocationMatchingDistance(
		const FAnimDatabaseFrameAttribute& LocationFrameAttribute,
		const FVector Location,
		const float Scale = 100.0f,
		const float Weight = 1.0f);

	/** Computes a float frame attribute representing the matching distance between some direction frame attribute and a constant direction value. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeDirectionMatchingDistance(
		const FAnimDatabaseFrameAttribute& DirectionFrameAttribute,
		const FVector Direction,
		const float Weight = 1.0f);

	/** Computes a float frame attribute representing the matching distance between some rotation frame attribute and a constant rotation value. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeRotationMatchingDistance(
		const FAnimDatabaseFrameAttribute& RotationFrameAttribute,
		const FQuat Rotation,
		const float Weight = 1.0f);

	/** Computes a float frame attribute representing the matching distance between some velocity frame attribute and a constant velocity value. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeVelocityMatchingDistance(
		const FAnimDatabaseFrameAttribute& VelocityFrameAttribute,
		const FVector Velocity,
		const float Scale = 200.0f,
		const float Weight = 1.0f);

	/** Computes a float frame attribute representing the matching distance between some event frame attribute and a constant event value. */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeEventMatchingDistance(
		const FAnimDatabaseFrameAttribute& EventFrameAttribute,
		const bool bTimeUntilEventKnown,
		const float TimeUntilEvent,
		const float Weight = 1.0f);

	/**
	 * Computes a float frame attribute representing the matching distance between some location and linear velocity frame attributes and a constant 
	 * location and velocity. Computes this distance using a combination of the maximum velocity offset that would be produced by inertialization,
	 * and the integral of the offset displacement produced from inertialization.
	 * 
	 * In general this provides a "nicer" distance metric than just comparing locations and velocities independently because it accounts for the fact
	 * that some velocity offsets can actually better correct location offsets during inertialization.
	 */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeInertializationMatchingDistance(
		const FAnimDatabaseFrameAttribute& LocationFrameAttribute,
		const FAnimDatabaseFrameAttribute& LinearVelocityFrameAttribute,
		const FVector Location,
		const FVector Velocity,
		const float BlendTime = 0.2f,
		const float LocationScale = 100.0f,
		const float VelocityScale = 200.0f,
		const float LocationWeight = 1.0f,
		const float VelocityWeight = 1.0f);

public:

	/** Computes the average of the std across all channels of a frame attribute */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API float FrameAttributeAverageStd(const FAnimDatabaseFrameAttribute& FrameAttribute);

	/** Computes the average of the std across all channels of multiple frame attributes */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API float FrameAttributesAverageStd(const TArray<FAnimDatabaseFrameAttribute>& FrameRanges);
	static UE_API float FrameAttributesAverageStdFromArrayView(const TArrayView<const FAnimDatabaseFrameAttribute> FrameAttributes);

public:

	/** Find the sequence and sequence frame with the smallest value in a float frame attribute */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API void FindFrameAttributeMinimum(
		int32& OutSequenceIdx,
		int32& OutSequenceFrame,
		float& OutMinimumValue,
		const FAnimDatabaseFrameAttribute& FrameAttribute);

	/** Find the sequence and sequence frame with the largest value in a float frame attribute */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API void FindFrameAttributeMaximum(
		int32& OutSequenceIdx,
		int32& OutSequenceFrame,
		float& OutMaximumValue,
		const FAnimDatabaseFrameAttribute& FrameAttribute);

	/** Find the sequence with the smallest value in a float frame attribute */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API void FindFrameAttributeMinimumSequence(
		UAnimSequence*& OutAnimSequence,
		float& OutAnimSequenceTime,
		bool& bOutAnimSequenceMirrored,
		float& OutMinimumValue,
		const FAnimDatabaseFrameAttribute& FrameAttribute,
		const UAnimDatabase* Database);

public:

	/** Checks if the frame ranges associated with two frame attributes are equal */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API bool FrameAttributeFrameRangesEqual(const FAnimDatabaseFrameAttribute& A, const FAnimDatabaseFrameAttribute& B);

	/** Returns the intersection of a frame attribute and some range set. */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "INTERSECTION"))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeIntersection(const FAnimDatabaseFrameAttribute& InFrameAttribute, const FAnimDatabaseFrameRanges& InFrameRanges);

	/** Converts a Bool frame attribute into frame ranges */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameRanges FrameAttributeBoolToFrameRanges(const FAnimDatabaseFrameAttribute& BoolFrameAttribute);

	/** Converts frame ranges to a Bool frame attribute  */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeFrameRangesToBool(const FAnimDatabaseFrameRanges& ActiveFrameRanges, const FAnimDatabaseFrameRanges& InFrameRanges);

	/** Selects between True and False based on the given condition bool frame attribute */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeSelect(const FAnimDatabaseFrameAttribute& Cond, const FAnimDatabaseFrameAttribute& True, const FAnimDatabaseFrameAttribute& False);


public:

	/** Creates a frame attribute that repeats the first value in each range for the rest of the range */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeRepeatFirstInRange(const FAnimDatabaseFrameAttribute& FrameAttribute);

public:

	/** Adds two frame attributes. Assumes types are compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CommutativeAssociativeBinaryOperator, CompactNodeTitle = "+"))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeAdd(FAnimDatabaseFrameAttribute A, FAnimDatabaseFrameAttribute B);

	/** Subtracts two frame attributes. Assumes types are compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CommutativeAssociativeBinaryOperator, CompactNodeTitle = "-"))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeSubtract(FAnimDatabaseFrameAttribute A, FAnimDatabaseFrameAttribute B);

	/** Multiplies two frame attributes. Assumes types are compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CommutativeAssociativeBinaryOperator, CompactNodeTitle = "*"))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeMultiply(FAnimDatabaseFrameAttribute A, FAnimDatabaseFrameAttribute B);

	/** Divides two frame attributes. Assumes types are compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CommutativeAssociativeBinaryOperator, CompactNodeTitle = "/"))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeDivide(FAnimDatabaseFrameAttribute A, FAnimDatabaseFrameAttribute B);

	/** Computes the component-wise maximum of two frame attributes. Assumes types are compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CommutativeAssociativeBinaryOperator, CompactNodeTitle = "MAX"))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeMax(FAnimDatabaseFrameAttribute A, FAnimDatabaseFrameAttribute B);

	/** Computes the component-wise minimum of two frame attributes. Assumes types are compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CommutativeAssociativeBinaryOperator, CompactNodeTitle = "MIN"))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeMin(FAnimDatabaseFrameAttribute A, FAnimDatabaseFrameAttribute B);

	/** Computes the dot product of two frame attributes. Assumes types are compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CommutativeAssociativeBinaryOperator, CompactNodeTitle = "DOT"))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeDot(FAnimDatabaseFrameAttribute A, FAnimDatabaseFrameAttribute B);

	/** Computes the cross product of two frame attributes. Assumes types are compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CommutativeAssociativeBinaryOperator, CompactNodeTitle = "CROSS"))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeCross(FAnimDatabaseFrameAttribute A, FAnimDatabaseFrameAttribute B);



	/** Adds a constant float value to a frame attribute. Assumes types are compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "+"))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeAddFloat(const FAnimDatabaseFrameAttribute& A, const float B);

	/** Subtracts a constant float value to a frame attribute. Assumes types are compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "-"))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeSubtractFloat(const FAnimDatabaseFrameAttribute& A, const float B);

	/** Multiplies a frame attribute by a constant float value. Assumes types are compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "*"))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeMultiplyFloat(const FAnimDatabaseFrameAttribute& A, const float B);

	/** Divides a frame attribute by a constant float value. Assumes types are compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "/"))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeDivideFloat(const FAnimDatabaseFrameAttribute& A, const float B);

	/** Computes the component-wise maximum of a frame attribute and a float value. Assumes types are compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "MAX"))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeMaxFloat(const FAnimDatabaseFrameAttribute& A, const float B);

	/** Computes the component-wise minimum of a frame attribute and a float value. Assumes types are compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "MIN"))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeMinFloat(const FAnimDatabaseFrameAttribute& A, const float B);


	/** Adds a constant vector value to a frame attribute. Assumes types are compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "+"))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeAddVector(const FAnimDatabaseFrameAttribute& A, const FVector B);

	/** Subtracts a constant vector value to a frame attribute. Assumes types are compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "-"))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeSubtractVector(const FAnimDatabaseFrameAttribute& A, const FVector B);

	/** Multiplies a frame attribute by a constant vector value. Assumes types are compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "*"))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeMultiplyVector(const FAnimDatabaseFrameAttribute& A, const FVector B);

	/** Divides a frame attribute by a constant vector value. Assumes types are compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "/"))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeDivideVector(const FAnimDatabaseFrameAttribute& A, const FVector B);


	/** Multiplies a frame attribute by a constant quat value. Assumes types are compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "*"))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeQuatMultiply(const FQuat A, const FAnimDatabaseFrameAttribute& B);

	/** Divides a frame attribute by a constant quat value. Assumes types are compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "/"))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeQuatDivide(const FQuat A, const FAnimDatabaseFrameAttribute& B);

	/** Multiplies a frame attribute by a constant quat value. Assumes types are compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "*"))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeMultiplyQuat(const FAnimDatabaseFrameAttribute& A, const FQuat B);

	/** Divides a frame attribute by a constant quat value. Assumes types are compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "/"))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeDivideQuat(const FAnimDatabaseFrameAttribute& A, const FQuat B);


	/** Multiplies a frame attribute by a constant transform value. Assumes types are compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "*"))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeTransformMultiply(const FTransform& A, const FAnimDatabaseFrameAttribute& B);

	/** Divides a frame attribute by a constant transform value. Assumes types are compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "/"))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeTransformDivide(const FTransform& A, const FAnimDatabaseFrameAttribute& B);

	/** Multiplies a frame attribute by a constant transform value. Assumes types are compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "*"))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeMultiplyTransform(const FAnimDatabaseFrameAttribute& A, const FTransform& B);

	/** Divides a frame attribute by a constant transform value. Assumes types are compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "/"))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeDivideTransform(const FAnimDatabaseFrameAttribute& A, const FTransform& B);

	/** Applies a constant transformation to a frame attribute. Assumes types are compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "*"))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeTransformApply(const FTransform& A, const FAnimDatabaseFrameAttribute& B);

	/** Applies a transformation to a constant location vector. Assumes types are compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeApplyTransformLocation(const FAnimDatabaseFrameAttribute& A, const FVector B);

	/** Applies a transformation to a constant direction vector. Assumes types are compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeApplyTransformDirection(const FAnimDatabaseFrameAttribute& A, const FVector B);


	/** Adds a constant angle value in degrees to an angle frame attribute. */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "+"))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeAddAngleDegrees(const FAnimDatabaseFrameAttribute& A, const float B);

	/** Subtracts a constant angle value in degrees to an angle frame attribute. */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "-"))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeSubtractAngleDegrees(const FAnimDatabaseFrameAttribute& A, const float B);

	/** Adds a constant angle value in radians to an angle frame attribute. */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "+"))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeAddAngleRadians(const FAnimDatabaseFrameAttribute& A, const float B);

	/** Subtracts a constant angle value in radians to an angle frame attribute. */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "-"))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeSubtractAngleRadians(const FAnimDatabaseFrameAttribute& A, const float B);



	/** Adds a frame attribute to multiple others. Assumes types are compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API void FrameAttributesAdd(TArray<FAnimDatabaseFrameAttribute>& Out, const TArray<FAnimDatabaseFrameAttribute>& A, const FAnimDatabaseFrameAttribute& B);
	static UE_API void FrameAttributesAddArrayView(const TArrayView<FAnimDatabaseFrameAttribute> Out, const TArrayView<const FAnimDatabaseFrameAttribute> A, const FAnimDatabaseFrameAttribute& B);

	/** Subtracts a frame attribute to multiple others. Assumes types are compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API void FrameAttributesSubtract(TArray<FAnimDatabaseFrameAttribute>& Out, const TArray<FAnimDatabaseFrameAttribute>& A, const FAnimDatabaseFrameAttribute& B);
	static UE_API void FrameAttributesSubtractArrayView(const TArrayView<FAnimDatabaseFrameAttribute> Out, const TArrayView<const FAnimDatabaseFrameAttribute> A, const FAnimDatabaseFrameAttribute& B);

	/** Multiplies a frame attribute to multiple others. Assumes types are compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API void FrameAttributesMultiply(TArray<FAnimDatabaseFrameAttribute>& Out, const TArray<FAnimDatabaseFrameAttribute>& A, const FAnimDatabaseFrameAttribute& B);
	static UE_API void FrameAttributesMultiplyArrayView(const TArrayView<FAnimDatabaseFrameAttribute> Out, const TArrayView<const FAnimDatabaseFrameAttribute> A, const FAnimDatabaseFrameAttribute& B);

	/** Divides a frame attribute to multiple others. Assumes types are compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API void FrameAttributesDivide(TArray<FAnimDatabaseFrameAttribute>& Out, const TArray<FAnimDatabaseFrameAttribute>& A, const FAnimDatabaseFrameAttribute& B);
	static UE_API void FrameAttributesDivideArrayView(const TArrayView<FAnimDatabaseFrameAttribute> Out, const TArrayView<const FAnimDatabaseFrameAttribute> A, const FAnimDatabaseFrameAttribute& B);

public:

	/** Computes the sum of multiple frame attributes. Assumes all are the same type */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeSum(const TArray<FAnimDatabaseFrameAttribute>& FrameAttributes);
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeSumFromArrayView(const TArrayView<const FAnimDatabaseFrameAttribute> FrameAttributes);

	/** Computes the weighted sum of multiple frame attributes. Assumes all are the same type */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeWeightedSum(const TArray<FAnimDatabaseFrameAttribute>& FrameAttributes, const TArray<float>& Weights);
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeWeightedSumFromArrayView(const TArrayView<const FAnimDatabaseFrameAttribute> FrameAttributes, const TArrayView<const float> Weights);

	/** Computes the weighted sum of multiple frame attributes where the weights are dynamic. Assumes all are the same type */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeDynamicWeightedSum(const TArray<FAnimDatabaseFrameAttribute>& FrameAttributes, const TArray<FAnimDatabaseFrameAttribute>& Weights);
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeDynamicWeightedSumFromArrayView(const TArrayView<const FAnimDatabaseFrameAttribute> FrameAttributes, const TArrayView<const FAnimDatabaseFrameAttribute> Weights);

	/** Computes the product of multiple frame attributes. Assumes all are the same type */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeProduct(const TArray<FAnimDatabaseFrameAttribute>& FrameAttributes);
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeProductFromArrayView(const TArrayView<const FAnimDatabaseFrameAttribute> FrameAttributes);

public:

	/** Copies a frame attribute. */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeCopy(const FAnimDatabaseFrameAttribute& A);

	/** Negates a frame attribute. Assumes types can be negated */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeNegate(const FAnimDatabaseFrameAttribute& A);

	/** Inverts a frame attribute. Assumes types can be inverted. For float attributes this computes 1/A */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeInvert(const FAnimDatabaseFrameAttribute& A);

	/** Takes the absolute value of a frame attribute. Assumes operation is valid on the input type */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeAbs(const FAnimDatabaseFrameAttribute& A);

	/** Takes the log value of a frame attribute. Assumes operation is valid on the input type */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeLog(const FAnimDatabaseFrameAttribute& A);

	/** Takes the exp value of a frame attribute. Assumes operation is valid on the input type */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeExp(const FAnimDatabaseFrameAttribute& A);

	/** Takes the sin value of a frame attribute. Assumes input is an Angle attribute. */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeSin(const FAnimDatabaseFrameAttribute& A);

	/** Takes the cos value of a frame attribute. Assumes input is an Angle attribute. */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeCos(const FAnimDatabaseFrameAttribute& A);

	/** Computes the atan2 value of two frame attributes. Assumes inputs are Float attributes. Output is an Angle attribute. */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeAtan2(const FAnimDatabaseFrameAttribute& Y, const FAnimDatabaseFrameAttribute& X);

public:

	/** Computes inv(A) x B for two frame attributes. Assumes types are compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeInverseMultiply(const FAnimDatabaseFrameAttribute& A, const FAnimDatabaseFrameAttribute& B);

	/** Computes A x inv(B) for two frame attributes. This is the same as division. Assumes types are compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeMultiplyInverse(const FAnimDatabaseFrameAttribute& A, const FAnimDatabaseFrameAttribute& B);

	/** Computes inv(A) x B for arrays of frame attributes. Assumes types are compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API void FrameAttributesInverseMultiply(TArray<FAnimDatabaseFrameAttribute>& Out, const TArray<FAnimDatabaseFrameAttribute>& A, const FAnimDatabaseFrameAttribute& B);
	static UE_API void FrameAttributesInverseMultiplyArrayView(const TArrayView<FAnimDatabaseFrameAttribute> Out, const TArrayView<const FAnimDatabaseFrameAttribute> A, const FAnimDatabaseFrameAttribute& B);

	/** Computes A x inv(B) for arrays of frame attributes. Assumes types are compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API void FrameAttributesMultiplyInverse(TArray<FAnimDatabaseFrameAttribute>& Out, const TArray<FAnimDatabaseFrameAttribute>& A, const FAnimDatabaseFrameAttribute& B);
	static UE_API void FrameAttributesMultiplyInverseArrayView(const TArrayView<FAnimDatabaseFrameAttribute> Out, const TArrayView<const FAnimDatabaseFrameAttribute> A, const FAnimDatabaseFrameAttribute& B);

public:

	/** Rotates a vector frame attribute by the given rotation frame attribute */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeRotate(const FAnimDatabaseFrameAttribute& Rotation, const FAnimDatabaseFrameAttribute& Vector);

	/** Unrotates a vector frame attribute by the given rotation frame attribute */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeUnrotate(const FAnimDatabaseFrameAttribute& Rotation, const FAnimDatabaseFrameAttribute& Vector);

	/** Rotates a vector frame attribute by the given rotation */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeQuatRotate(const FQuat& Rotation, const FAnimDatabaseFrameAttribute& Vector);

	/** Unrotates a vector frame attribute by the given rotation */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeQuatUnrotate(const FQuat& Rotation, const FAnimDatabaseFrameAttribute& Vector);


	/** Rotates the given constant direction by the rotation frame attribute */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeRotateDirection(const FAnimDatabaseFrameAttribute& Rotation, const FVector Direction = FVector::ForwardVector);

	/** Unrotates the given constant direction by the rotation frame attribute */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeUnrotateDirection(const FAnimDatabaseFrameAttribute& Rotation, const FVector Direction = FVector::ForwardVector);

	/** Compute the rotation between the given direction frame attribute and the constant direction provided */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeRotationBetweenDirection(const FAnimDatabaseFrameAttribute& Directions, const FVector Direction = FVector::ForwardVector);

	/** Rotates and Adds a vector frame attribute by the given addition and rotation frame attributes */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeRotateAndAdd(const FAnimDatabaseFrameAttribute& Vector, const FAnimDatabaseFrameAttribute& Rotation, const FAnimDatabaseFrameAttribute& Addition);

	/** Subtracts and Unrotates a vector frame attribute by the given subtract and rotation frame attributes */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeSubtractAndUnrotate(const FAnimDatabaseFrameAttribute& Vector, const FAnimDatabaseFrameAttribute& Subtraction, const FAnimDatabaseFrameAttribute& Rotation);

	/** Rotates multiple frame attributes by a rotation frame attribute. Assumes types are compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API void FrameAttributesRotate(TArray<FAnimDatabaseFrameAttribute>& OutVectors, const FAnimDatabaseFrameAttribute& Rotation, const TArray<FAnimDatabaseFrameAttribute>& InVectors);
	static UE_API void FrameAttributesRotateArrayView(const TArrayView<FAnimDatabaseFrameAttribute> OutVectors, const FAnimDatabaseFrameAttribute& Rotation, const TArrayView<const FAnimDatabaseFrameAttribute> InVectors);

	/** Unrotates multiple frame attributes by a rotation frame attribute. Assumes types are compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API void FrameAttributesUnrotate(TArray<FAnimDatabaseFrameAttribute>& OutVectors, const FAnimDatabaseFrameAttribute& Rotation, const TArray<FAnimDatabaseFrameAttribute>& InVectors);
	static UE_API void FrameAttributesUnrotateArrayView(const TArrayView<FAnimDatabaseFrameAttribute> OutVectors, const FAnimDatabaseFrameAttribute& Rotation, const TArrayView<const FAnimDatabaseFrameAttribute> InVectors);

	/** Rotates and adds multiple frame attributes by rotation and addition frame attributes. Assumes types are compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API void FrameAttributesRotateAndAdd(TArray<FAnimDatabaseFrameAttribute>& Out, const TArray<FAnimDatabaseFrameAttribute>& FrameAttributes, const FAnimDatabaseFrameAttribute& Rotation, const FAnimDatabaseFrameAttribute& Addition);
	static UE_API void FrameAttributesRotateAndAddArrayView(const TArrayView<FAnimDatabaseFrameAttribute> Out, const TArrayView<const FAnimDatabaseFrameAttribute> FrameAttributes, const FAnimDatabaseFrameAttribute& Rotation, const FAnimDatabaseFrameAttribute& Addition);

	/** Subtracts and unrotates multiple frame attributes by subtraction and rotation frame attributes. Assumes types are compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API void FrameAttributesSubtractAndUnrotate(TArray<FAnimDatabaseFrameAttribute>& Out, const TArray<FAnimDatabaseFrameAttribute>& FrameAttributes, const FAnimDatabaseFrameAttribute& Subtraction, const FAnimDatabaseFrameAttribute& Rotation);
	static UE_API void FrameAttributesSubtractAndUnrotateArrayView(const TArrayView<FAnimDatabaseFrameAttribute> Out, const TArrayView<const FAnimDatabaseFrameAttribute> FrameAttributes, const FAnimDatabaseFrameAttribute& Subtraction, const FAnimDatabaseFrameAttribute& Rotation);

	/** Converts a rotation frame attribute into an angular velocity frame attribute. */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeToRotationVector(const FAnimDatabaseFrameAttribute& A);

	/** Converts an angular velocity frame attribute into a rotation frame attribute. */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeFromRotationVector(const FAnimDatabaseFrameAttribute& A);

public:

	/** Extracts the X component of a frame attribute. Assumes type is subscriptable */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeX(const FAnimDatabaseFrameAttribute& A);

	/** Extracts the Y component of a frame attribute. Assumes type is subscriptable */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeY(const FAnimDatabaseFrameAttribute& A);

	/** Extracts the Z component of a frame attribute. Assumes type is subscriptable */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeZ(const FAnimDatabaseFrameAttribute& A);

	/** Extracts the W component of a frame attribute. Assumes type is subscriptable */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeW(const FAnimDatabaseFrameAttribute& A);

	/** Gets the Location part of a Transform frame attribute */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeTransformLocation(const FAnimDatabaseFrameAttribute& Transform);

	/** Gets the Rotation part of a Transform frame attribute */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeTransformRotation(const FAnimDatabaseFrameAttribute& Transform);

	/** Gets the Scale part of a Transform frame attribute */
	UFUNCTION(BlueprintPure = true, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeTransformScale(const FAnimDatabaseFrameAttribute& Transform);

	/** Computes the vector length of a frame attribute. Assumes type has a meaningful length */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "LENGTH"))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeLength(const FAnimDatabaseFrameAttribute& A);

	/** Projects a frame attribute with the given vector. Assumes type is compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeProject(const FAnimDatabaseFrameAttribute& A, const FVector Projection = FVector(1.0f, 1.0f, 0.0f));

	/** Extracts the Pitch component of a rotation frame attribute. */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributePitch(const FAnimDatabaseFrameAttribute& A);

	/** Extracts the Yaw component of a rotation frame attribute. */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeYaw(const FAnimDatabaseFrameAttribute& A);

	/** Extracts the Roll component of a rotation frame attribute. */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeRoll(const FAnimDatabaseFrameAttribute& A);

public:

	/** Returns a bool frame attribute describing if two frame attributes are numerically equal on each frame */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CommutativeAssociativeBinaryOperator, CompactNodeTitle = "=="))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeEqual(FAnimDatabaseFrameAttribute A, FAnimDatabaseFrameAttribute B);

	/** Returns a bool frame attribute describing if the given transform attribute is equal to the provided transform on each frame */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "=="))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeEqualTransform(const FAnimDatabaseFrameAttribute& TransformFrameAttribute, const FTransform& Transform);

	/** Compares if a float frame attribute is greater than the given threshold and returns a bool frame attribute */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = ">"))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeGreaterThanFloat(const FAnimDatabaseFrameAttribute& FloatFrameAttribute, const float Threshold);

	/** Compares if a float frame attribute is less than the given threshold and returns a bool frame attribute */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "<"))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeLessThanFloat(const FAnimDatabaseFrameAttribute& FloatFrameAttribute, const float Threshold);

	/** Compares if a angle frame attribute is greater than the given threshold in degrees and returns a bool frame attribute */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = ">"))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeGreaterThanAngleDegrees(const FAnimDatabaseFrameAttribute& AngleFrameAttribute, const float Threshold);

	/** Compares if a angle frame attribute is less than the given threshold in degrees and returns a bool frame attribute */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "<"))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeLessThanAngleDegrees(const FAnimDatabaseFrameAttribute& AngleFrameAttribute, const float Threshold);

	/** Compares if a angle frame attribute is greater than the given threshold in radians and returns a bool frame attribute */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = ">"))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeGreaterThanAngleRadians(const FAnimDatabaseFrameAttribute& AngleFrameAttribute, const float Threshold);

	/** Compares if a angle frame attribute is less than the given threshold in radians and returns a bool frame attribute */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "<"))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeLessThanAngleRadians(const FAnimDatabaseFrameAttribute& AngleFrameAttribute, const float Threshold);

	/** Computes the logical not of a bool frame attribute */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "NOT"))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeNot(const FAnimDatabaseFrameAttribute& A);

	/** Computes the logical and of two bool frame attributes */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CommutativeAssociativeBinaryOperator, CompactNodeTitle = "AND"))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeAnd(FAnimDatabaseFrameAttribute A, FAnimDatabaseFrameAttribute B);

	/** Computes the logical or of two bool frame attributes */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CommutativeAssociativeBinaryOperator, CompactNodeTitle = "OR"))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeOr(FAnimDatabaseFrameAttribute A, FAnimDatabaseFrameAttribute B);

	/** Converts a bool frame attribute to a float frame attribute */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeBoolToFloat(const FAnimDatabaseFrameAttribute& BoolFrameAttribute);

	/** Converts an angle frame attribute to a float frame attribute in degrees */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeAngleToFloatDegrees(const FAnimDatabaseFrameAttribute& AngleFrameAttribute);

	/** Converts an angle frame attribute to a float frame attribute in radians */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeAngleToFloatRadians(const FAnimDatabaseFrameAttribute& AngleFrameAttribute);

	/** Converts a float frame attribute to a bool frame attribute */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeFloatToBool(const FAnimDatabaseFrameAttribute& FloatFrameAttribute);

	/** Converts a float frame attribute to an angle frame attribute in degrees */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeFloatToAngleDegrees(const FAnimDatabaseFrameAttribute& FloatFrameAttribute);

	/** Converts a float frame attribute to an angle frame attribute in radians */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeFloatToAngleRadians(const FAnimDatabaseFrameAttribute& FloatFrameAttribute);

	/** Converts a direction frame attribute to a location frame attribute */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeDirectionToLocation(const FAnimDatabaseFrameAttribute& DirectionFrameAttribute);

	/** Converts a location frame attribute to a direction frame attribute (normalizing the result) */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeLocationToDirection(const FAnimDatabaseFrameAttribute& LocationFrameAttribute);

	/** Converts a velocity frame attribute to a direction frame attribute (normalizing the result) */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeVelocityToDirection(const FAnimDatabaseFrameAttribute& VelocityFrameAttribute);

public:

	/** Extracts a phase angle from two sets of frames representing the start of the phase and half way through the phase */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeExtractPhase(const FAnimDatabaseFrameRanges& FrameRanges, const FAnimDatabaseFrames& ZeroPhaseFrames, const FAnimDatabaseFrames& HalfPhaseFrames, const EAnimDatabasePhaseExtrapolationMode ExtrapolationMode = EAnimDatabasePhaseExtrapolationMode::Repeat);

public:

	/** Applies a Gaussian smoothing filter to the values in each range of a frame attribute. Assumes type is compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeFilterGaussian(const FAnimDatabaseFrameAttribute& FloatFrameAttribute, const float StandardDeviationInFrames = 3.0f);

	/** Applies a SavGol filter to the values in each range of a frame attribute. Assumes type is compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeFilterSavGol(const FAnimDatabaseFrameAttribute& FloatFrameAttribute, const int32 FilterWidthInFrames = 7, const int32 PolynomialDegree = 3, const bool bGaussianWindowed = true);

	/** Applies a majority vote filter to the values in each range of a frame attribute. Assumes type is compatible */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeFilterMajorityVote(const FAnimDatabaseFrameAttribute& BoolFrameAttribute, const int32 FilterWidthInFrames = 5);

public:

	/** Computes the mean and std of a location frame attribute */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API void FrameAttributeLocationMeanAndStd(FVector& OutMean, FVector& OutStd, const FAnimDatabaseFrameAttribute& LocationFrameAttribute);

	/** Computes the average of a float frame attribute */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API void FrameAttributeFloatAverage(float& OutAverageFloat, const FAnimDatabaseFrameAttribute& FloatFrameAttribute);

	/** Computes the average of an angle frame attribute in degrees */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API void FrameAttributeAngleAverageDegrees(float& OutAverageAngle, const FAnimDatabaseFrameAttribute& AngleFrameAttribute);

	/** Computes the average of an angle frame attribute in radians */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API void FrameAttributeAngleAverageRadians(float& OutAverageAngle, const FAnimDatabaseFrameAttribute& AngleFrameAttribute);

	/** Computes the min and max of a float frame attribute */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API void FrameAttributeFloatMinAndMax(float& OutMin, float& OutMax, const FAnimDatabaseFrameAttribute& FloatFrameAttribute);

	/** Computes the min and max of an angle frame attribute in degrees */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API void FrameAttributeAngleMinAndMaxDegrees(float& OutMin, float& OutMax, const FAnimDatabaseFrameAttribute& AngleFrameAttribute);

	/** Computes the min and max of an angle frame attribute in radians */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API void FrameAttributeAngleMinAndMaxRadians(float& OutMin, float& OutMax, const FAnimDatabaseFrameAttribute& AngleFrameAttribute);

	/** Computes the average of a location frame attribute */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API void FrameAttributeLocationAverage(FVector& OutAverageLocation, const FAnimDatabaseFrameAttribute& LocationFrameAttribute);

	/** Computes the average of a direction frame attribute */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API void FrameAttributeDirectionAverage(FVector& OutAverageDirection, const FAnimDatabaseFrameAttribute& DirectionFrameAttribute);

	/** Computes the mean and std (represented as an angular velocity) of a rotation frame attribute */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API void FrameAttributeRotationMeanAndStd(FRotator& OutMean, FVector& OutStd, const FAnimDatabaseFrameAttribute& RotationFrameAttribute);
	static UE_API void FrameAttributeRotationMeanAndStdAsQuat(FQuat& OutMean, FVector& OutStd, const FAnimDatabaseFrameAttribute& RotationFrameAttribute);

	/** Computes the mean and log std of a scale frame attribute */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API void FrameAttributeScaleMeanAndStd(FVector& OutMean, FVector& OutLogStd, const FAnimDatabaseFrameAttribute& ScaleFrameAttribute);

	/** Computes the mean and std of a velocity frame attribute */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API void FrameAttributeVelocityMeanAndStd(FVector& OutMean, FVector& OutStd, const FAnimDatabaseFrameAttribute& VelocityFrameAttribute);

	/** Computes the mean, std, min, and max distance traveled across each range in a location frame attribute */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API void FrameAttributeLocationDistanceTraveled(float& OutMean, float& OutStd, float& OutMin, float& OutMax, const FAnimDatabaseFrameAttribute& LocationFrameAttribute);

	/** Computes the mean, std, min, and max angle traveled (in degrees) across each range in a rotation frame attribute */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API void FrameAttributeRotationAngleTraveled(float& OutMean, float& OutStd, float& OutMin, float& OutMax, const FAnimDatabaseFrameAttribute& RotationFrameAttribute);

public:

	/** Converts a FrameAttribute object to a string. */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (BlueprintAutocast, CompactNodeTitle = "->", AutoCreateRefTerm = "FrameAttribute", Keywords="Print,Log,Display,Name"))
	static UE_API FString FrameAttributeToString(const FAnimDatabaseFrameAttribute& FrameAttribute);

	/** Converts a FrameAttribute object to a string object in full. */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta= (Keywords = "Print,Log,Display,Name"))
	static UE_API FString FrameAttributeToStringFormat(const UAnimDatabase* Database, const FAnimDatabaseFrameAttribute& FrameAttribute, const int32 Cutoff = 11);


public:

	/**
	 * Computes frame ranges across the given set of range CheckFrameRanges which may contain outliers or glitches in the character pose.
	 * 
	 * @param Database							Database to check for outliers in
	 * @param FrameRanges						Ranges in the database to check for outliers
	 * @param Padding							Padding to apply to returned outlier ranges in frames
	 * @param Threshold							Number of multiples of the standard deviation away from the mean to treat as an outlier
	 * @param LocationMinimumDifference			Minimum difference away from the mean (in cm) for bone locations to treat as outliers
	 * @param RotationMinimumDifference			Minimum difference away from the mean (in degrees) for bone rotations to treat as outliers
	 * @param ScaleMinimumDifference			Minimum difference away from the mean for bone scales to treat as outliers
	 * @param LinearVelocityMinimumDifference	Minimum difference away from the mean (in cm/s) for bone linear velocities to treat as outliers
	 * @param AngularVelocityMinimumDifference	Minimum difference away from the mean (in deg/s) for bone angular velocities to treat as outliers
	 * @param ScalarVelocityMinimumDifference	Minimum difference away from the mean for bone scalar velocities to treat as outliers
	 */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API FAnimDatabaseFrameRanges FrameAttributeOutlierFrameRanges(
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const int32 Padding = 60,
		const float Threshold = 10.0f,
		const float LocationMinimumDifference = 5.0f,
		const float RotationMinimumDifference = 5.0f,
		const float ScaleMinimumDifference = 0.1f,
		const float LinearVelocityMinimumDifference = 10.0f,
		const float AngularVelocityMinimumDifference = 10.0f,
		const float ScalarVelocityMinimumDifference = 0.2f);

	/**
	 * Computes a root transform by projecting bones onto the ground plane and smoothing the resulting location and direction with a SavGol filter.
	 *
	 * @param Database							Database to use
	 * @param FrameRanges						Ranges in the database to use
	 * @param RootLocationProjectionBoneName	Name of the bone to project onto the ground plane and use as the root location
	 * @param RootRotationProjectionBoneName	Name of the bone to project onto the ground plane and use as the root rotation
	 * @param RootRotationBoneForwardDirection	Local forward direction of the bone to use as the root rotation
	 * @param RootForwardDirection				Local forward direction of the desired root transform
	 * @param LocationSavGolFrameNum			Number of frames over which to apply the location savgol filter
	 * @param LocationSavGolPolynomialDegree	Degree of polynomial to use for the location savgol filter
	 * @param DirectionSavGolFrameNum			Number of frames over which to apply the location direction filter
	 * @param DirectionSavGolPolynomialDegree	Degree of polynomial to use for the direction savgol filter
	 * @param bGaussianWindowed					If to apply a gaussian weighted window to the savgol polynomial fitting
	 */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeSavGolSmoothedRootTransform(
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const FName RootLocationProjectionBoneName = TEXT("spine_02"),
		const FName RootRotationProjectionBoneName = TEXT("pelvis"),
		const FVector RootRotationBoneForwardDirection = FVector(0, 1, 0),
		const FVector RootForwardDirection = FVector(0, 1, 0),
		const int32 LocationSavGolFrameNum = 61,
		const int32 LocationSavGolPolynomialDegree = 3,
		const int32 DirectionSavGolFrameNum = 91,
		const int32 DirectionSavGolPolynomialDegree = 3,
		const bool bGaussianWindowed = true);
		
	/**
	 * Computes a root transform by projecting bones onto the ground plane and smoothing the resulting location and direction with a Gaussian filter.
	 *
	 * @param Database							Database to use
	 * @param FrameRanges						Ranges in the database to use
	 * @param RootLocationProjectionBoneName	Name of the bone to project onto the ground plane and use as the root location
	 * @param RootRotationProjectionBoneName	Name of the bone to project onto the ground plane and use as the root rotation
	 * @param RootRotationBoneForwardDirection	Local forward direction of the bone to use as the root rotation
	 * @param RootForwardDirection				Local forward direction of the desired root transform
	 * @param LocationSmoothingAmount			Amount of smoothing to apply to the root location projection (in frames)
	 * @param RotationSmoothingAmount			Amount of smoothing to apply to the root rotation projection (in frames)
	 */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeGaussianSmoothedRootTransform(
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const FName RootLocationProjectionBoneName = TEXT("spine_02"),
		const FName RootRotationProjectionBoneName = TEXT("pelvis"),
		const FVector RootRotationBoneForwardDirection = FVector(0, 1, 0),
		const FVector RootForwardDirection = FVector(0, 1, 0),
		const float LocationSmoothingAmount = 9.0f,
		const float RotationSmoothingAmount = 15.0f);

	/**
	 * Computes a float attribute representing the global speed of a given joint.
	 *
	 * @param Database							Database to use
	 * @param FrameRanges						Ranges in the database to use
	 * @param BoneName							Name of the bone to compute the speed of
	 */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeBoneSpeed(
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const FName BoneName = TEXT("foot_l"));

	/**
	 * Computes a float attribute representing a contact state for the given joint.
	 *
	 * @param Database							Database to use
	 * @param FrameRanges						Ranges in the database to use
	 * @param BoneName							Name of the bone to compute the contact state for
	 * @param HeightThreshold					Maximum height (in cm) for which the bone can be considered in contact
	 * @param VelocityThreshold					Maximum velocity (in cm/s) for which the bone can be considered in contact
	 * @param bFilterCurve						If to apply the majority vote filter to the curve
	 * @param MajorityVoteFilterWidth			Filter width (in frames) for the majority vote filter used to remove very short contact transitions.
	 *											contact periods (or non-contact periods) less than half this duration will be removed.
	 * @param bSmoothCurve						If to apply the smoothing filter to the curve
	 * @param SmoothingAmount					Amount of smoothing to apply to the resulting contact curve (in frames)
	 */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeContactCurve(
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const FName BoneName = TEXT("ball_l"),
		const float HeightThreshold = 20.0f,
		const float VelocityThreshold = 25.0f,
		const bool bFilterCurve = true,
		const int32 MajorityVoteFilterWidth = 5,
		const bool bSmoothCurve = true,
		const float SmoothingAmount = 3.0f);

	/**
	 * Computes a frame ranges object representing the contact down times for the given joint.
	 *
	 * @param Database							Database to use
	 * @param FrameRanges						Ranges in the database to use
	 * @param BoneName							Name of the bone to compute the contact state for
	 * @param HeightThreshold					Maximum height (in cm) for which the bone can be considered in contact
	 * @param VelocityThreshold					Maximum velocity (in cm/s) for which the bone can be considered in contact
	 * @param bFilterCurve						If to apply the majority vote filter to the curve
	 * @param MajorityVoteFilterWidth			Filter width (in frames) for the majority vote filter used to remove very short contact transitions.
	 *											contact periods (or non-contact periods) less than half this duration will be removed.
	 */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API FAnimDatabaseFrameRanges FrameAttributeContactRanges(
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const FName BoneName = TEXT("ball_l"),
		const float HeightThreshold = 20.0f,
		const float VelocityThreshold = 25.0f,
		const bool bFilterCurve = true,
		const int32 MajorityVoteFilterWidth = 5);

	/**
	 * Computes a frames object representing the contact down times for the given joint.
	 *
	 * @param Database							Database to use
	 * @param FrameRanges						Ranges in the database to use
	 * @param BoneName							Name of the bone to compute the contact state for
	 * @param HeightThreshold					Maximum height (in cm) for which the bone can be considered in contact
	 * @param VelocityThreshold					Maximum velocity (in cm/s) for which the bone can be considered in contact
	 * @param bFilterCurve						If to apply the majority vote filter to the curve
	 * @param MajorityVoteFilterWidth			Filter width (in frames) for the majority vote filter used to remove very short contact transitions.
	 *											contact periods (or non-contact periods) less than half this duration will be removed.
	 * @param FrameOffset						The amount of time before the contact (in seconds) to have the frame
	 * @param bIgnoreFirstFrame					Removes contact frames that occur on the first frame of the frame ranges
	 */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API FAnimDatabaseFrames FrameAttributeContactFrames(
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const FName BoneName = TEXT("ball_l"),
		const float HeightThreshold = 20.0f,
		const float VelocityThreshold = 25.0f,
		const bool bFilterCurve = true,
		const int32 MajorityVoteFilterWidth = 5,
		const float FrameOffset = 0.0f,
		const bool bIgnoreFirstFrame = true);

	/**
	 * Fits a transform from the given location and direction attributes
	 *
	 * @param Database							Database to use
	 * @param FrameRanges						Ranges in the database to use
	 * @param FitLocations						Array of attributes for computing the locations to use for the fit
	 * @param FitForwardDirections				Array of attributes for computing the forward directions to use for the fit
	 * @param FitRightDirections				Array of attributes for computing the right directions to use for the fit
	 * @param FitUpDirections					Array of attributes for computing the up directions to use for the fit
	 * @param FitLocationWeights				Optional array of weights to use for the fit locations
	 * @param FitForwardDirectionWeights		Optional array of weights to use for the fit forward directions
	 * @param FitRightDirectionWeights			Optional array of weights to use for the fit right directions
	 * @param FitUpDirectionWeights				Optional array of weights to use for the fit up directions
	 */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, AutoCreateRefTerm="FitLocationWeights, FitForwardDirectionWeights, FitRightDirectionWeights, FitUpDirectionWeights"))
	static UE_API FAnimDatabaseFrameAttribute FrameAttributeFitTransform(
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const TArray<FAnimDatabaseFrameAttribute>& FitLocations,
		const TArray<FAnimDatabaseFrameAttribute>& FitForwardDirections,
		const TArray<FAnimDatabaseFrameAttribute>& FitRightDirections,
		const TArray<FAnimDatabaseFrameAttribute>& FitUpDirections,
		const TArray<FAnimDatabaseFrameAttribute>& FitLocationWeights,
		const TArray<FAnimDatabaseFrameAttribute>& FitForwardDirectionWeights,
		const TArray<FAnimDatabaseFrameAttribute>& FitRightDirectionWeights,
		const TArray<FAnimDatabaseFrameAttribute>& FitUpDirectionWeights);

	static UE_API FAnimDatabaseFrameAttribute FrameAttributeFitTransformFromArrayViews(
		const UAnimDatabase* Database,
		const FAnimDatabaseFrameRanges& FrameRanges,
		const TArrayView<const FAnimDatabaseFrameAttribute> FitLocations,
		const TArrayView<const FAnimDatabaseFrameAttribute> FitForwardDirections,
		const TArrayView<const FAnimDatabaseFrameAttribute> FitRightDirections,
		const TArrayView<const FAnimDatabaseFrameAttribute> FitUpDirections,
		const TArrayView<const FAnimDatabaseFrameAttribute> FitLocationWeights,
		const TArrayView<const FAnimDatabaseFrameAttribute> FitForwardDirectionWeights,
		const TArrayView<const FAnimDatabaseFrameAttribute> FitRightDirectionWeights,
		const TArrayView<const FAnimDatabaseFrameAttribute> FitUpDirectionWeights);
};

/** A simple class that has a single implementable function which constructs a FAnimDatabaseFrameAttribute object from a UAnimDatabase. */
UCLASS(Abstract, HideDropdown, EditInlineNew, DefaultToInstanced, CollapseCategories, BlueprintType, Blueprintable)
class UAnimDatabaseFrameAttributeFunction : public UObject
{
	GENERATED_BODY()

public:

	/** Callback that constructs a FrameAttribute from a UAnimDatabase */
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "AnimDatabase")
	UE_API FAnimDatabaseFrameAttribute MakeFrameAttribute(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const;
};

/** Computes a bool Frame Attribute for the given Frame Ranges  */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Frame Ranges Frame Attribute"))
class UAnimDatabaseFrameAttributeFunction_FrameRanges : public UAnimDatabaseFrameAttributeFunction
{
	GENERATED_BODY()

public:

	/** Frame Ranges to use */
	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TObjectPtr<UAnimDatabaseFrameRangesFunction> FrameRangesFunction;

public:

	UE_API virtual FAnimDatabaseFrameAttribute MakeFrameAttribute_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Computes an Attribute for the given Frame Ranges  */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Filter Frame Attribute"))
class UAnimDatabaseFrameAttributeFunction_Filter : public UAnimDatabaseFrameAttributeFunction
{
	GENERATED_BODY()

public:

	/** Frame Ranges to filter to */
	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TObjectPtr<UAnimDatabaseFrameRangesFunction> FrameRangesFunction;

	/** Frame Attribute to create */
	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TObjectPtr<UAnimDatabaseFrameAttributeFunction> FrameAttributeFunction;

public:

	UE_API virtual FAnimDatabaseFrameAttribute MakeFrameAttribute_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Finds all the potential outlier frame ranges in a database */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Outliers Frame Attribute"))
class UAnimDatabaseFrameRangesFunction_Outliers : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	/** Padding to apply to returned outlier ranges in frames */
	UPROPERTY(EditAnywhere, Category = "Settings")
	int32 Padding = 60;
	
	/** Number of multiples of the standard deviation away from the mean to treat as an outlier */
	UPROPERTY(EditAnywhere, Category = "Settings")
	float Threshold = 10.0f;
	
	/** Minimum difference away from the mean (in cm) for bone locations to treat as outliers */
	UPROPERTY(EditAnywhere, Category = "Settings")
	float LocationMinimumDifference = 5.0f;
	
	/** Minimum difference away from the mean (in degrees) for bone rotations to treat as outliers */
	UPROPERTY(EditAnywhere, Category = "Settings")
	float RotationMinimumDifference = 5.0f;
	
	/** Minimum difference away from the mean for bone scales to treat as outliers */
	UPROPERTY(EditAnywhere, Category = "Settings")
	float ScaleMinimumDifference = 0.1f;
	
	/** Minimum difference away from the mean (in cm/s) for bone linear velocities to treat as outliers */
	UPROPERTY(EditAnywhere, Category = "Settings")
	float LinearVelocityMinimumDifference = 10.0f;
	
	/** Minimum difference away from the mean (in deg/s) for bone angular velocities to treat as outliers */
	UPROPERTY(EditAnywhere, Category = "Settings")
	float AngularVelocityMinimumDifference = 10.0f;

	/** Minimum difference away from the mean for bone scalar velocities to treat as outliers */
	UPROPERTY(EditAnywhere, Category = "Settings")
	float ScalarVelocityMinimumDifference = 0.2f;

	/** Ranges that are excluded from the outlier detection */
	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TObjectPtr<UAnimDatabaseFrameRangesFunction> ExcludedRanges;

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Computes a contact curve attribute */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Contact Curve Frame Attribute"))
class UAnimDatabaseFrameAttributeFunction_ContactCurve : public UAnimDatabaseFrameAttributeFunction
{
	GENERATED_BODY()

public:

	/** Name of the bone to compute the contact state for */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FName BoneName = NAME_None;

	/** Maximum height (in cm) for which the bone can be considered in contact */
	UPROPERTY(EditAnywhere, Category = "Settings")
	float HeightThreshold = 20.0f;

	/** Maximum velocity (in cm/s) for which the bone can be considered in contact */
	UPROPERTY(EditAnywhere, Category = "Settings")
	float VelocityThreshold = 25.0f;

	/** If to filter the curve */
	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bFilterCurve = true;

	/**
	 * Filter width (in frames) for the majority vote filter used to remove very short contact transitions.
	 *Contact periods (or non-contact periods) less than half this duration will be removed.
	 */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (EditCondition = "bFilterCurve", HideEditConditionToggle))
	int32 MajorityVoteFilterWidth = 5;

	/** If to smooth the curve */
	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bSmoothCurve = true;

	/* Amount of smoothing to apply to the resulting contact curve (in frames) */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (EditCondition = "bSmoothCurve", HideEditConditionToggle))
	float SmoothingAmount = 3.0f;

public:

	UE_API virtual FAnimDatabaseFrameAttribute MakeFrameAttribute_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Computes contact frames */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Contact Frames Frame Attribute"))
class UAnimDatabaseFramesFunction_ContactFrames : public UAnimDatabaseFramesFunction
{
	GENERATED_BODY()

public:

	/** Name of the bone to compute the contact state for */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FName BoneName = NAME_None;

	/** Maximum height (in cm) for which the bone can be considered in contact */
	UPROPERTY(EditAnywhere, Category = "Settings")
	float HeightThreshold = 20.0f;

	/** Maximum velocity (in cm/s) for which the bone can be considered in contact */
	UPROPERTY(EditAnywhere, Category = "Settings")
	float VelocityThreshold = 25.0f;

	/** If to filter the curve */
	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bFilterCurve = true;

	/**
	 * Filter width (in frames) for the majority vote filter used to remove very short contact transitions.
	 *Contact periods (or non-contact periods) less than half this duration will be removed.
	 */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (EditCondition = "bFilterCurve", HideEditConditionToggle))
	int32 MajorityVoteFilterWidth = 5;

	/** The amount of time before the contact (in seconds) to have the event */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ForceUnits = "s"))
	float EventOffset = 0.0f;

	/** If provided, will be used to filter the contact events */
	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TObjectPtr<UAnimDatabaseFrameRangesFunction> EventFilter;

public:

	UE_API virtual FAnimDatabaseFrames MakeFrames_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Computes a contact event attribute */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Contact Event Frame Attribute"))
class UAnimDatabaseFrameAttributeFunction_ContactEvent : public UAnimDatabaseFrameAttributeFunction
{
	GENERATED_BODY()

public:

	/** Name of the bone to compute the contact state for */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FName BoneName = NAME_None;

	/** Maximum height (in cm) for which the bone can be considered in contact */
	UPROPERTY(EditAnywhere, Category = "Settings")
	float HeightThreshold = 20.0f;

	/** Maximum velocity (in cm/s) for which the bone can be considered in contact */
	UPROPERTY(EditAnywhere, Category = "Settings")
	float VelocityThreshold = 25.0f;

	/** If to filter the curve */
	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bFilterCurve = true;

	/**
	 * Filter width (in frames) for the majority vote filter used to remove very short contact transitions.
	 *Contact periods (or non-contact periods) less than half this duration will be removed.
	 */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (EditCondition = "bFilterCurve", HideEditConditionToggle))
	int32 MajorityVoteFilterWidth = 5;

	/** The amount of time before the contact (in seconds) to have the event */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ForceUnits = "s"))
	float EventOffset = 0.0f;

	/** If provided, will be used to filter the contact events */
	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TObjectPtr<UAnimDatabaseFrameRangesFunction> EventFilter;

public:

	UE_API virtual FAnimDatabaseFrameAttribute MakeFrameAttribute_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Fits a transform to the given locations and directions */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Fit Transform Frame Attribute"))
class UAnimDatabaseFrameAttributeFunction_FitTransform : public UAnimDatabaseFrameAttributeFunction
{
	GENERATED_BODY()

public:

	/** Array of attributes for computing the locations to use for the fit */
	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TArray<TObjectPtr<UAnimDatabaseFrameAttributeFunction>> FitLocations;

	/** Array of attributes for computing the forward directions to use for the fit */
	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TArray<TObjectPtr<UAnimDatabaseFrameAttributeFunction>> FitForwardDirections;

	/** Array of attributes for computing the right directions to use for the fit */
	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TArray<TObjectPtr<UAnimDatabaseFrameAttributeFunction>> FitRightDirections;

	/** Array of attributes for computing the up directions to use for the fit */
	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TArray<TObjectPtr<UAnimDatabaseFrameAttributeFunction>> FitUpDirections;

	/** Optional array of weights to use for the fit locations */
	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TArray<TObjectPtr<UAnimDatabaseFrameAttributeFunction>> FitLocationWeights;

	/** Optional array of weights to use for the fit forward directions */
	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TArray<TObjectPtr<UAnimDatabaseFrameAttributeFunction>>FitForwardDirectionWeights;

	/** Optional array of weights to use for the fit right directions */
	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TArray<TObjectPtr<UAnimDatabaseFrameAttributeFunction>> FitRightDirectionWeights;

	/** Optional array of weights to use for the fit up directions */
	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TArray<TObjectPtr<UAnimDatabaseFrameAttributeFunction>> FitUpDirectionWeights;

public:

	UE_API virtual FAnimDatabaseFrameAttribute MakeFrameAttribute_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Computes a bone global transform attribute */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Bone Global Transform Frame Attribute"))
class UAnimDatabaseFrameAttributeFunction_BoneGlobalTransform : public UAnimDatabaseFrameAttributeFunction
{
	GENERATED_BODY()

public:

	/** Name of the bone to compute the transform of */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FName BoneName = NAME_None;

	/** Local offset transform */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FTransform LocalOffset = FTransform::Identity;

public:

	UE_API virtual FAnimDatabaseFrameAttribute MakeFrameAttribute_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Computes a bone global transform attribute relative to another  */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Bone Relative Transform Frame Attribute"))
class UAnimDatabaseFrameAttributeFunction_BoneRelativeTransform : public UAnimDatabaseFrameAttributeFunction
{
	GENERATED_BODY()

public:

	/** Name of the bone to compute the transform of */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FName BoneName = NAME_None;

	/** Name of the bone to compute the transform relative to */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FName RelativeBoneName = NAME_None;

public:

	UE_API virtual FAnimDatabaseFrameAttribute MakeFrameAttribute_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Computes a bone global angular velocity attribute relative to another  */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Bone Relative Angular Velocity Frame Attribute"))
class UAnimDatabaseFrameAttributeFunction_BoneRelativeAngularVelocity : public UAnimDatabaseFrameAttributeFunction
{
	GENERATED_BODY()

public:

	/** Name of the bone to compute the angular velocity of */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FName BoneName = NAME_None;

	/** Name of the bone to compute the angular velocity relative to */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FName RelativeBoneName = NAME_None;

public:

	UE_API virtual FAnimDatabaseFrameAttribute MakeFrameAttribute_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Computes a bone global location attribute */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Bone Global Location Frame Attribute"))
class UAnimDatabaseFrameAttributeFunction_BoneGlobalLocation : public UAnimDatabaseFrameAttributeFunction
{
	GENERATED_BODY()

public:

	/** Name of the bone to compute the location of */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FName BoneName = NAME_None;

	/** Local offset vector */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FVector LocalOffset = FVector::ZeroVector;

public:

	UE_API virtual FAnimDatabaseFrameAttribute MakeFrameAttribute_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Computes a bone global direction attribute */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Bone Global Direction Frame Attribute"))
class UAnimDatabaseFrameAttributeFunction_BoneGlobalDirection : public UAnimDatabaseFrameAttributeFunction
{
	GENERATED_BODY()

public:

	/** Name of the bone to compute the direction of */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FName BoneName = NAME_None;

	/** Local direction vector */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FVector LocalDirection = FVector::ForwardVector;

public:

	UE_API virtual FAnimDatabaseFrameAttribute MakeFrameAttribute_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Computes a bone velocity magnitude attribute */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Bone Global Linear Velocity Magnitude Frame Attribute"))
class UAnimDatabaseFrameAttributeFunction_BoneGlobalLinearVelocityMagnitude : public UAnimDatabaseFrameAttributeFunction
{
	GENERATED_BODY()

public:

	/** Name of the bone to compute the velocity magnitude for */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FName BoneName = NAME_None;

public:

	UE_API virtual FAnimDatabaseFrameAttribute MakeFrameAttribute_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Filter a frame attribute using a gaussian smoothing filter */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Gaussian Smoothing Filter Frame Attribute"))
class UAnimDatabaseFrameAttributeFunction_GaussianSmoothingFilter : public UAnimDatabaseFrameAttributeFunction
{
	GENERATED_BODY()

public:

	/** Frame attribute to filter */
	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TObjectPtr<UAnimDatabaseFrameAttributeFunction> FilterFrameAttribute;

	/** Standard Deviation (in seconds) for the gaussian smoothing filter. */
	UPROPERTY(EditAnywhere, Category = "Settings", meta=(ForceUnits="s"))
	float StandardDeviation = 3.0f;

public:

	UE_API virtual FAnimDatabaseFrameAttribute MakeFrameAttribute_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Filter frame ranges with a majority vote filter */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Majority Vote Filter Frame Attribute"))
class UAnimDatabaseFrameRangesFunction_MajorityVoteFilter : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	/** Frame ranges to filter */
	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TObjectPtr<UAnimDatabaseFrameRangesFunction> FilterFrameRanges;

	/**
	 * Filter width (in frames) for the majority vote filter used to remove very short frame ranges less than half this duration will be removed.
	 */
	UPROPERTY(EditAnywhere, Category = "Settings")
	int32 MajorityVoteFilterWidth = 15;

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Frame ranges where the character is moving in the given direction */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Moving in Direction Frame Attribute"))
class UAnimDatabaseFrameRangesFunction_MovingInDirection : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	/** Local direction to check if the character is moving in */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FVector Direction = FVector::RightVector;

	/** Character is only considered moving when the velocity is above this threshold */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm/s"))
	float VelocityThreshold = 25.0f;

	/** Character is considered moving in the direction when the angle between the velocity direction and the specified direction is less than this */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "deg"))
	float AngleThreshold = 45.0f;

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Frame ranges where the root linear velocity magnitude is below some threshold */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Root Linear Velocity Below Threshold Frame Attribute"))
class UAnimDatabaseFrameRangesFunction_RootLinearVelocityBelowThreshold : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm/s"))
	float Threshold = 10.0f;

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Frame ranges where the root linear velocity magnitude is above some threshold */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Root Linear Velocity Above Threshold Frame Attribute"))
class UAnimDatabaseFrameRangesFunction_RootLinearVelocityAboveThreshold : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm/s"))
	float Threshold = 10.0f;

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Entry specifying a frame attribute function */
USTRUCT(BlueprintType)
struct FAnimDatabaseFrameAttributeEntry
{
	GENERATED_BODY()

public:

	/** Frame Attribute name */
	UPROPERTY(EditAnywhere, Category = "Entry")
	FName Name = NAME_None;

	/** Frame Attribute function */
	UPROPERTY(EditAnywhere, Instanced, Category = "Entry")
	TObjectPtr<UAnimDatabaseFrameAttributeFunction> FrameAttribute;
};

UE_API bool operator==(const FAnimDatabaseFrameAttributeEntry& Lhs, const FAnimDatabaseFrameAttributeEntry& Rhs);

#undef UE_API