// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SubclassOf.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Misc/FrameRate.h"

#include "LearningArray.h"

#include "AnimDatabaseFrameRanges.generated.h"

#define UE_API ANIMDATABASE_API

class UAnimNotify;
class UAnimNotifyState;
class UAnimSequence;
class UAnimDatabase;
class UAnimDatabaseFramesFunction;
class UAnimDatabaseFrameRangesFunction;
struct FAnimDatabaseFrames;

namespace UE::Learning
{
	struct FFrameSet;
	struct FFrameRangeSet;
}

/**
 * Represents a set of frames within a UAnimDatabase.
 *
 * Internally, this is effectively a thin blueprint wrapper around the UE::Learning::FFrameSet data structure. Below we provide a Blueprint Function
 * Library that allows for the construction, modification and scripting of these objects in blueprints, but if you want to handle them in C++ at a
 * low level if you should access the internal FrameSet object directly.
 *
 * It is assumed that all FrameSet objects used by this wrapper are stored at the FrameRate of the database they were created from.
 */
USTRUCT(BlueprintType)
struct FAnimDatabaseFrames
{
	GENERATED_BODY()

	/** Checks if the given FrameSet is valid (i.e. the shared pointer is not null) */
	UE_API bool IsValid() const;

	/** Custom Serialization */
	UE_API bool Serialize(FArchive& Ar);

	/** Shared pointer to the actual FrameSet data structure */
	TSharedPtr<UE::Learning::FFrameSet, ESPMode::ThreadSafe> FrameSet;
};

UE_API bool operator==(const FAnimDatabaseFrames& Lhs, const FAnimDatabaseFrames& Rhs);

template<>
struct TStructOpsTypeTraits<FAnimDatabaseFrames> : TStructOpsTypeTraitsBase2<FAnimDatabaseFrames>
{
	enum
	{
		WithIdenticalViaEquality = true,
		WithSerializer = true
	};
};

/**
 * Represents a set of frame ranges within a UAnimDatabase.
 *
 * Internally, this is effectively a thin blueprint wrapper around the UE::Learning::FFrameRangeSet data structure. Below we provide a Blueprint 
 * Function Library that allows for the construction, modification and scripting of these objects in blueprints, but if you want to handle them in 
 * C++ at a low level if you can also access the internal FrameRangeSet object directly.
 *
 * It is assumed that all FrameRangeSet objects used by this wrapper are stored at the FrameRate of the database they were created from.
 */
USTRUCT(BlueprintType)
struct FAnimDatabaseFrameRanges
{
	GENERATED_BODY()

	/** Checks if the given FrameRangeSet is valid (i.e. the shared pointer is not null) */
	UE_API bool IsValid() const;

	/** Custom Serialization */
	UE_API bool Serialize(FArchive& Ar);

	/** Shared pointer to the actual FrameRangeSet data structure */
	TSharedPtr<UE::Learning::FFrameRangeSet, ESPMode::ThreadSafe> FrameRangeSet;
};

UE_API bool operator==(const FAnimDatabaseFrameRanges& Lhs, const FAnimDatabaseFrameRanges& Rhs);

template<>
struct TStructOpsTypeTraits<FAnimDatabaseFrameRanges> : TStructOpsTypeTraitsBase2<FAnimDatabaseFrameRanges>
{
	enum
	{
		WithIdenticalViaEquality = true,
		WithSerializer = true
	};
};

UENUM(BlueprintType)
enum class EAnimDatabaseFrameShiftBehavior : uint8
{
	// Remove frames which are shifted to beyond the start or end of an anim sequence
	Remove = 0,
	// Clamp frames which are shifted to beyond the start or end of an anim sequence to the start or end
	Clamp = 1,
};

UCLASS(BlueprintType, meta = (BlueprintThreadSafe))
class UAnimDatabaseFramesLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/** Make an empty set of Frames */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrames MakeEmptyFrames();

	/** Make a set of Frames using the given function */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API FAnimDatabaseFrames MakeFramesFromFunction(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const UAnimDatabaseFramesFunction* Function);

	/** Make a set of Frames using the given class */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API FAnimDatabaseFrames MakeFramesFromClass(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TSubclassOf<UAnimDatabaseFramesFunction> Class);

	/** Make a set of Frames from the times given by all the instances of the given AnimNotify class in the database */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API FAnimDatabaseFrames MakeFramesFromAnimNotify(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TSubclassOf<UAnimNotify> Notify);

	/** Make a set of Frames from the times given by all the instances of AnimNotifies in the database that are on a track with the given name */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API FAnimDatabaseFrames MakeFramesFromAnimNotifiesOnTrack(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName TrackName);

	/** Make a set of Frames from the times given by all the instances of the given SyncMarker in the database */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API FAnimDatabaseFrames MakeFramesFromSyncMarker(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName SyncMarkerName);

	/** Make a set of Frames from the start times of all the instances of the given UAnimNotifyState class in the database */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API FAnimDatabaseFrames MakeFramesFromAnimNotifyStateStart(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TSubclassOf<UAnimNotifyState> State);

	/** Make a set of Frames from the end times of all the instances of the given UAnimNotifyState class in the database */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API FAnimDatabaseFrames MakeFramesFromAnimNotifyStateEnd(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TSubclassOf<UAnimNotifyState> State);

	/** Make a set of Frames from all of the given anim notifies */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API FAnimDatabaseFrames MakeFramesFromAnimNotifyUnion(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArray<TSubclassOf<UAnimNotify>>& Notifies);
	static UE_API FAnimDatabaseFrames MakeFramesFromAnimNotifyArrayViewUnion(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const TSubclassOf<UAnimNotify>> Notifies);

	/** Make a set of Frames consisting of a single frame from the given Sequence Index in the Database, at the given time */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe))
	static UE_API FAnimDatabaseFrames MakeFramesFromSequenceAndTime(const UAnimDatabase* Database, const int32 SequenceIdx, const float Time);

public:

	/** Makes an empty set of frames. (This is the same as MakeEmptyFrames). */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "NONE"))
	static UE_API FAnimDatabaseFrames FramesNone();

	/** Check if the given set is empty */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API bool FramesIsEmpty(const FAnimDatabaseFrames& Frames);

	/** Make a set of Frames at a given number of frames before another set of frames. Will use ShiftBehavior to decided if to clamp or discard frames that fall out of the valid sequence range. */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, CompactNodeTitle = "BEFORE"))
	static UE_API FAnimDatabaseFrames FramesBefore(const UAnimDatabase* Database, const FAnimDatabaseFrames& Frames, const int32 FramesBefore, const EAnimDatabaseFrameShiftBehavior ShiftBehavior = EAnimDatabaseFrameShiftBehavior::Remove);

	/** Make a set of Frames at a given number of frames after another set of frames. Will use ShiftBehavior to decided if to clamp or discard frames that fall out of the valid sequence range. */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (NotBlueprintThreadSafe, CompactNodeTitle = "AFTER"))
	static UE_API FAnimDatabaseFrames FramesAfter(const UAnimDatabase* Database, const FAnimDatabaseFrames& Frames, const int32 FramesAfter, const EAnimDatabaseFrameShiftBehavior ShiftBehavior = EAnimDatabaseFrameShiftBehavior::Remove);

	/** Checks if two sets of Frames are equal */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "EQUAL"))
	static UE_API bool FramesEqual(const FAnimDatabaseFrames& A, const FAnimDatabaseFrames& B);

	/** Computes the union of two sets of Frames */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CommutativeAssociativeBinaryOperator, CompactNodeTitle = "UNION"))
	static UE_API FAnimDatabaseFrames FramesUnion(FAnimDatabaseFrames A, FAnimDatabaseFrames B);

	/** Computes the intersection of two sets of Frames */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CommutativeAssociativeBinaryOperator, CompactNodeTitle = "INTERSECTION"))
	static UE_API FAnimDatabaseFrames FramesIntersection(FAnimDatabaseFrames A, FAnimDatabaseFrames B);

	/** Computes the difference of two sets of Frames */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CommutativeAssociativeBinaryOperator, CompactNodeTitle = "DIFFERENCE"))
	static UE_API FAnimDatabaseFrames FramesDifference(FAnimDatabaseFrames A, FAnimDatabaseFrames B);

	/** Computes the union of an array of Frames objects */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "UNION"))
	static UE_API FAnimDatabaseFrames FramesUnionFromArray(const TArray<FAnimDatabaseFrames>& Frames);

	/** Computes the union of an array of Frames objects using an array view */
	static UE_API FAnimDatabaseFrames FramesUnionFromArrayView(const TArrayView<const FAnimDatabaseFrames> Frames);

	/** Computes the intersection of an array of Frames objects */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "INTERSECTION"))
	static UE_API FAnimDatabaseFrames FramesIntersectionFromArray(const TArray<FAnimDatabaseFrames>& Frames);

	/** Computes the intersection of an array of Frames objects using an array view */
	static UE_API FAnimDatabaseFrames FramesIntersectionFromArrayView(const TArrayView<const FAnimDatabaseFrames> Frames);

	/** Computes the difference of an array of Frames objects */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "DIFFERENCE"))
	static UE_API FAnimDatabaseFrames FramesDifferenceFromArray(const TArray<FAnimDatabaseFrames>& Frames);

	/** Computes the difference of an array of Frames objects using an array view */
	static UE_API FAnimDatabaseFrames FramesDifferenceFromArrayView(const TArrayView<const FAnimDatabaseFrames> Frames);

	/** Converts a Frames object to a string. */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (BlueprintAutocast, CompactNodeTitle = "->", AutoCreateRefTerm = "Frames", Keywords = "Print,Log,Display,Name"))
	static UE_API FString FramesToString(const FAnimDatabaseFrames& Frames);

	/** Converts a Frames object to a string object in full. */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (Keywords = "Print,Log,Display,Name"))
	static UE_API FString FramesToStringFormat(const UAnimDatabase* Database, const FAnimDatabaseFrames& Frames, const int32 Cutoff = 11);
};

/** A simple class that has a single implementable function which constructs a FAnimDatabaseFrames object from a UAnimDatabase. */
UCLASS(Abstract, HideDropdown, EditInlineNew, DefaultToInstanced, CollapseCategories, BlueprintType, Blueprintable)
class UAnimDatabaseFramesFunction : public UObject
{
	GENERATED_BODY()

public:

	/** Callback that constructs a Frames set from a UAnimDatabase */
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "AnimDatabase")
	UE_API FAnimDatabaseFrames MakeFrames(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const;
};

/** Returns the empty set of Frames */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Empty Frames"))
class UAnimDatabaseFramesFunction_Empty : public UAnimDatabaseFramesFunction
{
	GENERATED_BODY()

public:

	UE_API virtual FAnimDatabaseFrames MakeFrames_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Finds all the frames for a given Anim Notify Class */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Anim Notify Frames"))
class UAnimDatabaseFramesFunction_AnimNotify : public UAnimDatabaseFramesFunction
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = "Settings")
	TSubclassOf<UAnimNotify> AnimNotify;

public:

	UE_API virtual FAnimDatabaseFrames MakeFrames_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Finds all the frames for all Anim Notifies on a given track */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Anim Notifies on Track Frames"))
class UAnimDatabaseFramesFunction_AnimNotifiesOnTrack : public UAnimDatabaseFramesFunction
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = "Settings")
	FName TrackName = NAME_None;

public:

	UE_API virtual FAnimDatabaseFrames MakeFrames_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Finds all the frames for a given Sync Marker */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Sync Marker Frames"))
class UAnimDatabaseFramesFunction_SyncMarker : public UAnimDatabaseFramesFunction
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = "Settings")
	FName SyncMarker = NAME_None;

public:

	UE_API virtual FAnimDatabaseFrames MakeFrames_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Finds all the frames at the start of the given frame ranges */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Frame Range Start Frames"))
class UAnimDatabaseFramesFunction_FrameRangeStarts : public UAnimDatabaseFramesFunction
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TObjectPtr<UAnimDatabaseFrameRangesFunction> Function;

public:

	UE_API virtual FAnimDatabaseFrames MakeFrames_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Finds all the frames at the end of the given frame ranges */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Frame Range End Frames"))
class UAnimDatabaseFramesFunction_FrameRangeEnds : public UAnimDatabaseFramesFunction
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TObjectPtr<UAnimDatabaseFrameRangesFunction> Function;

public:

	UE_API virtual FAnimDatabaseFrames MakeFrames_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Entry specifying a frames function */
USTRUCT(BlueprintType)
struct FAnimDatabaseFramesEntry
{
	GENERATED_BODY()

public:

	/** Frames name */
	UPROPERTY(EditAnywhere, Category = "Entry")
	FName Name = NAME_None;

	/** Frames function */
	UPROPERTY(EditAnywhere, Instanced, Category = "Entry")
	TObjectPtr<UAnimDatabaseFramesFunction> Frames;
};

UE_API bool operator==(const FAnimDatabaseFramesEntry& Lhs, const FAnimDatabaseFramesEntry& Rhs);

/** Contains various statistics about a set of ranges within the UAnimDatabase */
USTRUCT(BlueprintType)
struct FAnimDatabaseFrameRangesStatistics
{
	GENERATED_BODY()

	/** Approximate size of the animation data in the ranges on disk in kilobytes. Will only count mirrored data once. */
	UPROPERTY(VisibleAnywhere, Category = "Statistics", meta = (Units = "Kilobytes"))
	float DiskSize = 0.0f;

	/** Approximate size of the uncompressed animation data in the ranges in kilobytes. Will only count mirrored data once. */
	UPROPERTY(VisibleAnywhere, Category = "Statistics", meta = (Units = "Kilobytes"))
	float UncompressedSize = 0.0f;

	/** Approximate size of the compressed animation data in the ranges in kilobytes. Will only count mirrored data once. */
	UPROPERTY(VisibleAnywhere, Category = "Statistics", meta = (Units = "Kilobytes"))
	float CompressedSize = 0.0f;

	/** The total number of ranges */
	UPROPERTY(VisibleAnywhere, Category = "Statistics")
	int32 TotalRangeNum = 0.0;

	/** The total number of unique sequences. Mirrored sequences are counted as a unique sequence. */
	UPROPERTY(VisibleAnywhere, Category = "Statistics")
	int32 TotalSequenceNum = 0.0;

	/** The total number of frames in the given ranges */
	UPROPERTY(VisibleAnywhere, Category = "Statistics")
	int32 TotalFrameNum = 0.0;

	/** The total duration of animation in the ranges in seconds */
	UPROPERTY(VisibleAnywhere, Category = "Statistics", meta = (Units = "Seconds"))
	float TotalDuration = 0.0;

	/** The average duration of a ranges in seconds */
	UPROPERTY(VisibleAnywhere, Category = "Statistics", meta = (Units = "Seconds"))
	float AverageRangeDuration = 0.0;

	/** The minimum duration from the ranges in seconds */
	UPROPERTY(VisibleAnywhere, Category = "Statistics", meta = (Units = "Seconds"))
	float MinimumRangeDuration = 0.0;

	/** The maximum duration from the ranges in seconds */
	UPROPERTY(VisibleAnywhere, Category = "Statistics", meta = (Units = "Seconds"))
	float MaximumRangeDuration = 0.0;

	/** Anim Notifies contained within range */
	UPROPERTY(VisibleAnywhere, Category = "Statistics")
	TArray<TSubclassOf<UAnimNotify>> AnimNotifies;

	/** Anim Notify States contained within range */
	UPROPERTY(VisibleAnywhere, Category = "Statistics")
	TArray<TSubclassOf<UAnimNotifyState>> AnimNotifyStates;
};

UCLASS(BlueprintType, meta = (BlueprintThreadSafe))
class UAnimDatabaseFrameRangesLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/** Make an empty set of FrameRanges */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameRanges MakeEmptyFrameRanges();

	/** Make a set of FrameRanges for all animations in a database */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (BlueprintAutocast, CompactNodeTitle = "->"))
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromDatabase(const UAnimDatabase* Database);

	/** Make a set of FrameRanges for just the non-mirrored animations in a set of frame ranges */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromNotMirrored(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges);

	/** Make a set of FrameRanges for just the mirrored animations in a set of frame ranges */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromMirrored(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges);

	/** Make a set of FrameRanges for just the animations with root motion enabled in a set of frame ranges */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromRootMotionEnabled(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges);

	/** Make a set of FrameRanges for just the animations with root motion disabled in a set of frame ranges */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromRootMotionDisabled(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges);

	/** Make a set of FrameRanges for just the animations with force root lock enabled in a set of frame ranges */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromForceRootLockEnabled(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges);

	/** Make a set of FrameRanges for just the animations with force root lock disabled in a set of frame ranges */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromForceRootLockDisabled(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges);

	/** Make a set of FrameRanges for just the animations which are not looped in a set of frame ranges */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromNotLooped(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges);

	/** Make a set of FrameRanges for just the animations which are looped in a set of frame ranges */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromLooped(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges);

	/** Make a set of FrameRanges for a single sequence in a database */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromSequenceIndex(const UAnimDatabase* Database, const int32 SequenceIdx);

	/** Make a set of FrameRanges for a single range in a sequence in a database */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromSequenceRange(const UAnimDatabase* Database, const int32 SequenceIdx, const int32 RangeStart, const int32 RangeLength);

	/** Make a set of FrameRanges for a given anim sequence in a database */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromAnimSequence(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, UAnimSequence* AnimSequence);

	/** Make a set of FrameRanges for a given anim sequence in a database */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromAnimSequences(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArray<UAnimSequence*>& AnimSequences);
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromAnimSequencesArrayView(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<UAnimSequence* const> AnimSequences);

	/** Make a set of FrameRanges for all sequences with a matching asset name in a set of frame ranges */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromAssetName(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FString& AssetName);
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromAssetNameStringView(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FStringView AssetName);

	/** Make a set of FrameRanges for all sequences with the given string as a prefix in their asset name */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromAssetNameStartsWith(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FString& Prefix);
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromAssetNameStartsWithStringView(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FStringView Prefix);

	/** Make a set of FrameRanges for all sequences with the given string as a suffix in their asset name */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromAssetNameEndsWith(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FString& Suffix);
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromAssetNameEndsWithStringView(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FStringView Suffix);

	/** Make a set of FrameRanges for all sequences with the given string contained in their asset name */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromAssetNameContains(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FString& Substring);
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromAssetNameContainsStringView(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FStringView Substring);

	/** Make a set of FrameRanges for all sequences where the asset name starts with any of the given prefixes */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromAssetNameStartsWithOneOf(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArray<FString>& Prefixes);
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromAssetNameStartsWithOneOfArrayView(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const FString> Prefixes);

	/** Make a set of FrameRanges for all sequences where the asset name ends with any of the given suffixes */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromAssetNameEndsWithOneOf(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArray<FString>& Suffixes);
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromAssetNameEndsWithOneOfArrayView(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const FString> Suffixes);

	/** Make a set of FrameRanges for all sequences where the asset name contains any of the given strings */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromAssetNameContainsOneOf(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArray<FString>& Substrings);
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromAssetNameContainsOneOfArrayView(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const FString> Substrings);

	/** Make a set of FrameRanges for all sequences where the package name contains the given substring */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromAssetPackageContains(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FString& Substring);
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromAssetPackageContainsStringView(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FStringView Substring);

	/** Make a set of FrameRanges for all sequences where the package name contains any of the given substrings */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromAssetPackageContainsOneOf(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArray<FString>& Substring);
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromAssetPackageContainsOneOfArrayView(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const FString> Substrings);

	/** Make a set of FrameRanges for out of all the given anim sequences */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromSequences(const UAnimDatabase* Database, const TArray<UAnimSequence*>& AnimSequences);
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromSequencesArrayView(const UAnimDatabase* Database, const TArrayView<UAnimSequence* const> AnimSequences);
	
	/** Make a set of FrameRanges using the given UAnimDatabaseFrameRangesFunction object */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromFunction(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const UAnimDatabaseFrameRangesFunction* Function);

	/** Make a set of FrameRanges using the given UAnimDatabaseFrameRangesFunction class */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromClass(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TSubclassOf<UAnimDatabaseFrameRangesFunction> Class);

	/** Make a set of FrameRanges with ranges specified by the given anim notify state */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromAnimNotifyState(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TSubclassOf<UAnimNotifyState> State);

	/** Make a set of FrameRanges with ranges specified by when the given anim notify state is not active */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromAnimNotifyStateNotActive(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TSubclassOf<UAnimNotifyState> State);

	/** Make several FrameRanges with ranges specified by the given anim notify states */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API void MakeFrameRangesFromAnimNotifyStates(TArray<FAnimDatabaseFrameRanges>& OutFrameRanges, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArray<TSubclassOf<UAnimNotifyState>>& States);
	static UE_API void MakeFrameRangesFromAnimNotifyStatesToArrayView(const TArrayView<FAnimDatabaseFrameRanges> OutFrameRanges, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const TSubclassOf<UAnimNotifyState>> States);

	/** Make a set of FrameRanges with ranges specified by any of the given anim notify states */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromAnimNotifyStatesUnion(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArray<TSubclassOf<UAnimNotifyState>>& States);
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromAnimNotifyStatesArrayViewUnion(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const TSubclassOf<UAnimNotifyState>> States);

	/** Make a set of FrameRanges with ranges specified by all of the given anim notify states */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromAnimNotifyStatesIntersection(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArray<TSubclassOf<UAnimNotifyState>>& States);
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromAnimNotifyStatesArrayViewIntersection(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TArrayView<const TSubclassOf<UAnimNotifyState>> States);

	/** Make a set of single frame FrameRanges from the given frames */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta=(BlueprintAutocast, CompactNodeTitle = "->"))
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromFrames(const FAnimDatabaseFrames& Frames);

	/** Make a set of FrameRanges for all the frames after a given set of frames */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromAfterFrames(const FAnimDatabaseFrameRanges& FrameRanges, const FAnimDatabaseFrames& Frames);

	/** Make a set of FrameRanges for all the frames before a given set of frames */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromBeforeFrames(const FAnimDatabaseFrameRanges& FrameRanges, const FAnimDatabaseFrames& Frames);

	/** Make a set of Frames from all the start frames of a given FrameRanges set */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrames MakeFramesAtFrameRangesStarts(const FAnimDatabaseFrameRanges& FrameRanges);

	/** Make a set of Frames from all the end frames of a given FrameRanges set */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrames MakeFramesAtFrameRangesEnds(const FAnimDatabaseFrameRanges& FrameRanges);

	/** Make a set of Frames from the given FrameRanges set sampling frames at a regular interval */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrames MakeFramesFromFrameRangesInterval(const FAnimDatabaseFrameRanges& FrameRanges, const int32 IntervalFrames = 10);

	/** Make a set of single frame FrameRanges from the given anim notify */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromAnimNotify(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const TSubclassOf<UAnimNotify> Notify);

	/** Make a set of single frame FrameRanges from the given sync marker */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameRanges MakeFrameRangesFromSyncMarker(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const FName SyncMarkerName);


public:

	/** Makes an empty set of frame ranges. (This is the same as MakeEmptyFrameRanges). */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "NONE"))
	static UE_API FAnimDatabaseFrameRanges FrameRangesNone();

	/** Make a set of frame ranges for all the data in a database. (This is the same as MakeFrameRangesFromDatabase). */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "ALL"))
	static UE_API FAnimDatabaseFrameRanges FrameRangesAll(const UAnimDatabase* Database);

	/** Check if a given set of frame ranges is empty */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API bool FrameRangesIsEmpty(const FAnimDatabaseFrameRanges& FrameRanges);

	/** Check if two sets of frame ranges are equal */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "EQUAL"))
	static UE_API bool FrameRangesEqual(const FAnimDatabaseFrameRanges& A, const FAnimDatabaseFrameRanges& B);

	/** Returns the union of one or more frame range sets */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CommutativeAssociativeBinaryOperator, CompactNodeTitle = "UNION"))
	static UE_API FAnimDatabaseFrameRanges FrameRangesUnion(FAnimDatabaseFrameRanges A, FAnimDatabaseFrameRanges B);

	/** Returns the intersection of one or more frame range sets */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CommutativeAssociativeBinaryOperator, CompactNodeTitle = "INTERSECTION"))
	static UE_API FAnimDatabaseFrameRanges FrameRangesIntersection(FAnimDatabaseFrameRanges A, FAnimDatabaseFrameRanges B);

	/** Returns the difference of one or more frame range sets */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CommutativeAssociativeBinaryOperator, CompactNodeTitle = "DIFFERENCE"))
	static UE_API FAnimDatabaseFrameRanges FrameRangesDifference(FAnimDatabaseFrameRanges A, FAnimDatabaseFrameRanges B);

	/** Returns the union of a set of frames and a set of frame ranges */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "UNION"))
	static UE_API FAnimDatabaseFrameRanges FramesFrameRangesUnion(const FAnimDatabaseFrames& Frames, const FAnimDatabaseFrameRanges& FrameRanges);

	/** Returns the intersection of a set of frames and a set of frame ranges */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "INTERSECTION"))
	static UE_API FAnimDatabaseFrames FramesFrameRangesIntersection(const FAnimDatabaseFrames& Frames, const FAnimDatabaseFrameRanges& FrameRanges);

	/** Returns the difference of a set of frames and a set of frame ranges */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "DIFFERENCE"))
	static UE_API FAnimDatabaseFrames FramesFrameRangesDifference(const FAnimDatabaseFrames& Frames, const FAnimDatabaseFrameRanges& FrameRanges);

	/** Returns the difference of a set of frame ranges and a set of frames */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "DIFFERENCE"))
	static UE_API FAnimDatabaseFrameRanges FrameRangesFramesDifference(const FAnimDatabaseFrameRanges& FrameRanges, const FAnimDatabaseFrames& Frames);

	/** Returns the union of multiple frame ranges */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "UNION"))
	static UE_API FAnimDatabaseFrameRanges FrameRangesUnionFromArray(const TArray<FAnimDatabaseFrameRanges>& FrameRanges);
	static UE_API FAnimDatabaseFrameRanges FrameRangesUnionFromArrayView(const TArrayView<const FAnimDatabaseFrameRanges> FrameRanges);

	/** Returns the intersection of multiple frame ranges */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "INTERSECTION"))
	static UE_API FAnimDatabaseFrameRanges FrameRangesIntersectionFromArray(const TArray<FAnimDatabaseFrameRanges>& FrameRanges);
	static UE_API FAnimDatabaseFrameRanges FrameRangesIntersectionFromArrayView(const TArrayView<const FAnimDatabaseFrameRanges> FrameRanges);

	/** Returns the difference of multiple frame ranges */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "DIFFERENCE"))
	static UE_API FAnimDatabaseFrameRanges FrameRangesDifferenceFromArray(const TArray<FAnimDatabaseFrameRanges>& FrameRanges);
	static UE_API FAnimDatabaseFrameRanges FrameRangesDifferenceFromArrayView(const TArrayView<const FAnimDatabaseFrameRanges> FrameRanges);

	/** Returns all the frame ranges in the data minus the given set of frame ranges. */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "NOT"))
	static UE_API FAnimDatabaseFrameRanges FrameRangesNot(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges);

	/** Returns the frame ranges that bound the given frame ranges in each sequence */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "BOUNDS"))
	static UE_API FAnimDatabaseFrameRanges FrameRangesBounds(const FAnimDatabaseFrameRanges& FrameRanges);

	/** Returns the frame ranges that bound the given frames in each sequence */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "BOUNDS"))
	static UE_API FAnimDatabaseFrameRanges FrameRangesBoundsFrames(const FAnimDatabaseFrames& Frames);

	/** Returns all the frame ranges from A which contain any part of the frame ranges from B */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "INTERSECTS"))
	static UE_API FAnimDatabaseFrameRanges FrameRangesIntersects(const FAnimDatabaseFrameRanges& A, const FAnimDatabaseFrameRanges& B);

	/** Returns all the frame ranges from A which contain any part of the frames from B */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "INTERSECTS"))
	static UE_API FAnimDatabaseFrameRanges FrameRangesIntersectsFrames(const FAnimDatabaseFrameRanges& A, const FAnimDatabaseFrames& B);
	
	/** Return a set of frame ranges with the given amount of frames trimmed from either end. Ranges shorter than the combined TrimStartFrames + TrimEndFrames will be removed. */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "TRIM"))
	static UE_API FAnimDatabaseFrameRanges FrameRangesTrim(const FAnimDatabaseFrameRanges& A, const int32 TrimStartFrames, const int32 TrimEndFrames);

	/** Return a set of frame ranges with the given amount of frames trimmed from the beginning. Ranges shorter than the TrimFrames will be removed. */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "TRIM START"))
	static UE_API FAnimDatabaseFrameRanges FrameRangesTrimStart(const FAnimDatabaseFrameRanges& A, const int32 TrimFrames);

	/** Return a set of frame ranges with the given amount of frames trimmed from the end. Ranges shorter than the TrimFrames will be removed. */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "TRIM END"))
	static UE_API FAnimDatabaseFrameRanges FrameRangesTrimEnd(const FAnimDatabaseFrameRanges& A, const int32 TrimFrames);

	/** Return a set of frame ranges with the given amount of frames added to either end. Will merge ranges if this causes overlaps. */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "PAD"))
	static UE_API FAnimDatabaseFrameRanges FrameRangesPad(const FAnimDatabaseFrameRanges& A, const int32 PadStartFrames, const int32 PadEndFrames);

	/** Return a set of frame ranges with the given amount of frames added to the beginning. Will merge ranges if this causes overlaps. */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "PAD START"))
	static UE_API FAnimDatabaseFrameRanges FrameRangesPadStart(const FAnimDatabaseFrameRanges& A, const int32 PadFrames);

	/** Return a set of frame ranges with the given amount of frames added to the end. Will merge ranges if this causes overlaps. */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (CompactNodeTitle = "PAD END"))
	static UE_API FAnimDatabaseFrameRanges FrameRangesPadEnd(const FAnimDatabaseFrameRanges& A, const int32 PadFrames);

	/** Returns a set of frame ranges with just the ranges of the given indices. */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameRanges FrameRangesGatherRanges(const FAnimDatabaseFrameRanges& FrameRanges, const TArray<int32>& RangeIndices);
	static UE_API FAnimDatabaseFrameRanges FrameRangesGatherRangesFromIndexSet(const FAnimDatabaseFrameRanges& FrameRanges, const UE::Learning::FIndexSet RangeIndices);

	/** Gets all the anim sequences included in a given set of frame ranges */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API void FrameRangesAnimSequences(TArray<UAnimSequence*>& OutAnimSequences, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges);

	/** Gets all the anim sequence assets included in a given set of frame ranges */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API void FrameRangesAnimSequenceAssets(TArray<UAnimSequence*>& OutAnimSequenceAssets, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges);

	/** Computes various useful statistics for a given set of frame ranges */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API void FrameRangesStatistics(FAnimDatabaseFrameRangesStatistics& OutStatistics, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges);

	/**
	 * Computes a hash representing the contents of a set of frame ranges in a database. This hash is not strong, so should not be relied on for 
	 * critical purposes, but can be used to detect data changes for things such as displaying warnings in UIs.
	 */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API int32 FrameRangesContentHash(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges);

	/** Check if a set of frame ranges contains the given sequence */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API bool FrameRangesContainsSequence(const FAnimDatabaseFrameRanges& FrameRanges, const int32 SequenceIdx);

	/** Check if a set of frame ranges contains the given sequence and frame */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API bool FrameRangesContains(const FAnimDatabaseFrameRanges& FrameRanges, const int32 SequenceIdx, const int32 FrameIdx);

	/** Check if a set of frame ranges contains the given sequence and time */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API bool FrameRangesContainsTime(const FAnimDatabaseFrameRanges& FrameRanges, const int32 SequenceIdx, const float SequenceTime, const FFrameRate& FrameRate);

public:

	/** Returns the total number of ranges in a set of frame ranges */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API int32 FrameRangesTotalRangeNum(const FAnimDatabaseFrameRanges& FrameRanges);

	/** Returns the total number of frames in a set of frame ranges */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API int32 FrameRangesTotalFrameNum(const FAnimDatabaseFrameRanges& FrameRanges);

	/** Returns the average number of frames in a set of frame ranges */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API float FrameRangesAverageFrameNum(const FAnimDatabaseFrameRanges& FrameRanges);

	/** Returns the smallest number of frames in a set of frame ranges */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API int32 FrameRangesMinFrameNum(const FAnimDatabaseFrameRanges& FrameRanges);

	/** Returns the largest number of frames in a set of frame ranges */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API int32 FrameRangesMaxFrameNum(const FAnimDatabaseFrameRanges& FrameRanges);

	/** Returns a single frame range from a set of frame ranges for the given index */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabaseFrameRanges FrameRangesRangeAtIndex(const FAnimDatabaseFrameRanges& FrameRanges, const int32 RangeIdx);

public:

	/** Converts a FrameRanges object to a string. */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta = (BlueprintAutocast, CompactNodeTitle = "->", AutoCreateRefTerm = "FrameRanges", Keywords = "Print,Log,Display,Name"))
	static UE_API FString FrameRangesToString(const FAnimDatabaseFrameRanges& FrameRanges);

	/** Converts a FrameRanges object to a string object in full. */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta=(Keywords = "Print,Log,Display,Name"))
	static UE_API FString FrameRangesToStringFormat(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges, const int32 Cutoff = 11);

public:

	/** Finds all of the Anim Notify Classes in the given frame ranges */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API void FindAnimNotifyClassesInFrameRanges(TArray<TSubclassOf<UAnimNotify>>& OutAnimNotifyClasses, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges);

	/** Finds all of the Anim Notify State Classes in the given frame ranges */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API void FindAnimNotifyStateClassesInFrameRanges(TArray<TSubclassOf<UAnimNotifyState>>& OutAnimNotifyStateClasses, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges);

	/** Finds all of the Curves present in the given frame ranges */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API void FindCurvesInFrameRanges(TArray<FName>& OutCurves, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges);

	/** Finds all of the sync markers which are present in the given frame ranges */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API void FindSyncMarkersInFrameRanges(TArray<FName>& SyncMarkers, const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges);

	/** Finds all of the AnimNotifyState Classes which are subclasses of the given base class (exclusive). Only returns classes loaded in memory. */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta=(BaseClass = "/Script/Engine.AnimNotifyState"))
	static UE_API void FindAllAnimNotifyStateClasses(TArray<TSubclassOf<UAnimNotifyState>>& OutClasses, const TSubclassOf<UAnimNotifyState> BaseClass);
};

/** A simple class that has a single implementable function which constructs a FAnimDatabaseFrameRanges object from a UAnimDatabase. */
UCLASS(Abstract, HideDropdown, EditInlineNew, DefaultToInstanced, CollapseCategories, BlueprintType, Blueprintable)
class UAnimDatabaseFrameRangesFunction : public UObject
{
	GENERATED_BODY()

public:

	/** Callback that constructs a FrameRanges set from a UAnimDatabase */
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "AnimDatabase")
	UE_API FAnimDatabaseFrameRanges MakeFrameRanges(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const;
};

/** Returns the empty set of frame ranges */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Empty Frame Ranges"))
class UAnimDatabaseFrameRangesFunction_Empty : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Returns the Frame Ranges given as input */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Identity Frame Ranges"))
class UAnimDatabaseFrameRangesFunction_Identity : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Returns the Frame Ranges given by a set of frames */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Frames Frame Ranges"))
class UAnimDatabaseFrameRangesFunction_Frames : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TObjectPtr<UAnimDatabaseFramesFunction> Frames;

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Ignores the frames given as input and returns all the frame ranges in the database */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "All Frame Ranges"))
class UAnimDatabaseFrameRangesFunction_All : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Finds all the frame ranges that are mirrored */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Mirrored Frame Ranges"))
class UAnimDatabaseFrameRangesFunction_Mirrored : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Finds all the frame ranges that are not mirrored */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Not Mirrored Frame Ranges"))
class UAnimDatabaseFrameRangesFunction_NotMirrored : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Finds all the frame ranges that have root motion enabled */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Root Motion Enabled Frame Ranges"))
class UAnimDatabaseFrameRangesFunction_RootMotionEnabled : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Finds all the frame ranges that have root motion disabled */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Root Motion Disabled Frame Ranges"))
class UAnimDatabaseFrameRangesFunction_RootMotionDisabled : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Finds all the frame ranges that have force root lock enabled */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Force Root Lock Enabled Frame Ranges"))
class UAnimDatabaseFrameRangesFunction_ForceRootLockEnabled : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Finds all the frame ranges that have force root lock disabled */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Force Root Lock Disabled Frame Ranges"))
class UAnimDatabaseFrameRangesFunction_ForceRootLockDisabled : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Finds all the frame ranges that are looped */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Looped Frame Ranges"))
class UAnimDatabaseFrameRangesFunction_Looped : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Finds all the frame ranges that are not looped */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Not Looped Frame Ranges"))
class UAnimDatabaseFrameRangesFunction_NotLooped : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Finds all the frame ranges corresponding to the given Anim Sequence */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Anim Sequence Frame Ranges"))
class UAnimDatabaseFrameRangesFunction_AnimSequence : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = "Settings")
	TObjectPtr<UAnimSequence> AnimSequence;

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Finds all the frame ranges corresponding to the given Anim Sequences */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Anim Sequences Frame Ranges"))
class UAnimDatabaseFrameRangesFunction_AnimSequences : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = "Settings")
	TArray<TObjectPtr<UAnimSequence>> AnimSequences;

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Finds all the frame ranges with a given Anim Notify Class */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Anim Notify Frame Ranges"))
class UAnimDatabaseFrameRangesFunction_AnimNotify : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = "Settings")
	TSubclassOf<UAnimNotify> AnimNotify;

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Finds all the frame ranges with a given Sync Marker */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Sync Marker Frame Ranges"))
class UAnimDatabaseFrameRangesFunction_SyncMarker : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = "Settings")
	FName SyncMarker = NAME_None;

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Finds all the frame ranges with a given Anim Notify State Class */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Anim Notify State Frame Ranges"))
class UAnimDatabaseFrameRangesFunction_AnimNotifyState : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = "Settings")
	TSubclassOf<UAnimNotifyState> AnimNotifyState;

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Finds all the frame ranges which do not have the given Anim Notify State Class active */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Anim Notify State Not Active Frame Ranges"))
class UAnimDatabaseFrameRangesFunction_AnimNotifyStateNotActive : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = "Settings")
	TSubclassOf<UAnimNotifyState> AnimNotifyState;

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Finds all the frame ranges with any of the given Anim Notify State Classes */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Anim Notify State Union Frame Ranges"))
class UAnimDatabaseFrameRangesFunction_AnimNotifyStateUnion : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = "Settings")
	TArray<TSubclassOf<UAnimNotifyState>> AnimNotifyStates;

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Finds all the frame ranges with all of the given Anim Notify State Classes */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Anim Notify State Intersection Frame Ranges"))
class UAnimDatabaseFrameRangesFunction_AnimNotifyStateIntersection : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = "Settings")
	TArray<TSubclassOf<UAnimNotifyState>> AnimNotifyStates;

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Finds all the frame ranges with a given Anim Notify State classes for mirrored and un-mirrored sequences */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Anim Notify State with Mirrored Frame Ranges"))
class UAnimDatabaseFrameRangesFunction_AnimNotifyStateWithMirrored : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = "Settings")
	TSubclassOf<UAnimNotifyState> NotMirroredAnimNotifyState;

	UPROPERTY(EditAnywhere, Category = "Settings")
	TSubclassOf<UAnimNotifyState> MirroredAnimNotifyState;

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Finds all the frame ranges with the given sub-string in the name */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Asset Name Contains Frame Ranges"))
class UAnimDatabaseFrameRangesFunction_AssetNameContains : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = "Settings")
	FString AssetNameSubstring;

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Finds all the frame ranges with any of the given sub-strings in the name */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Asset Name Contains One Of Frame Ranges"))
class UAnimDatabaseFrameRangesFunction_AssetNameContainsOneOf : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = "Settings")
	TArray<FString> AssetNameSubstrings;

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Finds all the frame ranges with the given sub-string in the package */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Asset Package Contains Frame Ranges"))
class UAnimDatabaseFrameRangesFunction_AssetPackageContains : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = "Settings")
	FString AssetPackageSubstring;

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Finds all the frame ranges with any of the given sub-strings in the package */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Asset Package Contains One Of Frame Ranges"))
class UAnimDatabaseFrameRangesFunction_AssetPackageContainsOneOf : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = "Settings")
	TArray<FString> AssetPackageSubstrings;

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Finds all the frame ranges that start with the given sub-string in the name */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Asset Name Starts With Frame Ranges"))
class UAnimDatabaseFrameRangesFunction_AssetNameStartsWith : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = "Settings")
	FString AssetNamePrefix;

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Finds all the frame ranges that start with any of the given sub-strings in the name */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Asset Name Starts With One Of Frame Ranges"))
class UAnimDatabaseFrameRangesFunction_AssetNameStartsWithOneOf : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = "Settings")
	TArray<FString> AssetNamePrefixes;

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Finds all the frame ranges that end with the given sub-string in the name */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Asset Name Ends With Frame Ranges"))
class UAnimDatabaseFrameRangesFunction_AssetNameEndsWith : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = "Settings")
	FString AssetNameSuffix;

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Finds all the frame ranges that end with any of the given sub-strings in the name */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Asset Name Ends With One Of Frame Ranges"))
class UAnimDatabaseFrameRangesFunction_AssetNameEndsWithOneOf : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = "Settings")
	TArray<FString> AssetNameSuffixes;

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Applies a sequence of FrameRangesFunctions to filter ranges */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Filter Frame Ranges"))
class UAnimDatabaseFrameRangesFunction_Filter : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TArray<TObjectPtr<UAnimDatabaseFrameRangesFunction>> Functions;

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Returns all the frame ranges in the database that are NOT given by the provided function  */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Not Frame Ranges"))
class UAnimDatabaseFrameRangesFunction_Not : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TObjectPtr<UAnimDatabaseFrameRangesFunction> Function;

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Finds all the frame ranges which are a union of the given sub-ranges */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Union Frame Ranges"))
class UAnimDatabaseFrameRangesFunction_Union : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TArray<TObjectPtr<UAnimDatabaseFrameRangesFunction>> Functions;

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Finds all the frame ranges which are an intersection of the given sub-ranges */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Intersection Frame Ranges"))
class UAnimDatabaseFrameRangesFunction_Intersection : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TArray<TObjectPtr<UAnimDatabaseFrameRangesFunction>> Functions;

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Finds all the frame ranges which are an difference of the given sub-ranges */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Difference Frame Ranges"))
class UAnimDatabaseFrameRangesFunction_Difference : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TArray<TObjectPtr<UAnimDatabaseFrameRangesFunction>> Functions;

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Finds all the frame ranges which bounds another set of ranges */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Bounds Frame Ranges"))
class UAnimDatabaseFrameRangesFunction_Bounds : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TObjectPtr<UAnimDatabaseFrameRangesFunction> Function;

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Finds all the frame ranges which contain any part of the provided frame ranges */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Intersects Frame Ranges"))
class UAnimDatabaseFrameRangesFunction_Intersects : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TObjectPtr<UAnimDatabaseFrameRangesFunction> Function;

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Finds all the frame ranges which contain any part of the provided frames */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Intersects Frames Frame Ranges"))
class UAnimDatabaseFrameRangesFunction_IntersectsFrames : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TObjectPtr<UAnimDatabaseFramesFunction> Function;

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Finds all the frame ranges which bounds another set of frames */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Bounds Frames Frame Ranges"))
class UAnimDatabaseFrameRangesFunction_BoundsFrames : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TObjectPtr<UAnimDatabaseFramesFunction> Function;

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Frame ranges consisting of single frames spaced at a regular interval */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Interval Frame Ranges"))
class UAnimDatabaseFrameRangesFunction_Interval : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ClampMin = "0", UIMin = "0"))
	int32 Interval = 10;

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Trims the given frame ranges */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Trim Frame Ranges"))
class UAnimDatabaseFrameRangesFunction_Trim : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TObjectPtr<UAnimDatabaseFrameRangesFunction> Function;

	/** Number of frames to trim from the start of the frame ranges */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ClampMin = "0", UIMin = "0"))
	int32 TrimStart = 0;

	/** Number of frames to trim from the end of the frame ranges */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ClampMin = "0", UIMin = "0"))
	int32 TrimEnd = 0;

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Pads the given frame ranges */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Pad Frame Ranges"))
class UAnimDatabaseFrameRangesFunction_Pad : public UAnimDatabaseFrameRangesFunction
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TObjectPtr<UAnimDatabaseFrameRangesFunction> Function;

	/** Number of frames to pad the start of the frame ranges with */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ClampMin = "0", UIMin = "0"))
	int32 PadStart = 0;

	/** Number of frames to pad the end of the frame ranges with */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ClampMin = "0", UIMin = "0"))
	int32 PadEnd = 0;

public:

	UE_API virtual FAnimDatabaseFrameRanges MakeFrameRanges_Implementation(const UAnimDatabase* Database, const FAnimDatabaseFrameRanges& FrameRanges) const override;
};

/** Entry specifying a frame ranges function */
USTRUCT(BlueprintType)
struct FAnimDatabaseFrameRangesEntry
{
	GENERATED_BODY()

public:

	/** Frame Ranges name */
	UPROPERTY(EditAnywhere, Category = "Entry")
	FName Name = NAME_None;

	/** Frame Ranges function */
	UPROPERTY(EditAnywhere, Instanced, Category = "Entry")
	TObjectPtr<UAnimDatabaseFrameRangesFunction> FrameRanges;
};

UE_API bool operator==(const FAnimDatabaseFrameRangesEntry& Lhs, const FAnimDatabaseFrameRangesEntry& Rhs);

#undef UE_API