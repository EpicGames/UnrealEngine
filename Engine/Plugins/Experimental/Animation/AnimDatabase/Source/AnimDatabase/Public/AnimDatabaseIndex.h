// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DataAsset.h"

#include "AnimDatabaseFrameRanges.h"
#include "AnimDatabaseFrameAttribute.h"

#include "AnimDatabaseIndex.generated.h"

#define UE_API ANIMDATABASE_API

class UAnimDatabase;

/** Function used to create an index from a database */
UCLASS(Abstract, HideDropdown, EditInlineNew, CollapseCategories, BlueprintType, Blueprintable)
class UAnimDatabaseIndexFunction : public UObject
{
	GENERATED_BODY()

public:

#if WITH_EDITOR

	/** Builds the index for the given database and frame ranges */
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Index", meta = (ForceAsFunction))
	UE_API void BuildIndex(
		TMap<FName, FAnimDatabaseFrames>& OutIndexFrames,
		TMap<FName, FAnimDatabaseFrameRanges>& OutIndexFrameRanges,
		TMap<FName, FAnimDatabaseFrameAttribute>& OutIndexFrameAttributes,
		UAnimDatabase* InDatabase,
		const FAnimDatabaseFrameRanges& InFrameRanges);

#endif
};

/**
 * A Database Index is an asset you can use to store Frames, FrameRanges, and FrameAttributes derived from an AnimDatabase
 */
UCLASS(Blueprintable)
class UAnimDatabaseIndex : public UDataAsset
{
	GENERATED_BODY()

public:

#if WITH_EDITORONLY_DATA
	/** Database to build the index for */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Build")
	TSoftObjectPtr<UAnimDatabase> Database;

	/** The frame ranges to build the index over */
	UPROPERTY(EditAnywhere, Instanced, BlueprintReadWrite, Category = "Build")
	TObjectPtr<UAnimDatabaseFrameRangesFunction> FrameRanges;

	/** Function used to build the index */
	UPROPERTY(EditAnywhere, Instanced, BlueprintReadWrite, Category = "Build")
	TObjectPtr<UAnimDatabaseIndexFunction> Function;
#endif

#if WITH_EDITOR
	/** Builds the index by calling the BuildIndex function */
	UFUNCTION(CallInEditor, Category = "Build")
	void Build();
#endif

public:

	/** Database this index corresponds to */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Index")
	TObjectPtr<UAnimDatabase> IndexDatabase;

	/** Hash of the database content at the time of indexing */
	UPROPERTY()
	int32 IndexContentHash = 0;

	/** Map of index AnimDatabaseFrames structures */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Index")
	TMap<FName,FAnimDatabaseFrames> IndexFrames;

	/** Map of index AnimDatabaseFrameRanges structures */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Index")
	TMap<FName,FAnimDatabaseFrameRanges> IndexFrameRanges;

	/** Map of index AnimDatabaseFrameAttributes structures */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Index")
	TMap<FName,FAnimDatabaseFrameAttribute> IndexFrameAttributes;
};

/** Function used to create an index from a set of different frames, frame ranges, and frame attribute entries */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Basic"))
class UAnimDatabaseIndexFunction_Basic : public UAnimDatabaseIndexFunction
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

	/** List of frames to store */
	UPROPERTY(EditAnywhere, Category = "Settings")
	TArray<FAnimDatabaseFramesEntry> Frames;

	/** List of frame ranges to store */
	UPROPERTY(EditAnywhere, Category = "Settings")
	TArray<FAnimDatabaseFrameRangesEntry> FrameRanges;

	/** List of frame attribute to store */
	UPROPERTY(EditAnywhere, Category = "Settings")
	TArray<FAnimDatabaseFrameAttributeEntry> FrameAttributes;

#endif

public:

#if WITH_EDITOR

	virtual void BuildIndex_Implementation(
		TMap<FName, FAnimDatabaseFrames>& OutIndexFrames,
		TMap<FName, FAnimDatabaseFrameRanges>& OutIndexFrameRanges,
		TMap<FName, FAnimDatabaseFrameAttribute>& OutIndexFrameAttributes,
		UAnimDatabase* InDatabase,
		const FAnimDatabaseFrameRanges& InFrameRanges) override;

#endif
};

/** Function used to create an index containing frame attribute and ranges for measuring transition cost */
UCLASS( BlueprintType, Blueprintable, meta = (DisplayName = "Transition Pose"))
class UAnimDatabaseIndexFunction_TransitionPose : public UAnimDatabaseIndexFunction
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

	/** Ranges to store the toe location and velocities for */
	UPROPERTY(EditAnywhere, Instanced, Category = "Settings")
	TObjectPtr<UAnimDatabaseFrameRangesFunction> TransitionRanges;

	/** Name of the left toe bone */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FName LeftToeBoneName = TEXT("ball_l");

	/** Name of the right toe bone */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FName RightToeBoneName = TEXT("ball_r");

	/** List of style ranges to store */
	UPROPERTY(EditAnywhere, Category = "Settings")
	TArray<FAnimDatabaseFrameRangesEntry> StyleRanges;

#endif

public:

#if WITH_EDITOR

	virtual void BuildIndex_Implementation(
		TMap<FName, FAnimDatabaseFrames>& OutIndexFrames,
		TMap<FName, FAnimDatabaseFrameRanges>& OutIndexFrameRanges,
		TMap<FName, FAnimDatabaseFrameAttribute>& OutIndexFrameAttributes,
		UAnimDatabase* InDatabase,
		const FAnimDatabaseFrameRanges& InFrameRanges) override;

#endif
};

/** Function used to create an index containing attach bone data */
UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Attach Bone"))
class UAnimDatabaseIndexFunction_AttachBone : public UAnimDatabaseIndexFunction
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

	UPROPERTY(EditAnywhere, Category = "Settings")
	FName AttachBoneName = TEXT("attach");

	UPROPERTY(EditAnywhere, Category = "Settings")
	FVector AttachForwardVector = FVector::ForwardVector;

#endif

public:

#if WITH_EDITOR

	virtual void BuildIndex_Implementation(
		TMap<FName, FAnimDatabaseFrames>& OutIndexFrames,
		TMap<FName, FAnimDatabaseFrameRanges>& OutIndexFrameRanges,
		TMap<FName, FAnimDatabaseFrameAttribute>& OutIndexFrameAttributes,
		UAnimDatabase* InDatabase,
		const FAnimDatabaseFrameRanges& InFrameRanges) override;

#endif
};

/** Blueprint library of helper functions for Animation Database Indices */
UCLASS(BlueprintType, meta = (BlueprintThreadSafe))
class UAnimDatabaseIndexLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/** Finds the best matching transition pose given the Transition Pose index  */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase", meta=(AutoCreateRefTerm = "RootTransform"))
	static UE_API bool FindTransitionPoseMatch(
		UAnimSequence*& OutAnimSequence,
		float& OutAnimSequenceTime,
		bool& bOutAnimSequenceMirrored,
		float& OutMinimumValue,
		const UAnimDatabaseIndex* Index,
		const FVector LeftToeBoneLocation,
		const FVector LeftToeBoneVelocity,
		const FVector RightToeBoneLocation,
		const FVector RightToeBoneVelocity,
		const FTransform& RootTransform,
		UPARAM(ref) int32& RandomState,
		const FName StyleName = NAME_None,
		const float BlendTime = 0.2f,
		const float LocationScale = 100.0f,
		const float VelocityScale = 200.0f,
		const float LocationWeight = 1.0f,
		const float VelocityWeight = 1.0f,
		const float RandomWeight = 0.1f);

	/** Finds the best matching pose for the provided attach bone location and direction  */
	UFUNCTION(BlueprintCallable, Category = "AnimDatabase", meta = (AutoCreateRefTerm = "RootTransform"))
	static UE_API bool FindAttachBoneMatch(
		UAnimSequence*& OutAnimSequence,
		float& OutAnimSequenceTime,
		bool& bOutAnimSequenceMirrored,
		float& OutMinimumValue,
		const UAnimDatabaseIndex* Index,
		const FVector AttachBoneLocation,
		const FVector AttachBoneDirection,
		const FVector RootLinearVelocity,
		const FTransform& RootTransform,
		const float LocationScale = 100.0f,
		const float VelocityScale = 200.0f,
		const float LocationWeight = 1.0f,
		const float DirectionWeight = 1.0f,
		const float VelocityWeight = 1.0f);

};

#undef UE_API