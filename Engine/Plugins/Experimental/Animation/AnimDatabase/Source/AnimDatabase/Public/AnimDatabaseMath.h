// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LearningArray.h"

#define UE_API ANIMDATABASE_API

/**
 * Here we provide various mathematical functions and helpers used in AnimDatabase. 
 * 
 * More specifically, this contains various methods used in dead-blending such as spring-based extrapolation and half life computation, some 
 * inertialization methods, some methods for hermite interpolation, various methods for cubic, cubic monotone, linear, and nearest interpolation,
 * some overall statistical methods such as mean and std computation, and some methods for working with bone indices.
 */
namespace UE::AnimDatabase::Math
{
	// Vector Ops

	UE_API FVector3f VectorMax(const FVector3f V, const float W);
	UE_API FVector3f VectorMin(const FVector3f V, const float W);
	UE_API FVector3f VectorMax(const FVector3f V, const FVector3f W);
	UE_API FVector3f VectorMin(const FVector3f V, const FVector3f W);
	UE_API FVector3f VectorDivMax(const float V, const FVector3f W, const float Epsilon = UE_SMALL_NUMBER);
	UE_API FVector3f VectorDivMax(const FVector3f V, const FVector3f W, const float Epsilon = UE_SMALL_NUMBER);
	UE_API FVector3f VectorInvExpApprox(const FVector3f V);
	UE_API FVector3f VectorEerp(const FVector3f V, const FVector3f W, const float Alpha, const float Epsilon = UE_SMALL_NUMBER);
	UE_API FVector3f VectorLog(const FVector3f V);
	UE_API FVector3f VectorExp(const FVector3f V);
	UE_API FVector3f VectorLogSafe(const FVector3f V, const float Epsilon = UE_SMALL_NUMBER);
	UE_API FVector3f VectorExpSafe(const FVector3f V, const float Max = 10.0f);
	UE_API FVector3f VectorSqrt(const FVector3f A);
	UE_API FVector3f VectorGt(const FVector3f A, const float W);
	UE_API FVector3f VectorSign(const FVector3f A);
	UE_API FVector3f VectorAbs(const FVector3f A);
	UE_API FVector3f VectorEq(const FVector3f A, const FVector3f W);
	UE_API float VectorLength(const FVector4f X);
	UE_API FVector4f VectorNormalize(const FVector4f X);
	UE_API FVector VectorLogSafe(const FVector V, const double Epsilon = UE_SMALL_NUMBER);
	UE_API FVector VectorExpSafe(const FVector V, const double Max = 10.0);

	// Right Handed things used for BVH export

	UE_API FQuat4f MakeQuatFromMatrixRightHanded(const FMatrix44f Matrix);

	// Lerp/Inplace

	UE_API void LerpToTargetInplace(
		const TLearningArrayView<1, float> InOut,
		const TLearningArrayView<1, const float> Target,
		const float Alpha);

	UE_API void LerpToTargetInplace(
		const TLearningArrayView<2, float> InOut,
		const TLearningArrayView<2, const float> Target,
		const float Alpha);

	// Springs

	UE_API float HalfLifeToDamping(const float HalfLife);
	UE_API float DampingToHalfLife(const float Damping);

	UE_API void CriticalSpringUpdate(
		FVector& InOutValue,
		FVector& InOutVelocity,
		const FVector& DesiredValue,
		const float HalfLife,
		const float DeltaTime);

	UE_API void CriticalSpringUpdate(
		FQuat& InOutValue,
		FVector& InOutVelocity,
		const FQuat& DesiredValue,
		const float HalfLife,
		const float DeltaTime);

	UE_API void CriticalSpringUpdate(
		FRotator& InOutValue,
		FVector& InOutVelocity,
		const FRotator& DesiredValue,
		const float HalfLife,
		const float DeltaTime);

	UE_API void CriticalSpringUpdatePositionFromVelocity(
		FVector& InOutPosition,
		FVector& InOutVelocity,
		FVector& InOutAcceleration,
		const FVector& DesiredVelocity,
		const float HalfLife,
		const float DeltaTime);

	// Extrapolation

	UE_API void ExtrapolateTranslation(
		FVector3f& OutTranslation,
		FVector3f& OutVelocity,
		const FVector3f Translation,
		const FVector3f Velocity,
		const float Time,
		const FVector3f DecayHalflife,
		const float Epsilon = UE_SMALL_NUMBER);

	UE_API void ExtrapolateTranslation(
		FVector& OutTranslation,
		FVector3f& OutVelocity,
		const FVector Translation,
		const FVector3f Velocity,
		const float Time,
		const FVector3f DecayHalflife,
		const float Epsilon = UE_SMALL_NUMBER);

	UE_API void ExtrapolateRotation(
		FQuat4f& OutRotation,
		FVector3f& OutVelocity,
		const FQuat4f Rotation,
		const FVector3f Velocity,
		const float Time,
		const FVector3f DecayHalflife,
		const float Epsilon = UE_SMALL_NUMBER);

	UE_API void ExtrapolateScale(
		FVector3f& OutScale,
		FVector3f& OutVelocity,
		const FVector3f Scale,
		const FVector3f Velocity,
		const float Time,
		const FVector3f DecayHalflife,
		const float Epsilon = UE_SMALL_NUMBER);

	UE_API void ExtrapolateCurve(
		float& OutCurve,
		float& OutVelocity,
		const float Curve,
		const float Velocity,
		const float Time,
		const float DecayHalflife,
		const float Epsilon = UE_SMALL_NUMBER);

	// Offset Functions

	UE_API void DecayCubic(
		FVector3f& OutPosition,
		FVector3f& OutVelocity,
		const FVector3f Position,
		const FVector3f Velocity,
		const float Time,
		const float DecayDuration);

	// Half-Life Fitting

	UE_API float ClipMagnitudeToGreaterThanEpsilon(const float X, const float Epsilon = UE_KINDA_SMALL_NUMBER);

	UE_API float ComputeDecayHalfLifeFromDiffAndVelocity(
		const float SrcDstDiff,
		const float SrcVelocity,
		const float HalfLife,
		const float HalfLifeMin,
		const float HalfLifeMax,
		const float Epsilon = UE_KINDA_SMALL_NUMBER);

	UE_API FVector3f ComputeDecayHalfLifeFromDiffAndVelocity(
		const FVector3f SrcDstDiff,
		const FVector3f SrcVelocity,
		const float HalfLife,
		const float HalfLifeMin,
		const float HalfLifeMax,
		const float Epsilon = UE_KINDA_SMALL_NUMBER);

	// Inertialization

	UE_API void LocationInertializeCubicUpdate(
		FVector& OutLocation,
		FVector3f& OutLinearVelocity,
		float& InOutTimeSinceTransition,
		const FVector InLocation,
		const FVector3f InLinearVelocity,
		const FVector3f InLocationOffset,
		const FVector3f InLinearVelocityOffset,
		const float DeltaTime,
		const float BlendTime = 0.2f);

	UE_API void LocationInertializeCubicTransition(
		FVector3f& InOutLocationOffset,
		FVector3f& InOutLinearVelocityOffset,
		float& InOutTimeSinceTransition,
		const FVector InSrcLocation,
		const FVector3f InSrcLinearVelocity,
		const FVector InDstLocation,
		const FVector3f InDstLinearVelocity,
		const float BlendTime = 0.2f);

	// Hermite Interpolation

	UE_API void HermiteValueInterpolate(float& OutValue, float& OutValueVelocity, const float P0, const float P1, const float V0, const float V1, const float X, const float FrameTime);
	UE_API void HermiteAngleInterpolate(float& OutAngle, float& OutAngularVelocity, const float A0, const float A1, const float V0, const float V1, const float X, const float FrameTime);
	UE_API void HermiteLocationInterpolate(FVector& OutPosition, FVector3f& OutLinearVelocity, const FVector P0, const FVector P1, const FVector3f V0, const FVector3f V1, const float X, const float FrameTime);
	UE_API void HermiteLocationInterpolate(FVector3f& OutPosition, FVector3f& OutLinearVelocity, const FVector3f P0, const FVector3f P1, const FVector3f V0, const FVector3f V1, const float X, const float FrameTime);
	UE_API void HermiteRotationInterpolate(FQuat4f& OutRotation, FVector3f& OutAngularVelocity, const FQuat4f R0, const FQuat4f R1, const FVector3f V0, const FVector3f V1, const float X, const float FrameTime);
	UE_API void HermiteScaleInterpolate(FVector3f& OutScale, FVector3f& OutScalarVelocity, const FVector3f S0, const FVector3f S1, const FVector3f V0, const FVector3f V1, const float X, const float FrameTime);
	UE_API void HermiteArrayInterpolate(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<1, float> OutValueVelocities,
		const TLearningArrayView<1, const float> P0,
		const TLearningArrayView<1, const float> P1,
		const TLearningArrayView<1, const float> V0,
		const TLearningArrayView<1, const float> V1,
		const float X,
		const float FrameTime);

	UE_API void HermiteValueInterpolate(float& OutValue, const float P0, const float P1, const float V0, const float V1, const float X);
	UE_API void HermiteAngleInterpolate(float& OutAngle , const float A0, const float A1, const float V0, const float V1, const float X);
	UE_API void HermiteLocationInterpolate(FVector& OutPosition, const FVector P0, const FVector P1, const FVector3f V0, const FVector3f V1, const float X);
	UE_API void HermiteLocationInterpolate(FVector3f& OutPosition, const FVector3f P0, const FVector3f P1, const FVector3f V0, const FVector3f V1, const float X);
	UE_API void HermiteRotationInterpolate(FQuat4f& OutRotation, const FQuat4f R0, const FQuat4f R1, const FVector3f V0, const FVector3f V1, const float X);
	UE_API void HermiteScaleInterpolate(FVector3f& OutScale, const FVector3f S0, const FVector3f S1, const FVector3f V0, const FVector3f V1, const float X);
	UE_API void HermiteArrayInterpolate(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<1, const float> P0,
		const TLearningArrayView<1, const float> P1,
		const TLearningArrayView<1, const float> V0,
		const TLearningArrayView<1, const float> V1,
		const float X);

	// Linear Interpolation

	UE_API void ValueInterpolateLinear(float& OutValue, float& OutValueVelocity, const float P0, const float P1, const float Alpha, const float FrameTime);
	UE_API void AngleInterpolateLinear(float& OutAngle, float& OutAngularVelocity, const float A0, const float A1, const float Alpha, const float FrameTime);
	UE_API void LocationInterpolateLinear(FVector& OutLocation, FVector3f& OutLinearVelocity, const FVector P0, const FVector P1, const float Alpha, const float FrameTime);
	UE_API void LocationInterpolateLinear(FVector3f& OutLocation, FVector3f& OutLinearVelocity, const FVector3f P0, const FVector3f P1, const float Alpha, const float FrameTime);
	UE_API void RotationInterpolateLinear(FQuat4f& OutRotation, FVector3f& OutAngularVelocity, const FQuat4f R0, const FQuat4f R1, const float Alpha, const float FrameTime);
	UE_API void ScaleInterpolateLinear(FVector3f& OutScale, FVector3f& OutScalarVelocity, const FVector3f S0, const FVector3f S1, const float Alpha, const float FrameTime);
	UE_API void ArrayInterpolateLinear(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<1, float> OutValueVelocities,
		const TLearningArrayView<1, const float> P0,
		const TLearningArrayView<1, const float> P1,
		const float Alpha,
		const float FrameTime);

	UE_API void ValueInterpolateLinear(float& OutValue, const float P0, const float P1, const float Alpha);
	UE_API void AngleInterpolateLinear(float& OutAngle, const float A0, const float A1, const float Alpha);
	UE_API void LocationInterpolateLinear(FVector& OutLocation, const FVector P0, const FVector P1, const float Alpha);
	UE_API void LocationInterpolateLinear(FVector3f& OutLocation, const FVector3f P0, const FVector3f P1, const float Alpha);
	UE_API void RotationInterpolateLinear(FQuat4f& OutRotation, const FQuat4f R0, const FQuat4f R1, const float Alpha);
	UE_API void ScaleInterpolateLinear(FVector3f& OutScale, const FVector3f S0, const FVector3f S1, const float Alpha);
	UE_API void DirectionInterpolateLinear(FVector3f& OutDirection, const FVector3f D0, const FVector3f D1, const float Alpha);
	UE_API void TransformInterpolateLinear(FTransform3f& OutTransform, const FTransform3f T0, const FTransform3f T1, const float Alpha);
	UE_API void ArrayInterpolateLinear(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<1, const float> P0,
		const TLearningArrayView<1, const float> P1,
		const float Alpha);

	// Cubic Interpolation

	UE_API void ValueInterpolateCubic(float& OutValue, float& OutValueVelocity, const float P0, const float P1, const float P2, const float P3, const float Alpha, const float FrameTime);
	UE_API void AngleInterpolateCubic(float& OutAngle, float& OutAngularVelocity, const float A0, const float A1, const float A2, const float A3, const float Alpha, const float FrameTime);
	UE_API void LocationInterpolateCubic(FVector& OutLocation, FVector3f& OutLinearVelocity, const FVector P0, const FVector P1, const FVector P2, const FVector P3, const float Alpha, const float FrameTime);
	UE_API void LocationInterpolateCubic(FVector3f& OutLocation, FVector3f& OutLinearVelocity, const FVector3f P0, const FVector3f P1, const FVector3f P2, const FVector3f P3, const float Alpha, const float FrameTime);
	UE_API void RotationInterpolateCubic(FQuat4f& OutRotation, FVector3f& OutAngularVelocity, const FQuat4f R0, const FQuat4f R1, const FQuat4f R2, const FQuat4f R3, const float Alpha, const float FrameTime);
	UE_API void ScaleInterpolateCubic(FVector3f& OutScale, FVector3f& OutScalarVelocity, const FVector3f S0, const FVector3f S1, const FVector3f S2, const FVector3f S3, const float Alpha, const float FrameTime);
	UE_API void ArrayInterpolateCubic(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<1, float> OutValueVelocities,
		const TLearningArrayView<1, const float> P0,
		const TLearningArrayView<1, const float> P1,
		const TLearningArrayView<1, const float> P2,
		const TLearningArrayView<1, const float> P3,
		const float Alpha,
		const float FrameTime);

	UE_API void ValueInterpolateCubic(float& OutValue, const float P0, const float P1, const float P2, const float P3, const float Alpha);
	UE_API void AngleInterpolateCubic(float& OutAngle, const float A0, const float A1, const float A2, const float A3, const float Alpha);
	UE_API void LocationInterpolateCubic(FVector& OutLocation, const FVector P0, const FVector P1, const FVector P2, const FVector P3, const float Alpha);
	UE_API void LocationInterpolateCubic(FVector3f& OutLocation, const FVector3f P0, const FVector3f P1, const FVector3f P2, const FVector3f P3, const float Alpha);
	UE_API void RotationInterpolateCubic(FQuat4f& OutRotation, const FQuat4f R0, const FQuat4f R1, const FQuat4f R2, const FQuat4f R3, const float Alpha);
	UE_API void ScaleInterpolateCubic(FVector3f& OutScale, const FVector3f S0, const FVector3f S1, const FVector3f S2, const FVector3f S3, const float Alpha);
	UE_API void DirectionInterpolateCubic(FVector3f& OutDirection, const FVector3f D0, const FVector3f D1, const FVector3f D2, const FVector3f D3, const float Alpha);
	UE_API void TransformInterpolateCubic(FTransform3f& OutTransform, const FTransform3f T0, const FTransform3f T1, const FTransform3f T2, const FTransform3f T3, const float Alpha);
	UE_API void ArrayInterpolateCubic(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<1, const float> P0,
		const TLearningArrayView<1, const float> P1,
		const TLearningArrayView<1, const float> P2,
		const TLearningArrayView<1, const float> P3,
		const float Alpha);

	// Cubic Interpolation Start

	UE_API void ValueInterpolateCubicStart(float& OutValue, float& OutValueVelocity, const float P1, const float P2, const float P3, const float Alpha, const float FrameTime);
	UE_API void AngleInterpolateCubicStart(float& OutAngle, float& OutAngularVelocity, const float A1, const float A2, const float A3, const float Alpha, const float FrameTime);
	UE_API void LocationInterpolateCubicStart(FVector& OutLocation, FVector3f& OutLinearVelocity, const FVector P1, const FVector P2, const FVector P3, const float Alpha, const float FrameTime);
	UE_API void LocationInterpolateCubicStart(FVector3f& OutLocation, FVector3f& OutLinearVelocity, const FVector3f P1, const FVector3f P2, const FVector3f P3, const float Alpha, const float FrameTime);
	UE_API void RotationInterpolateCubicStart(FQuat4f& OutRotation, FVector3f& OutAngularVelocity, const FQuat4f R1, const FQuat4f R2, const FQuat4f R3, const float Alpha, const float FrameTime);
	UE_API void ScaleInterpolateCubicStart(FVector3f& OutScale, FVector3f& OutScalarVelocity, const FVector3f S1, const FVector3f S2, const FVector3f S3, const float Alpha, const float FrameTime);
	UE_API void ArrayInterpolateCubicStart(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<1, float> OutValueVelocities,
		const TLearningArrayView<1, const float> P1,
		const TLearningArrayView<1, const float> P2,
		const TLearningArrayView<1, const float> P3,
		const float Alpha,
		const float FrameTime);

	UE_API void ValueInterpolateCubicStart(float& OutValue, const float P1, const float P2, const float P3, const float Alpha);
	UE_API void AngleInterpolateCubicStart(float& OutAngle, const float A1, const float A2, const float A3, const float Alpha);
	UE_API void LocationInterpolateCubicStart(FVector& OutLocation, const FVector P1, const FVector P2, const FVector P3, const float Alpha);
	UE_API void LocationInterpolateCubicStart(FVector3f& OutLocation, const FVector3f P1, const FVector3f P2, const FVector3f P3, const float Alpha);
	UE_API void RotationInterpolateCubicStart(FQuat4f& OutRotation, const FQuat4f R1, const FQuat4f R2, const FQuat4f R3, const float Alpha);
	UE_API void ScaleInterpolateCubicStart(FVector3f& OutScale, const FVector3f S1, const FVector3f S2, const FVector3f S3, const float Alpha);
	UE_API void DirectionInterpolateCubicStart(FVector3f& OutDirection, const FVector3f D1, const FVector3f D2, const FVector3f D3, const float Alpha);
	UE_API void TransformInterpolateCubicStart(FTransform3f& OutTransform, const FTransform3f T1, const FTransform3f T2, const FTransform3f T3, const float Alpha);
	UE_API void ArrayInterpolateCubicStart(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<1, const float> P1,
		const TLearningArrayView<1, const float> P2,
		const TLearningArrayView<1, const float> P3,
		const float Alpha);

	// Cubic Interpolation End

	UE_API void ValueInterpolateCubicEnd(float& OutValue, float& OutValueVelocity, const float P0, const float P1, const float P2, const float Alpha, const float FrameTime);
	UE_API void AngleInterpolateCubicEnd(float& OutAngle, float& OutAngularVelocity, const float A0, const float A1, const float A2, const float Alpha, const float FrameTime);
	UE_API void LocationInterpolateCubicEnd(FVector& OutLocation, FVector3f& OutLinearVelocity, const FVector P0, const FVector P1, const FVector P2, const float Alpha, const float FrameTime);
	UE_API void LocationInterpolateCubicEnd(FVector3f& OutLocation, FVector3f& OutLinearVelocity, const FVector3f P0, const FVector3f P1, const FVector3f P2, const float Alpha, const float FrameTime);
	UE_API void RotationInterpolateCubicEnd(FQuat4f& OutRotation, FVector3f& OutAngularVelocity, const FQuat4f R0, const FQuat4f R1, const FQuat4f R2, const float Alpha, const float FrameTime);
	UE_API void ScaleInterpolateCubicEnd(FVector3f& OutScale, FVector3f& OutScalarVelocity, const FVector3f S0, const FVector3f S1, const FVector3f S2, const float Alpha, const float FrameTime);
	UE_API void ArrayInterpolateCubicEnd(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<1, float> OutValueVelocities,
		const TLearningArrayView<1, const float> P0,
		const TLearningArrayView<1, const float> P1,
		const TLearningArrayView<1, const float> P2,
		const float Alpha,
		const float FrameTime);

	UE_API void ValueInterpolateCubicEnd(float& OutValue, const float P0, const float P1, const float P2, const float Alpha);
	UE_API void AngleInterpolateCubicEnd(float& OutAngle, const float A0, const float A1, const float A2, const float Alpha);
	UE_API void LocationInterpolateCubicEnd(FVector& OutLocation, const FVector P0, const FVector P1, const FVector P2, const float Alpha);
	UE_API void LocationInterpolateCubicEnd(FVector3f& OutLocation, const FVector3f P0, const FVector3f P1, const FVector3f P2, const float Alpha);
	UE_API void RotationInterpolateCubicEnd(FQuat4f& OutRotation, const FQuat4f R0, const FQuat4f R1, const FQuat4f R2, const float Alpha);
	UE_API void ScaleInterpolateCubicEnd(FVector3f& OutScale, const FVector3f S0, const FVector3f S1, const FVector3f S2, const float Alpha);
	UE_API void DirectionInterpolateCubicEnd(FVector3f& OutDirection, const FVector3f D0, const FVector3f D1, const FVector3f D2, const float Alpha);
	UE_API void TransformInterpolateCubicEnd(FTransform3f& OutTransform, const FTransform3f T0, const FTransform3f T1, const FTransform3f T2, const float Alpha);
	UE_API void ArrayInterpolateCubicEnd(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<1, const float> P0,
		const TLearningArrayView<1, const float> P1,
		const TLearningArrayView<1, const float> P2,
		const float Alpha);

	// Monotone Velocities

	UE_API float ComputeMonotoneVelocity(const float D0, const float D1);
	UE_API FVector3f ComputeMonotoneVelocity(const FVector3f D0, const FVector3f D1);

	// Cubic Monotone Interpolation

	UE_API void ValueInterpolateCubicMono(float& OutValue, float& OutValueVelocity, const float P0, const float P1, const float P2, const float P3, const float Alpha, const float FrameTime);
	UE_API void AngleInterpolateCubicMono(float& OutAngle, float& OutAngularVelocity, const float A0, const float A1, const float A2, const float A3, const float Alpha, const float FrameTime);
	UE_API void LocationInterpolateCubicMono(FVector& OutLocation, FVector3f& OutLinearVelocity, const FVector P0, const FVector P1, const FVector P2, const FVector P3, const float Alpha, const float FrameTime);
	UE_API void LocationInterpolateCubicMono(FVector3f& OutLocation, FVector3f& OutLinearVelocity, const FVector3f P0, const FVector3f P1, const FVector3f P2, const FVector3f P3, const float Alpha, const float FrameTime);
	UE_API void RotationInterpolateCubicMono(FQuat4f& OutRotation, FVector3f& OutAngularVelocity, const FQuat4f R0, const FQuat4f R1, const FQuat4f R2, const FQuat4f R3, const float Alpha, const float FrameTime);
	UE_API void ScaleInterpolateCubicMono(FVector3f& OutScale, FVector3f& OutScalarVelocity, const FVector3f S0, const FVector3f S1, const FVector3f S2, const FVector3f S3, const float Alpha, const float FrameTime);
	UE_API void ArrayInterpolateCubicMono(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<1, float> OutValueVelocities,
		const TLearningArrayView<1, const float> P0,
		const TLearningArrayView<1, const float> P1,
		const TLearningArrayView<1, const float> P2,
		const TLearningArrayView<1, const float> P3,
		const float Alpha,
		const float FrameTime);

	UE_API void ValueInterpolateCubicMono(float& OutValue, const float P0, const float P1, const float P2, const float P3, const float Alpha);
	UE_API void AngleInterpolateCubicMono(float& OutAngle, const float A0, const float A1, const float A2, const float A3, const float Alpha);
	UE_API void LocationInterpolateCubicMono(FVector& OutLocation, const FVector P0, const FVector P1, const FVector P2, const FVector P3, const float Alpha);
	UE_API void LocationInterpolateCubicMono(FVector3f& OutLocation, const FVector3f P0, const FVector3f P1, const FVector3f P2, const FVector3f P3, const float Alpha);
	UE_API void RotationInterpolateCubicMono(FQuat4f& OutRotation, const FQuat4f R0, const FQuat4f R1, const FQuat4f R2, const FQuat4f R3, const float Alpha);
	UE_API void ScaleInterpolateCubicMono(FVector3f& OutScale, const FVector3f S0, const FVector3f S1, const FVector3f S2, const FVector3f S3, const float Alpha);
	UE_API void DirectionInterpolateCubicMono(FVector3f& OutDirection, const FVector3f D0, const FVector3f D1, const FVector3f D2, const FVector3f D3, const float Alpha);
	UE_API void TransformInterpolateCubicMono(FTransform3f& OutTransform, const FTransform3f T0, const FTransform3f T1, const FTransform3f T2, const FTransform3f T3, const float Alpha);
	UE_API void ArrayInterpolateCubicMono(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<1, const float> P0,
		const TLearningArrayView<1, const float> P1,
		const TLearningArrayView<1, const float> P2,
		const TLearningArrayView<1, const float> P3,
		const float Alpha);

	// Cubic Monotone Interpolation Start

	UE_API void ValueInterpolateCubicMonoStart(float& OutValue, float& OutValueVelocity, const float P1, const float P2, const float P3, const float Alpha, const float FrameTime);
	UE_API void AngleInterpolateCubicMonoStart(float& OutAngle, float& OutAngularVelocity, const float A1, const float A2, const float A3, const float Alpha, const float FrameTime);
	UE_API void LocationInterpolateCubicMonoStart(FVector& OutLocation, FVector3f& OutLinearVelocity, const FVector P1, const FVector P2, const FVector P3, const float Alpha, const float FrameTime);
	UE_API void LocationInterpolateCubicMonoStart(FVector3f& OutLocation, FVector3f& OutLinearVelocity, const FVector3f P1, const FVector3f P2, const FVector3f P3, const float Alpha, const float FrameTime);
	UE_API void RotationInterpolateCubicMonoStart(FQuat4f& OutRotation, FVector3f& OutAngularVelocity, const FQuat4f R1, const FQuat4f R2, const FQuat4f R3, const float Alpha, const float FrameTime);
	UE_API void ScaleInterpolateCubicMonoStart(FVector3f& OutScale, FVector3f& OutScalarVelocity, const FVector3f S1, const FVector3f S2, const FVector3f S3, const float Alpha, const float FrameTime);
	UE_API void ArrayInterpolateCubicMonoStart(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<1, float> OutValueVelocities,
		const TLearningArrayView<1, const float> P1,
		const TLearningArrayView<1, const float> P2,
		const TLearningArrayView<1, const float> P3,
		const float Alpha,
		const float FrameTime);

	UE_API void ValueInterpolateCubicMonoStart(float& OutValue, const float P1, const float P2, const float P3, const float Alpha);
	UE_API void AngleInterpolateCubicMonoStart(float& OutAngle, const float A1, const float A2, const float A3, const float Alpha);
	UE_API void LocationInterpolateCubicMonoStart(FVector& OutLocation, const FVector P1, const FVector P2, const FVector P3, const float Alpha);
	UE_API void LocationInterpolateCubicMonoStart(FVector3f& OutLocation, const FVector3f P1, const FVector3f P2, const FVector3f P3, const float Alpha);
	UE_API void RotationInterpolateCubicMonoStart(FQuat4f& OutRotation, const FQuat4f R1, const FQuat4f R2, const FQuat4f R3, const float Alpha);
	UE_API void ScaleInterpolateCubicMonoStart(FVector3f& OutScale, const FVector3f S1, const FVector3f S2, const FVector3f S3, const float Alpha);
	UE_API void DirectionInterpolateCubicMonoStart(FVector3f& OutDirection, const FVector3f D1, const FVector3f D2, const FVector3f D3, const float Alpha);
	UE_API void TransformInterpolateCubicMonoStart(FTransform3f& OutTransform, const FTransform3f T1, const FTransform3f T2, const FTransform3f T3, const float Alpha);
	UE_API void ArrayInterpolateCubicMonoStart(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<1, const float> P1,
		const TLearningArrayView<1, const float> P2,
		const TLearningArrayView<1, const float> P3,
		const float Alpha);

	// Cubic Monotone Interpolation End

	UE_API void ValueInterpolateCubicMonoEnd(float& OutValue, float& OutValueVelocity, const float P0, const float P1, const float P2, const float Alpha, const float FrameTime);
	UE_API void AngleInterpolateCubicMonoEnd(float& OutAngle, float& OutAngularVelocity, const float A0, const float A1, const float A2, const float Alpha, const float FrameTime);
	UE_API void LocationInterpolateCubicMonoEnd(FVector& OutLocation, FVector3f& OutLinearVelocity, const FVector P0, const FVector P1, const FVector P2, const float Alpha, const float FrameTime);
	UE_API void LocationInterpolateCubicMonoEnd(FVector3f& OutLocation, FVector3f& OutLinearVelocity, const FVector3f P0, const FVector3f P1, const FVector3f P2, const float Alpha, const float FrameTime);
	UE_API void RotationInterpolateCubicMonoEnd(FQuat4f& OutRotation, FVector3f& OutAngularVelocity, const FQuat4f R0, const FQuat4f R1, const FQuat4f R2, const float Alpha, const float FrameTime);
	UE_API void ScaleInterpolateCubicMonoEnd(FVector3f& OutScale, FVector3f& OutScalarVelocity, const FVector3f S0, const FVector3f S1, const FVector3f S2, const float Alpha, const float FrameTime);
	UE_API void ArrayInterpolateCubicMonoEnd(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<1, float> OutValueVelocities,
		const TLearningArrayView<1, const float> P0,
		const TLearningArrayView<1, const float> P1,
		const TLearningArrayView<1, const float> P2,
		const float Alpha,
		const float FrameTime);

	UE_API void ValueInterpolateCubicMonoEnd(float& OutValue, const float P0, const float P1, const float P2, const float Alpha);
	UE_API void AngleInterpolateCubicMonoEnd(float& OutAngle, const float A0, const float A1, const float A2, const float Alpha);
	UE_API void LocationInterpolateCubicMonoEnd(FVector& OutLocation, const FVector P0, const FVector P1, const FVector P2, const float Alpha);
	UE_API void LocationInterpolateCubicMonoEnd(FVector3f& OutLocation, const FVector3f P0, const FVector3f P1, const FVector3f P2, const float Alpha);
	UE_API void RotationInterpolateCubicMonoEnd(FQuat4f& OutRotation, const FQuat4f R0, const FQuat4f R1, const FQuat4f R2, const float Alpha);
	UE_API void ScaleInterpolateCubicMonoEnd(FVector3f& OutScale, const FVector3f S0, const FVector3f S1, const FVector3f S2, const float Alpha);
	UE_API void DirectionInterpolateCubicMonoEnd(FVector3f& OutDirection, const FVector3f D0, const FVector3f D1, const FVector3f D2, const float Alpha);
	UE_API void TransformInterpolateCubicMonoEnd(FTransform3f& OutTransform, const FTransform3f T0, const FTransform3f T1, const FTransform3f T2, const float Alpha);
	UE_API void ArrayInterpolateCubicMonoEnd(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<1, const float> P0,
		const TLearningArrayView<1, const float> P1,
		const TLearningArrayView<1, const float> P2,
		const float Alpha);

	// Frame Sample Indices and Alpha

	UE_API int64 ComputeNearestSampleFrame(
		const float FrameTime,
		const int64 FrameNum);

	UE_API void ComputeLinearSampleFramesAndAlpha(
		int64& OutSampleFrame0,
		int64& OutSampleFrame1,
		float& OutSampleAlpha,
		const float FrameTime,
		const int64 FrameNum);

	UE_API void ComputeCubicSampleFramesAndAlpha(
		int64& OutSampleFrame0,
		int64& OutSampleFrame1,
		int64& OutSampleFrame2,
		int64& OutSampleFrame3,
		float& OutSampleAlpha,
		const float FrameTime,
		const int64 FrameNum);

	UE_API int64 ComputeNearestSampleFrame(
		const int64 FrameTime,
		const int64 FrameNum);

	UE_API void ComputeLinearSampleFramesAndAlpha(
		int64& OutSampleFrame0,
		int64& OutSampleFrame1,
		float& OutSampleAlpha,
		const int64 FrameTime,
		const int64 FrameNum);

	UE_API void ComputeCubicSampleFramesAndAlpha(
		int64& OutSampleFrame0,
		int64& OutSampleFrame1,
		int64& OutSampleFrame2,
		int64& OutSampleFrame3,
		float& OutSampleAlpha,
		const int64 FrameTime,
		const int64 FrameNum);

	// Sampling

	UE_API void ValueSampleNearest(
		float& OutValue,
		const TLearningArrayView<1, const float> InValues,
		const int32 SampleFrame);

	UE_API void AngleSampleNearest(
		float& OutAngle,
		const TLearningArrayView<1, const float> InAngles,
		const int32 SampleFrame);

	UE_API void LocationSampleNearest(
		FVector& OutLocation,
		const TLearningArrayView<1, const FVector> InLocations,
		const int32 SampleFrame);

	UE_API void LocationSampleNearest(
		FVector& OutLocation,
		FVector3f& OutLinearVelocity,
		const TLearningArrayView<1, const FVector> InLocations,
		const int32 SampleFrame);

	UE_API void RotationSampleNearest(
		FQuat4f& OutRotation,
		const TLearningArrayView<1, const FQuat4f> InRotations,
		const int32 SampleFrame);

	UE_API void RotationSampleNearest(
		FQuat4f& OutRotation,
		FVector3f& OutAngularVelocity,
		const TLearningArrayView<1, const FQuat4f> InRotations,
		const int32 SampleFrame);

	UE_API void TransformSampleNearest(
		FVector& OutLocation,
		FQuat4f& OutRotation,
		FVector3f& OutScale,
		const TLearningArrayView<1, const FVector> InLocations,
		const TLearningArrayView<1, const FQuat4f> InRotations,
		const TLearningArrayView<1, const FVector3f> InScales,
		const int32 SampleFrame);

	UE_API void TransformSampleNearest(
		FVector& OutLocation,
		FQuat4f& OutRotation,
		FVector3f& OutScale,
		FVector3f& OutLinearVelocity,
		FVector3f& OutAngularVelocity,
		FVector3f& OutScalarVelocity,
		const TLearningArrayView<1, const FVector> InLocations,
		const TLearningArrayView<1, const FQuat4f> InRotations,
		const TLearningArrayView<1, const FVector3f> InScales,
		const int32 SampleFrame);

	UE_API void TransformSampleNearest(
		const TLearningArrayView<1, FVector3f> OutLocations,
		const TLearningArrayView<1, FQuat4f> OutRotations,
		const TLearningArrayView<1, FVector3f> OutScales,
		const TLearningArrayView<1, FVector3f> OutLinearVelocities,
		const TLearningArrayView<1, FVector3f> OutAngularVelocities,
		const TLearningArrayView<1, FVector3f> OutScalarVelocities,
		const TLearningArrayView<2, const FVector3f> InLocations,
		const TLearningArrayView<2, const FQuat4f> InRotations,
		const TLearningArrayView<2, const FVector3f> InScales,
		const int32 SampleFrame);

	UE_API void ArraySampleNearest(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<2, const float> InValues,
		const int32 SampleFrame);

	UE_API void ArraySampleNearest(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<1, float> OutVelocities,
		const TLearningArrayView<2, const float> InValues,
		const int32 SampleFrame);

	UE_API void ValueSampleLinear(
		float& OutValue,
		const TLearningArrayView<1, const float> InValues,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const float SampleAlpha);

	UE_API void AngleSampleLinear(
		float& OutAngle,
		const TLearningArrayView<1, const float> InAngles,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const float SampleAlpha);

	UE_API void LocationSampleLinear(
		FVector& OutLocation,
		const TLearningArrayView<1, const FVector> InLocations,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const float SampleAlpha);

	UE_API void LocationSampleLinear(
		FVector& OutLocation,
		FVector3f& OutLinearVelocity,
		const TLearningArrayView<1, const FVector> InLocations,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const float SampleAlpha,
		const float FrameDeltaTime);

	UE_API void RotationSampleLinear(
		FQuat4f& OutRotation,
		const TLearningArrayView<1, const FQuat4f> InRotations,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const float SampleAlpha);

	UE_API void RotationSampleLinear(
		FQuat4f& OutRotation,
		FVector3f& OutAngularVelocity,
		const TLearningArrayView<1, const FQuat4f> InRotations,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const float SampleAlpha,
		const float FrameDeltaTime);

	UE_API void TransformSampleLinear(
		FVector& OutLocation,
		FQuat4f& OutRotation,
		FVector3f& OutScale,
		const TLearningArrayView<1, const FVector> InLocations,
		const TLearningArrayView<1, const FQuat4f> InRotations,
		const TLearningArrayView<1, const FVector3f> InScales,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const float SampleAlpha);

	UE_API void TransformSampleLinear(
		FVector& OutLocation,
		FQuat4f& OutRotation,
		FVector3f& OutScale,
		FVector3f& OutLinearVelocity,
		FVector3f& OutAngularVelocity,
		FVector3f& OutScalarVelocity,
		const TLearningArrayView<1, const FVector> InLocations,
		const TLearningArrayView<1, const FQuat4f> InRotations,
		const TLearningArrayView<1, const FVector3f> InScales,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const float SampleAlpha,
		const float FrameDeltaTime);

	UE_API void TransformSampleLinear(
		const TLearningArrayView<1, FVector3f> OutLocations,
		const TLearningArrayView<1, FQuat4f> OutRotations,
		const TLearningArrayView<1, FVector3f> OutScales,
		const TLearningArrayView<1, FVector3f> OutLinearVelocities,
		const TLearningArrayView<1, FVector3f> OutAngularVelocities,
		const TLearningArrayView<1, FVector3f> OutScalarVelocities,
		const TLearningArrayView<2, const FVector3f> InLocations,
		const TLearningArrayView<2, const FQuat4f> InRotations,
		const TLearningArrayView<2, const FVector3f> InScales,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const float SampleAlpha,
		const float FrameDeltaTime);

	UE_API void ArraySampleLinear(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<2, const float> InValues,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const float SampleAlpha);

	UE_API void ArraySampleLinear(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<1, float> OutVelocities,
		const TLearningArrayView<2, const float> InValues,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const float SampleAlpha,
		const float FrameDeltaTime);

	UE_API void ValueSampleCubic(
		float& OutValue,
		const TLearningArrayView<1, const float> InValues,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha);

	UE_API void AngleSampleCubic(
		float& OutAngle,
		const TLearningArrayView<1, const float> InAngles,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha);

	UE_API void LocationSampleCubic(
		FVector& OutLocation,
		const TLearningArrayView<1, const FVector> InLocations,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha);

	UE_API void LocationSampleCubic(
		FVector& OutLocation,
		FVector3f& OutLinearVelocity,
		const TLearningArrayView<1, const FVector> InLocations,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha,
		const float FrameDeltaTime);

	UE_API void RotationSampleCubic(
		FQuat4f& OutRotation,
		const TLearningArrayView<1, const FQuat4f> InRotations,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha);

	UE_API void RotationSampleCubic(
		FQuat4f& OutRotation,
		FVector3f& OutAngularVelocity,
		const TLearningArrayView<1, const FQuat4f> InRotations,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha,
		const float FrameDeltaTime);

	UE_API void TransformSampleCubic(
		FVector& OutLocation,
		FQuat4f& OutRotation,
		FVector3f& OutScale,
		const TLearningArrayView<1, const FVector> InLocations,
		const TLearningArrayView<1, const FQuat4f> InRotations,
		const TLearningArrayView<1, const FVector3f> InScales,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha);

	UE_API void TransformSampleCubic(
		FVector& OutLocation,
		FQuat4f& OutRotation,
		FVector3f& OutScale,
		FVector3f& OutLinearVelocity,
		FVector3f& OutAngularVelocity,
		FVector3f& OutScalarVelocity,
		const TLearningArrayView<1, const FVector> InLocations,
		const TLearningArrayView<1, const FQuat4f> InRotations,
		const TLearningArrayView<1, const FVector3f> InScales,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha,
		const float FrameDeltaTime);

	UE_API void TransformSampleCubic(
		const TLearningArrayView<1, FVector3f> OutLocations,
		const TLearningArrayView<1, FQuat4f> OutRotations,
		const TLearningArrayView<1, FVector3f> OutScales,
		const TLearningArrayView<1, FVector3f> OutLinearVelocities,
		const TLearningArrayView<1, FVector3f> OutAngularVelocities,
		const TLearningArrayView<1, FVector3f> OutScalarVelocities,
		const TLearningArrayView<2, const FVector3f> InLocations,
		const TLearningArrayView<2, const FQuat4f> InRotations,
		const TLearningArrayView<2, const FVector3f> InScales,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha,
		const float FrameDeltaTime);

	UE_API void ArraySampleCubic(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<2, const float> InValues,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha);

	UE_API void ArraySampleCubic(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<1, float> OutVelocities,
		const TLearningArrayView<2, const float> InValues,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha,
		const float FrameDeltaTime);

	UE_API void ValueSampleCubicMono(
		float& OutValue,
		const TLearningArrayView<1, const float> InValues,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha);

	UE_API void AngleSampleCubicMono(
		float& OutAngle,
		const TLearningArrayView<1, const float> InAngles,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha);

	UE_API void LocationSampleCubicMono(
		FVector& OutLocation,
		const TLearningArrayView<1, const FVector> InLocations,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha);

	UE_API void LocationSampleCubicMono(
		FVector& OutLocation,
		FVector3f& OutLinearVelocity,
		const TLearningArrayView<1, const FVector> InLocations,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha,
		const float FrameDeltaTime);

	UE_API void RotationSampleCubicMono(
		FQuat4f& OutRotation,
		const TLearningArrayView<1, const FQuat4f> InRotations,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha);

	UE_API void RotationSampleCubicMono(
		FQuat4f& OutRotation,
		FVector3f& OutAngularVelocity,
		const TLearningArrayView<1, const FQuat4f> InRotations,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha,
		const float FrameDeltaTime);

	UE_API void TransformSampleCubicMono(
		FVector& OutLocation,
		FQuat4f& OutRotation,
		FVector3f& OutScale,
		const TLearningArrayView<1, const FVector> InLocations,
		const TLearningArrayView<1, const FQuat4f> InRotations,
		const TLearningArrayView<1, const FVector3f> InScales,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha);

	UE_API void TransformSampleCubicMono(
		FVector& OutLocation,
		FQuat4f& OutRotation,
		FVector3f& OutScale,
		FVector3f& OutLinearVelocity,
		FVector3f& OutAngularVelocity,
		FVector3f& OutScalarVelocity,
		const TLearningArrayView<1, const FVector> InLocations,
		const TLearningArrayView<1, const FQuat4f> InRotations,
		const TLearningArrayView<1, const FVector3f> InScales,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha,
		const float FrameDeltaTime);

	UE_API void TransformSampleCubicMono(
		const TLearningArrayView<1, FVector3f> OutLocations,
		const TLearningArrayView<1, FQuat4f> OutRotations,
		const TLearningArrayView<1, FVector3f> OutScales,
		const TLearningArrayView<1, FVector3f> OutLinearVelocities,
		const TLearningArrayView<1, FVector3f> OutAngularVelocities,
		const TLearningArrayView<1, FVector3f> OutScalarVelocities,
		const TLearningArrayView<2, const FVector3f> InLocations,
		const TLearningArrayView<2, const FQuat4f> InRotations,
		const TLearningArrayView<2, const FVector3f> InScales,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha,
		const float FrameDeltaTime);

	UE_API void ArraySampleCubicMono(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<2, const float> InValues,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha);

	UE_API void ArraySampleCubicMono(
		const TLearningArrayView<1, float> OutValues,
		const TLearningArrayView<1, float> OutVelocities,
		const TLearningArrayView<2, const float> InValues,
		const int32 SampleFrame0,
		const int32 SampleFrame1,
		const int32 SampleFrame2,
		const int32 SampleFrame3,
		const float SampleAlpha,
		const float FrameDeltaTime);

	// Sampling

	UE_API void NearestSample(
		float& OutSampledValue,
		const TLearningArrayView<1, const float> InValues,
		const float Time,
		const float FrameDeltaTime);

	UE_API void NearestSampleArray(
		const TLearningArrayView<1, float> OutSampledVector,
		const TLearningArrayView<2, const float> InVectors,
		const float Time,
		const float FrameDeltaTime);

	UE_API void LinearSample(
		float& OutSampledValue,
		const TLearningArrayView<1, const float> InValues,
		const float Time,
		const float FrameDeltaTime);

	UE_API void LinearSampleArray(
		const TLearningArrayView<1, float> OutSampledVector,
		const TLearningArrayView<2, const float> InVectors,
		const float Time,
		const float FrameDeltaTime);

	UE_API void CubicSample(
		float& OutSampledValue,
		const TLearningArrayView<1, const float> InValues,
		const float Time,
		const float FrameDeltaTime);

	UE_API void CubicSampleArray(
		const TLearningArrayView<1, float> OutSampledVector,
		const TLearningArrayView<2, const float> InVectors,
		const float Time,
		const float FrameDeltaTime);

	UE_API void CubicMonoSample(
		float& OutSampledValue,
		const TLearningArrayView<1, const float> InValues,
		const float Time,
		const float FrameDeltaTime);

	UE_API void CubicMonoSampleLocation(
		FVector& OutSampledLocation,
		const TLearningArrayView<1, const FVector> InLocations,
		const float Time,
		const float FrameDeltaTime);

	UE_API void CubicMonoSampleLocation(
		FVector& OutSampledLocation,
		FVector3f& OutSampledVelocity,
		const TLearningArrayView<1, const FVector> InLocations,
		const float Time,
		const float FrameDeltaTime);

	UE_API void CubicMonoSampleArray(
		const TLearningArrayView<1, float> OutSampledVector,
		const TLearningArrayView<2, const float> InVectors,
		const float Time,
		const float FrameDeltaTime);

	// Mean/Std/Scale/Min/Max
	
	UE_API float Sum(const TLearningArrayView<1, const float> Data);

	UE_API void ComputeMean(
		float& OutMean,
		const TLearningArrayView<1, const float> Data);

	UE_API void ComputeMeanStd(
		float& OutMean, 
		float& OutStd, 
		const TLearningArrayView<1, const float> Data);

	UE_API void ComputeMean(
		const TLearningArrayView<1, float> OutMean,
		const TLearningArrayView<2, const float> Data);

	UE_API void ComputeMeanStd(
		const TLearningArrayView<1, float> OutMean,
		const TLearningArrayView<1, float> OutStd,
		const TLearningArrayView<2, const float> Data);

	UE_API void ComputeMeanStd(
		const TLearningArrayView<1, float> OutMean,
		const TLearningArrayView<1, float> OutStd,
		const TLearningArrayView<2, const float> Data,
		const int32 ColSliceStart,
		const int32 ColSliceNum);

	UE_API void ComputeMeanStdMasked(
		const TLearningArrayView<1, float> OutMean,
		const TLearningArrayView<1, float> OutStd,
		const TLearningArrayView<2, const float> Data,
		const TLearningArrayView<1, const bool> Mask,
		const int32 ColSliceStart,
		const int32 ColSliceNum);

	UE_API void ComputeMeanStd(
		FVector3f& OutMean,
		FVector3f& OutStd,
		const TLearningArrayView<1, const FVector3f> Data);

	UE_API void ComputeLocalMeanStd(
		FVector3f& OutMean,
		FVector3f& OutStd,
		const TLearningArrayView<1, const FVector3f> Data,
		const TLearningArrayView<1, const FQuat4f> Reference);

	UE_API void ComputeMeanStdOfLog(
		FVector3f& OutMeanLog,
		FVector3f& OutStdLog,
		const TLearningArrayView<1, const FVector3f> Data);

	UE_API void ComputeMeanStd(
		const TLearningArrayView<1, FVector3f> OutMeans,
		const TLearningArrayView<1, FVector3f> OutStds,
		const TLearningArrayView<2, const FVector3f> Vectors);

	UE_API void ComputeMeanStdOfLog(
		const TLearningArrayView<1, FVector3f> OutMeanLogs,
		const TLearningArrayView<1, FVector3f> OutStdLogs,
		const TLearningArrayView<2, const FVector3f> Vectors);

	UE_API FVector4f DominantEigenVector(
		const FMatrix44f& A,
		const FVector4f V0);

	UE_API void ComputeMeanStd(
		const TLearningArrayView<1, FQuat4f> OutMeans,
		const TLearningArrayView<1, FMatrix44f> OutAccum,
		const TLearningArrayView<1, FVector3f> OutStds,
		const TLearningArrayView<2, const FQuat4f> Rotations);

	UE_API void ComputeMeanStdWithReference(
		FQuat4f& OutMean,
		FVector3f& OutStd,
		const TLearningArrayView<1, const FQuat4f> Rotations,
		const FQuat4f Reference);

	UE_API void ComputeMeanStdWithReference(
		const TLearningArrayView<1, FQuat4f> OutMeans,
		const TLearningArrayView<1, FMatrix44f> OutAccum,
		const TLearningArrayView<1, FVector3f> OutStds,
		const TLearningArrayView<2, const FQuat4f> Rotations,
		const TLearningArrayView<1, const FQuat4f> Reference);

	UE_API void ComputeMeanStdAroundReference(
		const TLearningArrayView<1, FQuat4f> OutMeans,
		const TLearningArrayView<1, FVector3f> OutStds,
		const TLearningArrayView<2, const FQuat4f> Rotations,
		const TLearningArrayView<1, const FQuat4f> Reference);

	UE_API void ComputeMinMax(
		float& OutMin,
		float& OutMax,
		const TLearningArrayView<1, const float> Data);

	UE_API void ComputeMinMax(
		const TLearningArrayView<1, float> OutMin,
		const TLearningArrayView<1, float> OutMax,
		const TLearningArrayView<2, const float> Data);

	UE_API void ComputeMinMax(
		const TLearningArrayView<1, float> OutMin,
		const TLearningArrayView<1, float> OutMax,
		const TLearningArrayView<2, const float> Data,
		const int32 ColSliceStart,
		const int32 ColSliceNum);

	UE_API void ComputeMaskedMinMax(
		const TLearningArrayView<1, float> OutMin,
		const TLearningArrayView<1, float> OutMax,
		const TLearningArrayView<2, const float> Data,
		const TLearningArrayView<2, const bool> Mask);

	// Normalization and Clamping

	UE_API void ScaleInplace(
		const TLearningArrayView<2, float> InOutData,
		const TLearningArrayView<1, const float> Scale);

	UE_API void NormalizeInplace(
		const TLearningArrayView<2, float> InOutData,
		const TLearningArrayView<1, const float> Mean,
		const TLearningArrayView<1, const float> Std,
		const float Eps = UE_SMALL_NUMBER);

	UE_API void NormalizeInplace(
		const TLearningArrayView<2, float> InOutData,
		const TLearningArrayView<1, const float> Mean,
		const float Std,
		const float Eps = UE_SMALL_NUMBER);

	UE_API void DenormalizeInplace(
		const TLearningArrayView<2, float> InOutData,
		const TLearningArrayView<1, const float> Mean,
		const TLearningArrayView<1, const float> Std);

	UE_API void DenormalizeInplace(
		const TLearningArrayView<2, float> InOutData,
		const TLearningArrayView<1, const float> Mean,
		const float Std);

	UE_API void ClampInplace(
		const TLearningArrayView<2, float> InOutData,
		const TLearningArrayView<1, const float> Min,
		const TLearningArrayView<1, const float> Max);

	UE_API void MaskedClampInplace(
		const TLearningArrayView<2, float> InOutData,
		const TLearningArrayView<2, const bool> Mask,
		const TLearningArrayView<1, const float> Min,
		const TLearningArrayView<1, const float> Max);

	UE_API void ClampNormalizedInplace(
		const TLearningArrayView<2, float> InOutData,
		const TLearningArrayView<1, const float> Mean,
		const float Std,
		const TLearningArrayView<1, const float> Min,
		const TLearningArrayView<1, const float> Max);


	// Bone Indices

	UE_API bool AnyBoneIndicesInvalid(const UE::Learning::FIndexSet Indices);

	UE_API bool BoneIndicesAreSortedAndUnique(const UE::Learning::FIndexSet Indices);

	UE_API int32 BoneAscendantsInclusiveNum(
		const int32 BoneIdx, 
		const TLearningArrayView<1, const int32> BoneParents);
	
	UE_API void BoneAscendantsInclusive(
		const TLearningArrayView<1, int32> OutBoneAscendants, 
		const int32 BoneIdx, 
		const TLearningArrayView<1, const int32> BoneParents);

	UE_API void BoneAscendantsInclusive(
		TArray<int32>& OutBoneAscendants,
		const int32 BoneIdx,
		const TLearningArrayView<1, const int32> BoneParents);

	UE_API int32 BoneUnionNum(
		const UE::Learning::FIndexSet Lhs,
		const UE::Learning::FIndexSet Rhs);

	UE_API void BoneUnion(
		const TLearningArrayView<1, int32> OutBoneUnion,
		const UE::Learning::FIndexSet Lhs, 
		const UE::Learning::FIndexSet Rhs);

	UE_API void BoneUnion(
		TArray<int32>& OutBoneUnion,
		const UE::Learning::FIndexSet Lhs,
		const UE::Learning::FIndexSet Rhs);

	UE_API void BoneFindIndicesOf(
		const TLearningArrayView<1, int32> OutIndicesOf,
		const UE::Learning::FIndexSet Items,
		const UE::Learning::FIndexSet Elements);

	UE_API void BoneChildrenMatrix(
		TLearningArrayView<2, bool> OutBoneChildren,
		const TLearningArrayView<1, const int32> BoneParents);

	UE_API void BoneDescedantsMatrix(
		TLearningArrayView<2, bool> OutBoneDescendants,
		const TLearningArrayView<1, const int32> BoneParents);
}

#undef UE_API 