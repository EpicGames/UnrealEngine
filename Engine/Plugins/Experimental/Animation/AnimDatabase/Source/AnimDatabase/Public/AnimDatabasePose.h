// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LearningArray.h"

#include "Kismet/BlueprintFunctionLibrary.h"

#include "AnimDatabasePose.generated.h"

#define UE_API ANIMDATABASE_API

enum class EAnimDatabaseAttributeType : uint8;

namespace UE::AnimDatabase
{
	/*
	 * Included here are various data structures used to store/represent/view raw pose data in a format which is appropriate for converting to and 
	 * from the flat vectorized representation used by various AnimDatabase systems. Pose data itself is made up of three parts - root data, local bone 
	 * transforms, and attribute data. The Pose Data structures here store pose data for multiple frames or characters, and allow operations in batch 
	 * along this dimension. They also store velocities for locations, rotations, and scales. Functions and data structures are provided for 
	 * performing conversion to global space using Forward Kinematics.
	 * 
	 * In most places these pose representations use single precision (one exception being the root location and global bone locations): this is 
	 * because neural-network-based methods are far from accurate enough to make use of double-precision anyway.
	 * 
	 * Right now this pose representation does not have complete support for LODs and while it can be used to represent only a subset of joints, it
	 * often encodes the full skeleton. In the future proper support will be added once support for LODs is implemented in the various AnimDatabase 
	 * systems that use this representation.
	 */

	//--------------------------------------------------

	/** Const View of Pose Root Data */
	struct FPoseRootDataConstView
	{
		UE_API int32 GetFrameNum() const;

		UE_API FPoseRootDataConstView Slice(const int32 FrameStart, const int32 FrameNum) const;

		TLearningArrayView<1, const FVector> RootLocations;
		TLearningArrayView<1, const FQuat4f> RootRotations;
		TLearningArrayView<1, const FVector3f> RootScales;
		TLearningArrayView<1, const FVector3f> RootLinearVelocities;
		TLearningArrayView<1, const FVector3f> RootAngularVelocities;
		TLearningArrayView<1, const FVector3f> RootScalarVelocities;
	};

	/** Non-Const View of Pose Root Data */
	struct FPoseRootDataView
	{
		UE_API int32 GetFrameNum() const;

		UE_API FPoseRootDataConstView ConstView() const;
		UE_API FPoseRootDataView Slice(const int32 FrameStart, const int32 FrameNum) const;

		TLearningArrayView<1, FVector> RootLocations;
		TLearningArrayView<1, FQuat4f> RootRotations;
		TLearningArrayView<1, FVector3f> RootScales;
		TLearningArrayView<1, FVector3f> RootLinearVelocities;
		TLearningArrayView<1, FVector3f> RootAngularVelocities;
		TLearningArrayView<1, FVector3f> RootScalarVelocities;
	};

	/** Pose Root Data */
	struct FPoseRootData
	{
		UE_API int32 GetFrameNum() const;

		UE_API void Resize(const int32 FrameNum);
		UE_API bool IsEmpty() const;
		UE_API void Empty();

		UE_API FPoseRootDataView View();
		UE_API FPoseRootDataConstView ConstView() const;
		UE_API FPoseRootDataView Slice(const int32 FrameStart, const int32 FrameNum);
		UE_API FPoseRootDataConstView ConstSlice(const int32 FrameStart, const int32 FrameNum) const;

		TLearningArray<1, FVector> RootLocations;
		TLearningArray<1, FQuat4f> RootRotations;
		TLearningArray<1, FVector3f> RootScales;
		TLearningArray<1, FVector3f> RootLinearVelocities;
		TLearningArray<1, FVector3f> RootAngularVelocities;
		TLearningArray<1, FVector3f> RootScalarVelocities;
	};

	//--------------------------------------------------

	/** Const View of Pose Local Bone Transforms and Velocities */
	struct FPoseLocalBoneDataConstView
	{
		UE_API int32 GetFrameNum() const;
		UE_API int32 GetBoneNum() const;

		UE_API FPoseLocalBoneDataConstView Slice(const int32 FrameStart, const int32 FrameNum) const;

		TLearningArrayView<2, const FVector3f> BoneLocations;
		TLearningArrayView<2, const FQuat4f> BoneRotations;
		TLearningArrayView<2, const FVector3f> BoneScales;
		TLearningArrayView<2, const FVector3f> BoneLinearVelocities;
		TLearningArrayView<2, const FVector3f> BoneAngularVelocities;
		TLearningArrayView<2, const FVector3f> BoneScalarVelocities;
	};

	/** Non-Const View of Pose Local Bone Transforms and Velocities */
	struct FPoseLocalBoneDataView
	{
		UE_API int32 GetFrameNum() const;
		UE_API int32 GetBoneNum() const;

		UE_API FPoseLocalBoneDataConstView ConstView() const;
		UE_API FPoseLocalBoneDataView Slice(const int32 FrameStart, const int32 FrameNum) const;

		TLearningArrayView<2, FVector3f> BoneLocations;
		TLearningArrayView<2, FQuat4f> BoneRotations;
		TLearningArrayView<2, FVector3f> BoneScales;
		TLearningArrayView<2, FVector3f> BoneLinearVelocities;
		TLearningArrayView<2, FVector3f> BoneAngularVelocities;
		TLearningArrayView<2, FVector3f> BoneScalarVelocities;
	};

	/** Pose Local Bone Transforms and Velocities */
	struct FPoseLocalBoneData
	{	
		UE_API int32 GetFrameNum() const;
		UE_API int32 GetBoneNum() const;

		UE_API void Resize(const int32 FrameNum, const int32 BoneNum);
		UE_API bool IsEmpty() const;
		UE_API void Empty();

		UE_API FPoseLocalBoneDataView View();
		UE_API FPoseLocalBoneDataConstView ConstView() const;
		UE_API FPoseLocalBoneDataView Slice(const int32 FrameStart, const int32 FrameNum);
		UE_API FPoseLocalBoneDataConstView ConstSlice(const int32 FrameStart, const int32 FrameNum) const;

		TLearningArray<2, FVector3f> BoneLocations;
		TLearningArray<2, FQuat4f> BoneRotations;
		TLearningArray<2, FVector3f> BoneScales;
		TLearningArray<2, FVector3f> BoneLinearVelocities;
		TLearningArray<2, FVector3f> BoneAngularVelocities;
		TLearningArray<2, FVector3f> BoneScalarVelocities;
	};

	//--------------------------------------------------

	/** Const View of Pose Global Bone Transforms and Velocities */
	struct FPoseGlobalBoneDataConstView
	{
		UE_API int32 GetFrameNum() const;
		UE_API int32 GetBoneNum() const;

		UE_API FPoseGlobalBoneDataConstView Slice(const int32 FrameStart, const int32 FrameNum) const;

		TLearningArrayView<2, const FVector> BoneLocations;
		TLearningArrayView<2, const FQuat4f> BoneRotations;
		TLearningArrayView<2, const FVector3f> BoneScales;
		TLearningArrayView<2, const FVector3f> BoneLinearVelocities;
		TLearningArrayView<2, const FVector3f> BoneAngularVelocities;
		TLearningArrayView<2, const FVector3f> BoneScalarVelocities;
	};

	/** Non-Const View of Pose Global Bone Transforms and Velocities */
	struct FPoseGlobalBoneDataView
	{
		UE_API int32 GetFrameNum() const;
		UE_API int32 GetBoneNum() const;

		UE_API FPoseGlobalBoneDataView Slice(const int32 FrameStart, const int32 FrameNum) const;

		TLearningArrayView<2, FVector> BoneLocations;
		TLearningArrayView<2, FQuat4f> BoneRotations;
		TLearningArrayView<2, FVector3f> BoneScales;
		TLearningArrayView<2, FVector3f> BoneLinearVelocities;
		TLearningArrayView<2, FVector3f> BoneAngularVelocities;
		TLearningArrayView<2, FVector3f> BoneScalarVelocities;
	};

	/** Pose Global Bone Transforms and Velocities */
	struct FPoseGlobalBoneData
	{
		UE_API int32 GetFrameNum() const;
		UE_API int32 GetBoneNum() const;

		UE_API void Resize(const int32 FrameNum, const int32 BoneNum);
		UE_API bool IsEmpty() const;
		UE_API void Empty();

		UE_API FPoseGlobalBoneDataView View();
		UE_API FPoseGlobalBoneDataConstView ConstView() const;
		UE_API FPoseGlobalBoneDataView Slice(const int32 FrameStart, const int32 FrameNum);
		UE_API FPoseGlobalBoneDataConstView ConstSlice(const int32 FrameStart, const int32 FrameNum) const;

		TLearningArray<2, FVector> BoneLocations;
		TLearningArray<2, FQuat4f> BoneRotations;
		TLearningArray<2, FVector3f> BoneScales;
		TLearningArray<2, FVector3f> BoneLinearVelocities;
		TLearningArray<2, FVector3f> BoneAngularVelocities;
		TLearningArray<2, FVector3f> BoneScalarVelocities;
	};

	//--------------------------------------------------

	/** Const View of Pose Attribute Data */
	struct FPoseAttributeDataConstView
	{
		UE_API int32 GetFrameNum() const;
		UE_API int32 GetAttributeNum() const;
		UE_API TLearningArrayView<1, const EAnimDatabaseAttributeType> GetAttributeTypes() const;
		UE_API TLearningArrayView<1, const FName> GetAttributeNames() const;

		UE_API FPoseAttributeDataConstView Slice(const int32 FrameStart, const int32 FrameNum) const;

		UE_API EAnimDatabaseAttributeType GetAttributeType(const int32 AttributeIdx) const;
		UE_API FName GetAttributeName(const int32 AttributeIdx) const;
		UE_API int32 GetAttributeOffset(const int32 AttributeIdx) const;
		UE_API int32 GetAttributeSize(const int32 AttributeIdx) const;
		UE_API bool GetAttributeActive(const int32 FrameIdx, const int32 AttributeIdx) const;

		UE_API bool GetBool(const int32 FrameIdx, const int32 AttributeIdx) const;
		UE_API float GetFloat(const int32 FrameIdx, const int32 AttributeIdx) const;
		UE_API float GetAngle(const int32 FrameIdx, const int32 AttributeIdx) const;
		UE_API FVector3f GetLocation(const int32 FrameIdx, const int32 AttributeIdx) const;
		UE_API FQuat4f GetRotation(const int32 FrameIdx, const int32 AttributeIdx) const;
		UE_API FVector3f GetScale(const int32 FrameIdx, const int32 AttributeIdx) const;
		UE_API FVector3f GetLinearVelocity(const int32 FrameIdx, const int32 AttributeIdx) const;
		UE_API FVector3f GetAngularVelocity(const int32 FrameIdx, const int32 AttributeIdx) const;
		UE_API FVector3f GetScalarVelocity(const int32 FrameIdx, const int32 AttributeIdx) const;
		UE_API FVector3f GetDirection(const int32 FrameIdx, const int32 AttributeIdx) const;
		UE_API FTransform3f GetTransform(const int32 FrameIdx, const int32 AttributeIdx) const;
		UE_API void GetEvent(bool& bOutTimeUntilEventKnown, float& OutTimeUntilEvent, const int32 FrameIdx, const int32 AttributeIdx) const;

		TLearningArrayView<1, const EAnimDatabaseAttributeType> AttributeTypes;
		TLearningArrayView<1, const FName> AttributeNames;
		TLearningArrayView<1, const int32> AttributeOffsets;
		TLearningArrayView<2, const bool> AttributeActive;
		TLearningArrayView<2, const float> AttributeData;
	};

	/** Non-Const View of Pose Attribute Data */
	struct FPoseAttributeDataView
	{
		UE_API int32 GetFrameNum() const;
		UE_API int32 GetAttributeNum() const;
		UE_API TLearningArrayView<1, const EAnimDatabaseAttributeType> GetAttributeTypes() const;
		UE_API TLearningArrayView<1, const FName> GetAttributeNames() const;

		UE_API FPoseAttributeDataConstView ConstView() const;
		UE_API FPoseAttributeDataView Slice(const int32 FrameStart, const int32 FrameNum) const;

		UE_API EAnimDatabaseAttributeType GetAttributeType(const int32 AttributeIdx) const;
		UE_API FName GetAttributeName(const int32 AttributeIdx) const;
		UE_API int32 GetAttributeOffset(const int32 AttributeIdx) const;
		UE_API int32 GetAttributeSize(const int32 AttributeIdx) const;
		UE_API bool GetAttributeActive(const int32 FrameIdx, const int32 AttributeIdx) const;

		UE_API bool GetBool(const int32 FrameIdx, const int32 AttributeIdx) const;
		UE_API float GetFloat(const int32 FrameIdx, const int32 AttributeIdx) const;
		UE_API float GetAngle(const int32 FrameIdx, const int32 AttributeIdx) const;
		UE_API FVector3f GetLocation(const int32 FrameIdx, const int32 AttributeIdx) const;
		UE_API FQuat4f GetRotation(const int32 FrameIdx, const int32 AttributeIdx) const;
		UE_API FVector3f GetScale(const int32 FrameIdx, const int32 AttributeIdx) const;
		UE_API FVector3f GetLinearVelocity(const int32 FrameIdx, const int32 AttributeIdx) const;
		UE_API FVector3f GetAngularVelocity(const int32 FrameIdx, const int32 AttributeIdx) const;
		UE_API FVector3f GetScalarVelocity(const int32 FrameIdx, const int32 AttributeIdx) const;
		UE_API FVector3f GetDirection(const int32 FrameIdx, const int32 AttributeIdx) const;
		UE_API FTransform3f GetTransform(const int32 FrameIdx, const int32 AttributeIdx) const;
		UE_API void GetEvent(bool& bOutTimeUntilEventKnown, float& OutTimeUntilEvent, const int32 FrameIdx, const int32 AttributeIdx) const;

		UE_API void SetAttributeActive(const int32 FrameIdx, const int32 AttributeIdx, const bool bActive) const;

		UE_API void SetBool(const int32 FrameIdx, const int32 AttributeIdx, const bool bValue) const;
		UE_API void SetFloat(const int32 FrameIdx, const int32 AttributeIdx, const float Value) const;
		UE_API void SetAngle(const int32 FrameIdx, const int32 AttributeIdx, const float Value) const;
		UE_API void SetLocation(const int32 FrameIdx, const int32 AttributeIdx, const FVector3f Location) const;
		UE_API void SetRotation(const int32 FrameIdx, const int32 AttributeIdx, const FQuat4f Rotation) const;
		UE_API void SetScale(const int32 FrameIdx, const int32 AttributeIdx, const FVector3f Scale) const;
		UE_API void SetLinearVelocity(const int32 FrameIdx, const int32 AttributeIdx, const FVector3f LinearVelocity) const;
		UE_API void SetAngularVelocity(const int32 FrameIdx, const int32 AttributeIdx, const FVector3f AngularVelocity) const;
		UE_API void SetScalarVelocity(const int32 FrameIdx, const int32 AttributeIdx, const FVector3f ScalarVelocity) const;
		UE_API void SetDirection(const int32 FrameIdx, const int32 AttributeIdx, const FVector3f Direction) const;
		UE_API void SetTransform(const int32 FrameIdx, const int32 AttributeIdx, const FTransform3f Transform) const;
		UE_API void SetEvent(const int32 FrameIdx, const int32 AttributeIdx, const bool bTimeUntilEventKnown, const float TimeUntilEvent) const;

		TLearningArrayView<1, EAnimDatabaseAttributeType> AttributeTypes;
		TLearningArrayView<1, FName> AttributeNames;
		TLearningArrayView<1, int32> AttributeOffsets;
		TLearningArrayView<2, bool> AttributeActive;
		TLearningArrayView<2, float> AttributeData;
	};

	/** Pose Attribute Data */
	struct FPoseAttributeData
	{
		UE_API int32 GetFrameNum() const;
		UE_API int32 GetAttributeNum() const;
		UE_API TLearningArrayView<1, const EAnimDatabaseAttributeType> GetAttributeTypes() const;
		UE_API TLearningArrayView<1, const FName> GetAttributeNames() const;

		UE_API void Resize(const int32 FrameNum, const TLearningArrayView<1, const EAnimDatabaseAttributeType> InAttributeTypes, const TLearningArrayView<1, const FName> InAttributeNames);
		UE_API bool IsEmpty() const;
		UE_API void Empty();

		UE_API FPoseAttributeDataView View();
		UE_API FPoseAttributeDataConstView ConstView() const;
		UE_API FPoseAttributeDataView Slice(const int32 FrameStart, const int32 FrameNum);
		UE_API FPoseAttributeDataConstView ConstSlice(const int32 FrameStart, const int32 FrameNum) const;

		UE_API EAnimDatabaseAttributeType GetAttributeType(const int32 AttributeIdx) const;
		UE_API FName GetAttributeName(const int32 AttributeIdx) const;
		UE_API int32 GetAttributeOffset(const int32 AttributeIdx) const;
		UE_API int32 GetAttributeSize(const int32 AttributeIdx) const;
		UE_API bool GetAttributeActive(const int32 FrameIdx, const int32 AttributeIdx) const;

		UE_API bool GetBool(const int32 FrameIdx, const int32 AttributeIdx) const;
		UE_API float GetFloat(const int32 FrameIdx, const int32 AttributeIdx) const;
		UE_API float GetAngle(const int32 FrameIdx, const int32 AttributeIdx) const;
		UE_API FVector3f GetLocation(const int32 FrameIdx, const int32 AttributeIdx) const;
		UE_API FQuat4f GetRotation(const int32 FrameIdx, const int32 AttributeIdx) const;
		UE_API FVector3f GetScale(const int32 FrameIdx, const int32 AttributeIdx) const;
		UE_API FVector3f GetLinearVelocity(const int32 FrameIdx, const int32 AttributeIdx) const;
		UE_API FVector3f GetAngularVelocity(const int32 FrameIdx, const int32 AttributeIdx) const;
		UE_API FVector3f GetScalarVelocity(const int32 FrameIdx, const int32 AttributeIdx) const;
		UE_API FVector3f GetDirection(const int32 FrameIdx, const int32 AttributeIdx) const;
		UE_API FTransform3f GetTransform(const int32 FrameIdx, const int32 AttributeIdx) const;
		UE_API void GetEvent(bool& bOutTimeUntilEventKnown, float& OutTimeUntilEvent, const int32 FrameIdx, const int32 AttributeIdx) const;

		TLearningArray<1, EAnimDatabaseAttributeType> AttributeTypes;
		TLearningArray<1, FName> AttributeNames;
		TLearningArray<1, int32> AttributeOffsets;
		TLearningArray<2, bool> AttributeActive;
		TLearningArray<2, float> AttributeData;
	};

	//--------------------------------------------------

	/** Const View of Pose Data */
	struct FPoseDataConstView
	{
		UE_API int32 GetFrameNum() const;
		UE_API int32 GetBoneNum() const;
		UE_API int32 GetAttributeNum() const;
		UE_API TLearningArrayView<1, const EAnimDatabaseAttributeType> GetAttributeTypes() const;
		UE_API TLearningArrayView<1, const FName> GetAttributeNames() const;

		UE_API FPoseDataConstView Slice(const int32 FrameStart, const int32 FrameNum) const;

		FPoseRootDataConstView RootData;
		FPoseLocalBoneDataConstView LocalBoneData;
		FPoseAttributeDataConstView AttributeData;
	};

	/** Non-Const View of Pose Data */
	struct FPoseDataView
	{
		UE_API int32 GetFrameNum() const;
		UE_API int32 GetBoneNum() const;
		UE_API int32 GetAttributeNum() const;
		UE_API TLearningArrayView<1, const EAnimDatabaseAttributeType> GetAttributeTypes() const;
		UE_API TLearningArrayView<1, const FName> GetAttributeNames() const;

		UE_API FPoseDataView Slice(const int32 FrameStart, const int32 FrameNum) const;

		FPoseRootDataView RootData;
		FPoseLocalBoneDataView LocalBoneData;
		FPoseAttributeDataView AttributeData;
	};

	/** Full Pose Data containing root data, local bone transforms, and attribute data */
	struct FPoseData
	{
		UE_API int32 GetFrameNum() const;
		UE_API int32 GetBoneNum() const;
		UE_API int32 GetAttributeNum() const;
		UE_API TLearningArrayView<1, const EAnimDatabaseAttributeType> GetAttributeTypes() const;
		UE_API TLearningArrayView<1, const FName> GetAttributeNames() const;

		UE_API void Resize(const int32 FrameNum, const int32 BoneNum, const TLearningArrayView<1, const EAnimDatabaseAttributeType> InAttributeTypes, const TLearningArrayView<1, const FName> InAttributeNames);
		UE_API bool IsEmpty() const;
		UE_API void Empty();

		UE_API FPoseDataView View();
		UE_API FPoseDataConstView ConstView() const;
		UE_API FPoseDataView Slice(const int32 FrameStart, const int32 FrameNum);
		UE_API FPoseDataConstView ConstSlice(const int32 FrameStart, const int32 FrameNum) const;

		FPoseRootData RootData;
		FPoseLocalBoneData LocalBoneData;
		FPoseAttributeData AttributeData;
	};

	//--------------------------------------------------

	namespace PoseData
	{
		/** Resets root data */
		UE_API void Reset(const FPoseRootDataView& OutRootData);

		/** Resets local bone data to the reference pose */
		UE_API void Reset(const FPoseLocalBoneDataView& OutPoseLocalBoneData, const TLearningArrayView<1, const FTransform> ReferenceTransforms, const TLearningArrayView<1, const int32> IgnoreBoneIndices = {});

		/** Reset attribute data to zero */
		UE_API void Reset(const FPoseAttributeDataView& OutAttributeData);

		/** Reset pose data to the reference pose */
		UE_API void Reset(const FPoseDataView& OutPoseData, const TLearningArrayView<1, const FTransform> ReferenceTransforms, const TLearningArrayView<1, const int32> IgnoreBoneIndices = {});


		/** Copy root data */
		UE_API void Copy(const FPoseRootDataView& OutRootData, const FPoseRootDataConstView& InRootData);

		/** Copy local pose data */
		UE_API void Copy(const FPoseLocalBoneDataView& OutPoseLocalBoneData, const FPoseLocalBoneDataConstView& InPoseLocalBoneData);

		/** Copy attribute data */
		UE_API void Copy(const FPoseAttributeDataView& OutAttributeData, const FPoseAttributeDataConstView& InAttributeData);

		/** Copy one pose to another */
		UE_API void Copy(const FPoseDataView& OutPoseData, const FPoseDataConstView& InPoseData);

		/**
		 * Compute Forward Kinematics on a subset of bones to produce global transforms from local transforms.
		 * 
		 * The OutPoseGlobalBoneData and InPoseLocalBoneData parameters are expected to be allocated for just the subset of bones being used, which 
		 * is then provided by BoneIndices. The BoneParents array however should still refer to bone indices in the full reference skeleton. All 
		 * required parent bones must be present in BoneIndices.
		 *
		 * @param OutPoseGlobalBoneData		Output Global Bone Transforms of shape (FrameNum, BoneIndicesNum)
		 * @param InPoseLocalBoneData		Input Local Bone Transforms of shape (FrameNum, BoneIndicesNum)
		 * @param InPoseRootData			Input Root Transforms of shape (FrameNum)
		 * @param BoneParents				Input Bone Parents Array of shape (BoneNum)
		 * @param BoneIndices				Input Bone Indices Array of shape (BoneIndicesNum)
		 */
		UE_API void ForwardKinematics(
			const FPoseGlobalBoneDataView& OutPoseGlobalBoneData,
			const FPoseLocalBoneDataConstView& InPoseLocalBoneData,
			const FPoseRootDataConstView& InPoseRootData,
			const TLearningArrayView<1, const int32> BoneParents,
			const UE::Learning::FIndexSet BoneIndices);

		/**
		 * Compute Forward Kinematics for all bones to produce global transforms from local transforms.
		 *
		 * @param OutPoseGlobalBoneData		Output Global Bone Transforms of shape (FrameNum, BoneNum)
		 * @param InPoseLocalBoneData		Input Local Bone Transforms of shape (FrameNum, BoneNum)
		 * @param InPoseRootData			Input Root Transforms of shape (FrameNum)
		 * @param BoneParents				Input Bone Parents Array of shape (BoneNum)
		 */
		UE_API void ForwardKinematics(
			const FPoseGlobalBoneDataView& OutPoseGlobalBoneData,
			const FPoseLocalBoneDataConstView& InPoseLocalBoneData,
			const FPoseRootDataConstView& InPoseRootData,
			const TLearningArrayView<1, const int32> BoneParents);

		/**
		 * Compute Forward Kinematics on a subset of bones to produce global transforms from local transforms.
		 *
		 * The OutPoseGlobalBoneData and InPoseLocalBoneData parameters are expected to be allocated for just the subset of bones being used, which
		 * is then provided by BoneIndices. The BoneParents array however should still refer to bone indices in the full reference skeleton. All
		 * required parent bones must be present in BoneIndices.
		 *
		 * @param OutPoseGlobalBoneData		Output Global Bone Transforms of shape (FrameNum, BoneIndicesNum)
		 * @param InPoseLocalBoneData		Input Local Bone Transforms of shape (FrameNum, BoneIndicesNum)
		 * @param InPoseRootData			Input Root Transforms of shape (FrameNum)
		 * @param BoneParents				Input Bone Parents Array of shape (BoneNum)
		 * @param BoneIndices				Input Bone Indices Array of shape (BoneIndicesNum)
		 * @param BonesToUpdate				Which bones to compute the global transforms for
		 */
		UE_API void ForwardKinematicsPartial(
			const FPoseGlobalBoneDataView& OutPoseGlobalBoneData,
			const FPoseLocalBoneDataConstView& InPoseLocalBoneData,
			const FPoseRootDataConstView& InPoseRootData,
			const TLearningArrayView<1, const int32> BoneParents,
			const UE::Learning::FIndexSet BoneIndices,
			const UE::Learning::FIndexSet BonesToUpdate);

		/**
		 * Computes the bone indices which are different from the reference pose transforms in the given pose data
		 * 
		 * @param OutBoneIndices			Output used bone indices
		 * @param InPoseLocalBoneData		Input Local Bone Transforms of shape (FrameNum, BoneNum)
		 * @param ReferenceTransforms		Reference pose transforms (BoneNum)
		 * @param bRootIsAlwaysUsed			If to always include the root bone (bone 0) in the used bones
		 * @param Eps						Epsilon to use when comparing bone transforms
		 */
		UE_API void ComputeUsedBones(
			TArray<int32>& OutBoneIndices,
			const FPoseLocalBoneDataConstView& InPoseLocalBoneData,
			const TLearningArrayView<1, const FTransform> ReferenceTransforms,
			const bool bRootIsAlwaysUsed = true,
			const float Eps = UE_KINDA_SMALL_NUMBER);

		/**
		 * Computes the default bone values and required bone indices for some given pose data.
		 * 
		 * Here, the default bone values will be their average/mean in the dataset, unless they are included in ExcludedBoneIndices, in which case 
		 * they will be the transforms in the reference pose.
		 * 
		 * The required bone indices will be all of the bones which have some variation in the dataset (i.e. are not just described by the mean 
		 * value). This is found by computing the standard deviation of each of the bone properties and checking if they are below the given 
		 * thresholds. These thresholds are important because sometimes things like mirroring and compression can produce very small numerical 
		 * variation in the data which does not really imply any visual difference and so these bones are not required to be predicted.
		 *
		 * @param OutDefaultBoneLocations	Output default bone locations (BoneNum)
		 * @param OutDefaultBoneRotations	Output default bone rotations (BoneNum)
		 * @param OutDefaultBoneScales		Output default bone scales (BoneNum)
		 * @param OutBoneLocationIndices	Output required bone location indices
		 * @param OutBoneRotationIndices	Output required bone rotation indices
		 * @param OutBoneScaleIndices		Output required bone scale indices
		 * @param InPoseLocalBoneData		Input Local Bone Transforms of shape (FrameNum, BoneNum)
		 * @param ReferenceTransforms		Reference pose transforms (BoneNum)
		 * @param ExcludedBoneIndices		Bones in exclude. Will use the reference transform as the default.
		 * @param BoneLocationThreshold		Threshold for including a bone position in cm
		 * @param BoneRotationThreshold		Threshold for including a bone rotation in radians
		 * @param BoneLogScaleThreshold		Threshold for including a bone in log scale space
		 */
		UE_API void ComputeDefaultBoneValuesAndIndices(
			TLearningArrayView<1, FVector3f> OutDefaultBoneLocations,
			TLearningArrayView<1, FQuat4f> OutDefaultBoneRotations,
			TLearningArrayView<1, FVector3f> OutDefaultBoneScales,
			TArray<int32>& OutBoneLocationIndices,
			TArray<int32>& OutBoneRotationIndices,
			TArray<int32>& OutBoneScaleIndices,
			const FPoseLocalBoneDataConstView& InPoseLocalBoneData,
			const TLearningArrayView<1, const FTransform> ReferenceTransforms,
			const TLearningArrayView<1, const int32> ExcludedBoneIndices,
			const float BoneLocationThreshold = 1.0f, // Std of 1.0 cm
			const float BoneRotationThreshold = 0.0174533f, // Std of 1.0 deg (in radians)
			const float BoneLogScaleThreshold = 0.1f);  // Std of 0.1 Log Scale

		/** Computes the full set of required bones given the individual bones required for location, rotation, and scales */
		UE_API void ComputeRequiredBoneIndices(
			TArray<int32>& OutBoneRequiredIndices,
			const TLearningArrayView<1, const int32> BoneLocationIndices,
			const TLearningArrayView<1, const int32> BoneRotationIndices,
			const TLearningArrayView<1, const int32> BoneScaleIndices);

		/** Gets the encoding size for a given attribute type */
		UE_API int32 AttributeTypeEncodingSize(const EAnimDatabaseAttributeType Type);

		/** Gets the pose vector size for a given set of bones and attributes. */
		UE_API int32 PoseVectorSize(
			const int32 BoneLocationNum,
			const int32 BoneRotationNum,
			const int32 BoneScaleNum,
			const TLearningArrayView<1, const EAnimDatabaseAttributeType> AttributeTypes);

		/** Initialize a pose vector with some set of default bones. */
		UE_API void SetDefaultPoseData(
			const FPoseDataView& OutPoseData,
			const TLearningArrayView<1, const FVector3f> DefaultBoneLocations,
			const TLearningArrayView<1, const FQuat4f> DefaultBoneRotations,
			const TLearningArrayView<1, const FVector3f> DefaultBoneScales);

		/** Initialize a pose vector with some set of default bones, where the pose encodes only the given set of bones. */
		UE_API void SetDefaultPoseData(
			const FPoseDataView& OutPoseData,
			const TLearningArrayView<1, const FVector3f> DefaultBoneLocations,
			const TLearningArrayView<1, const FQuat4f> DefaultBoneRotations,
			const TLearningArrayView<1, const FVector3f> DefaultBoneScales,
			const Learning::FIndexSet BoneIndices);

		/** Convert a pose to a pose vector. */
		UE_API void ToPoseVectors(
			const TLearningArrayView<2, float> OutPoseVectors,
			const FPoseDataConstView& InPoseData,
			const Learning::FIndexSet BoneLocationIndices,
			const Learning::FIndexSet BoneRotationIndices,
			const Learning::FIndexSet BoneScaleIndices,
			const TLearningArrayView<1, const FVector3f> DefaultBoneLocations,
			const TLearningArrayView<1, const FQuat4f> DefaultBoneRotations,
			const TLearningArrayView<1, const FVector3f> DefaultBoneScales);

		/** Convert a pose to a pose vector, where the pose encodes only the given set of bones. */
		UE_API void ToPoseVectors(
			const TLearningArrayView<2, float> OutPoseVectors,
			const FPoseDataConstView& InPoseData,
			const Learning::FIndexSet BoneLocationIndices,
			const Learning::FIndexSet BoneRotationIndices,
			const Learning::FIndexSet BoneScaleIndices,
			const TLearningArrayView<1, const FVector3f> DefaultBoneLocations,
			const TLearningArrayView<1, const FQuat4f> DefaultBoneRotations,
			const TLearningArrayView<1, const FVector3f> DefaultBoneScales,
			const Learning::FIndexSet BoneIndices);

		/** Get a pose from a pose vector. */
		UE_API void FromPoseVectors(
			const FPoseDataView& OutPoseData,
			const TLearningArrayView<2, const float> InPoseVectors,
			const TLearningArrayView<1, const FVector> InRootLocations,
			const TLearningArrayView<1, const FQuat4f> InRootRotations,
			const Learning::FIndexSet BoneLocationIndices,
			const Learning::FIndexSet BoneRotationIndices,
			const Learning::FIndexSet BoneScaleIndices,
			const TLearningArrayView<1, const FVector3f> DefaultBoneLocations,
			const TLearningArrayView<1, const FQuat4f> DefaultBoneRotations,
			const TLearningArrayView<1, const FVector3f> DefaultBoneScales);

		/** Get a pose from a pose vector, where the pose encodes only the given set of bones. */
		UE_API void FromPoseVectors(
			const FPoseDataView& OutPoseData,
			const TLearningArrayView<2, const float> InPoseVectors,
			const TLearningArrayView<1, const FVector> InRootLocations,
			const TLearningArrayView<1, const FQuat4f> InRootRotations,
			const Learning::FIndexSet BoneLocationIndices,
			const Learning::FIndexSet BoneRotationIndices,
			const Learning::FIndexSet BoneScaleIndices,
			const TLearningArrayView<1, const FVector3f> DefaultBoneLocations,
			const TLearningArrayView<1, const FQuat4f> DefaultBoneRotations,
			const TLearningArrayView<1, const FVector3f> DefaultBoneScales,
			const Learning::FIndexSet BoneIndices);

		/** Fits normalization offset and scale to a set of pose vectors */
		UE_API void FitPoseVectorNormalization(
			TLearningArrayView<1, float> OutPoseVectorOffset,
			TLearningArrayView<1, float> OutPoseVectorScale,
			const TLearningArrayView<2, const float> InPoseVectors,
			const int32 BoneLocationNum,
			const int32 BoneRotationNum,
			const int32 BoneScaleNum,
			const TLearningArrayView<1, const EAnimDatabaseAttributeType> AttributeTypes);

		/** Compute pose vector weights using descendant bone lengths */
		UE_API void ComputePoseVectorWeightsFromDescendantBoneLengths(
			TLearningArrayView<1, float> OutPoseVectorWeights,
			const TLearningArrayView<1, const FTransform> ReferenceTransforms,
			const TLearningArrayView<1, const int32> BoneParents,
			const Learning::FIndexSet BoneLocationIndices,
			const Learning::FIndexSet BoneRotationIndices,
			const Learning::FIndexSet BoneScaleIndices,
			const float RootWeight = 10.0f);

		/** Compute pose vector weights that allow for error compensation down the joint chain */
		UE_API void ComputePoseVectorWeightsCylinderApprox(
			TLearningArrayView<1, float> OutPoseVectorWeights,
			const TLearningArrayView<1, const FTransform> ReferenceTransforms,
			const TLearningArrayView<1, const int32> BoneParents,
			const Learning::FIndexSet RequiredIndices,
			const Learning::FIndexSet BoneLocationIndices,
			const Learning::FIndexSet BoneRotationIndices,
			const Learning::FIndexSet BoneScaleIndices,
			const float RootWeight = 10.0f,
			const float BoneBaseWeightMultiplier = 1.0f,
			const float AttributeWeight = 10.0f);

		/** Makes the provided pose data loop seamlessly */
		UE_API void MakeLooped(
			const FPoseDataView& InOutPoseData, 
			const FFrameRate FrameRate,
			const float StartEndRatio,
			const float BlendInTime,
			const float BlendOutTime);

		/** Replaces the given bones in the data in the middle with an interpolation between the first and last frames */
		UE_API void PatchInterpolate(const FPoseDataView& InOutPoseData, const FFrameRate FrameRate, const bool bApplyToRoot, const UE::Learning::FIndexSet Bones);

		/** Recomputes Velocities using finite difference assuming the first dimension is frames */
		UE_API void RecomputeVelocities(const FPoseRootDataView& InOutRootData, const float DeltaTime);

		/** Teleport some pose data to a new location and rotation. Adjusts the velocities appropriately. */
		UE_API void Teleport(
			const FPoseRootDataView& InOutRootData,
			const TLearningArrayView<1, const FVector> RootLocations,
			const TLearningArrayView<1, const FQuat4f> RootRotations);

		/** Blend root data using a given alpha value */
		UE_API void BlendRoot(
			const FPoseRootDataView& OutPoseData,
			const FPoseRootDataConstView& InPoseA,
			const FPoseRootDataConstView& InPoseB,
			const float Alpha);

		/** Integrates the root location and rotation using the velocities. */
		UE_API void IntegrateRoot(const FPoseRootDataView& InOutRootData, const float DeltaTime);

		/** Blend pose data of two poses, ensuring to blend via the given set of rotations. Updates the blend via rotations. */
		UE_API void BlendVia(
			const FPoseLocalBoneDataView& OutPoseData,
			const TLearningArrayView<2, FQuat4f> InOutViaRotations,
			const FPoseLocalBoneDataConstView& InPoseA,
			const FPoseLocalBoneDataConstView& InPoseB,
			const float Alpha);

		/** Fit extrapolation half-lives given source and target pose data */
		UE_API void FitExtrapolationHalfLives(
			const TLearningArrayView<2, FVector3f> OutBoneLocationHalfLives,
			const TLearningArrayView<2, FVector3f> OutBoneRotationHalfLives,
			const TLearningArrayView<2, FVector3f> OutBoneScaleHalfLives,
			const FPoseLocalBoneDataConstView& InPoseSource,
			const FPoseLocalBoneDataConstView& InPoseTarget,
			const float ExtrapolationHalfLife = 1.0f,
			const float ExtrapolationHalfLifeMin = 0.05f,
			const float ExtrapolationHalfLifeMax = 1.0f,
			const float MaxLinearVelocity = 500.0f,
			const float MaxAngularVelocity = UE_TWO_PI,
			const float MaxScalarVelocity = 4.0f);

		/** Extrapolate the root data forward given a half-life */
		UE_API void ExtrapolateRoot(
			const FPoseRootDataView& OutPoseData,
			const FPoseRootDataConstView& InPoseData,
			const float Time,
			const float RootHalfLife = 1.0f);

		/** Extrapolate pose data forward in time given sets of half-lives */
		UE_API void Extrapolate(
			const FPoseLocalBoneDataView& OutPoseData,
			const FPoseLocalBoneDataConstView& InPoseData,
			const float Time,
			const TLearningArrayView<2, const FVector3f> BoneLocationHalfLives,
			const TLearningArrayView<2, const FVector3f> BoneRotationHalfLives,
			const TLearningArrayView<2, const FVector3f> BoneScaleHalfLives);

		/** Fit inertialization offsets to the provided root data */
		UE_API void FitInertializationOffsetsRoot(
			const FPoseRootDataView& OutOffsetPose,
			const FPoseRootDataConstView& InPoseSource,
			const FPoseRootDataConstView& InPoseTarget);

		/** Apply inertialization offsets to the given root data */
		UE_API void ApplyInertializationOffsetsRoot(
			const FPoseRootDataView& OutFinalPose,
			const FPoseRootDataConstView& InOffsetPose,
			const FPoseRootDataConstView& InPoseTarget,
			const float BlendTime,
			const float BlendDuration);

		/** Fit inertialization offsets to the provided pose data */
		UE_API void FitInertializationOffsets(
			const FPoseLocalBoneDataView& OutOffsetPose,
			const FPoseLocalBoneDataConstView& InPoseSource,
			const FPoseLocalBoneDataConstView& InPoseTarget);

		/** Apply inertialization offsets to the given pose data */
		UE_API void ApplyInertializationOffsets(
			const FPoseLocalBoneDataView& OutFinalPose,
			const FPoseLocalBoneDataConstView& InOffsetPose,
			const FPoseLocalBoneDataConstView& InPoseTarget,
			const float BlendTime,
			const float BlendDuration);

		/** Fit inertialization offsets to the provided attribute data */
		UE_API void FitInertializationOffsetsAttributes(
			const FPoseAttributeDataView& OutOffsetPose,
			const FPoseAttributeDataConstView& InPoseSource,
			const FPoseAttributeDataConstView& InPoseTarget);

		/** Apply inertialization offsets to the given attribute data */
		UE_API void ApplyInertializationOffsetsAttributes(
			const FPoseAttributeDataView& OutFinalPose,
			const FPoseAttributeDataConstView& InOffsetPose,
			const FPoseAttributeDataConstView& InPoseTarget,
			const float BlendTime,
			const float BlendDuration);

		/** Update root data in-place using a mix of integrating the velocity and the next target pose */
		UE_API void UpdateInplaceUsingVelocityIntegrationMix(
			const FPoseRootDataView& InOutPose,
			const FPoseRootDataConstView& InTargetPose,
			const float DeltaTime,
			const float VelocityAlpha = 0.25f);

		/** Update pose data in-place using a mix of integrating the velocity and the next target pose */
		UE_API void UpdateInplaceUsingVelocityIntegrationMix(
			const FPoseLocalBoneDataView& InOutPose,
			const FPoseLocalBoneDataConstView& InTargetPose,
			const float DeltaTime,
			const float VelocityAlpha = 0.25f);

		/** Update pose data in-place using a mix of integrating the velocity and the next target pose */
		UE_API void DampInplaceUsingVelocityIntegrationMix(
			const FPoseLocalBoneDataView& InOutPose,
			const FPoseLocalBoneDataConstView& InTargetPose,
			const float DeltaTime,
			const float PoseSmoothingTime = 0.2f);

		/** Update pose data in-place using an adaptive mix of integrating the velocity and the next target pose based on the absolute velocity magnitude */
		UE_API void DampInplaceUsingAdaptiveVelocityIntegrationMix(
			const FPoseLocalBoneDataView& InOutPose,
			const FPoseLocalBoneDataConstView& InTargetPose,
			const float DeltaTime,
			const float MinPoseSmoothingTime = 0.0f,
			const float MaxPoseSmoothingTime = 0.25f,
			const float LinearVelocitySmoothingThreshold = 5.0f,
			const float AngularVelocitySmoothingThreshold = UE_PI / 8.0f,
			const float ScalarVelocitySmoothingThreshold = 0.5f);

		/** Update pose data representing a sequence of frames in-place using an adaptive mix of integrating the velocity and the next target pose based on the absolute velocity magnitude */
		UE_API void DampFramesInplaceUsingAdaptiveVelocityIntegrationMix(
			const FPoseLocalBoneDataView& InOutPoses,
			const float DeltaTime,
			const float MinPoseSmoothingTime = 0.0f,
			const float MaxPoseSmoothingTime = 0.25f,
			const float LinearVelocitySmoothingThreshold = 5.0f,
			const float AngularVelocitySmoothingThreshold = UE_PI / 8.0f,
			const float ScalarVelocitySmoothingThreshold = 0.5f);

		/** Update attributes in-place damping them toward the target attributes with the given smoothing time */
		UE_API void DampAttributesInPlace(
			const FPoseAttributeDataView& InOutAttributes,
			const FPoseAttributeDataConstView& InTargetAttributes,
			const float DeltaTime,
			const float SmoothingTime);

		/** Helper functions for performing contact pinning */
		namespace ContactPinning
		{
			/** Const View of Contact States */
			struct FContactStateConstView
			{
				UE_API int32 GetFrameNum() const { return Locked.Num<0>(); }
				UE_API int32 GetBoneNum() const { return Locked.Num<1>(); }
				UE_API FContactStateConstView Slice(const int32 FrameStart, const int32 FrameNum) const;

				TLearningArrayView<2, const bool> Locked;
				TLearningArrayView<2, const float> TimeSinceTransition;
				TLearningArrayView<2, const FVector> Position;
				TLearningArrayView<2, const FVector3f> Velocity;
				TLearningArrayView<2, const FVector> Point;
				TLearningArrayView<2, const FVector3f> OffsetPosition;
				TLearningArrayView<2, const FVector3f> OffsetVelocity;
			};

			/** Non-Const View of Contact States */
			struct FContactStateView
			{
				UE_API int32 GetFrameNum() const { return Locked.Num<0>(); }
				UE_API int32 GetBoneNum() const { return Locked.Num<1>(); }
				UE_API FContactStateView Slice(const int32 FrameStart, const int32 FrameNum) const;

				TLearningArrayView<2, bool> Locked;
				TLearningArrayView<2, float> TimeSinceTransition;
				TLearningArrayView<2, FVector> Position;
				TLearningArrayView<2, FVector3f> Velocity;
				TLearningArrayView<2, FVector> Point;
				TLearningArrayView<2, FVector3f> OffsetPosition;
				TLearningArrayView<2, FVector3f> OffsetVelocity;
			};

			/** Contact State data */
			struct FContactState
			{
				UE_API int32 GetFrameNum() const { return Locked.Num<0>(); }
				UE_API int32 GetBoneNum() const { return Locked.Num<1>(); }
				UE_API void Resize(const int32 FrameNum, const int32 BoneNum);
				UE_API bool IsEmpty() const;
				UE_API void Empty();

				UE_API FContactStateView View();
				UE_API FContactStateConstView ConstView() const;
				UE_API FContactStateView Slice(const int32 FrameStart, const int32 FrameNum);
				UE_API FContactStateConstView ConstSlice(const int32 FrameStart, const int32 FrameNum) const;

				TLearningArray<2, bool> Locked;
				TLearningArray<2, float> TimeSinceTransition;
				TLearningArray<2, FVector> Position;
				TLearningArray<2, FVector3f> Velocity;
				TLearningArray<2, FVector> Point;
				TLearningArray<2, FVector3f> OffsetPosition;
				TLearningArray<2, FVector3f> OffsetVelocity;
			};

			/** Contact look-at IK helper function */
			UE_API void ContactLookAtIK(
				FQuat4f& InOutBoneRotation,
				const FQuat4f GlobalParentRotation,
				const FQuat4f GlobalRotation,
				const FVector GlobalPosition,
				const FVector ChildPosition,
				const FVector TargetPosition,
				const float Eps = 1e-5f);

			/** Contact two-bone IK helper function */
			UE_API void ContactTwoBoneIK(
				FQuat4f& BoneRootLocalRotation,
				FQuat4f& BoneMidLocalRotation,
				const FVector BoneRoot,
				const FVector BoneMid,
				const FVector BoneEnd,
				const FVector Target,
				const FVector3f RotationAxis,
				const FQuat4f BoneRootGlobalRotation,
				const FQuat4f BoneMidGlobalRotation,
				const FQuat4f BoneParentGlobalRotation,
				const float MaxLengthBuffer = 1.0f);

			/** Resets the contact pinning state given the pose data and toe indices */
			UE_API void ResetContactPinning(
				const FContactStateView& OutContactState,
				const FPoseGlobalBoneDataConstView& InPoseGlobalBoneData,
				const TArrayView<const int32> ToeBoneIndices);

			/** Contact pinning settings */
			struct FContactPinningSettings
			{
				float BlendTime = 0.25f;
				float LockRadius = 5.0f;
				float UnlockRadius = 20.0f;
				float ContactHeight = 0.0f;
				float MinToeHeight = 0.0f;
				float MinAnkleHeight = 7.0f;
				float ToeLength = 5.0f;
				float HyperExtensionLimit = 0.5f;
			};

			/** Updates the contact pinning state using the provided pose data, bone indices, and settings */
			UE_API void UpdateContactPinning(
				const FContactStateView& InOutContactState,
				const FPoseGlobalBoneDataView& InOutPoseGlobalBoneData,
				const FPoseLocalBoneDataView& InOutPoseLocalBoneData,
				const FPoseRootDataConstView& InPoseRootData,
				const FPoseAttributeDataConstView& InPoseAttributeData,
				const TArrayView<const int32> InBoneParents,
				const TArrayView<const int32> ContactStateAttributeIndices,
				const TArrayView<const int32> PelvisBoneIndices,
				const TArrayView<const int32> HipBoneIndices,
				const TArrayView<const int32> KneeBoneIndices,
				const TArrayView<const int32> AnkleBoneIndices,
				const TArrayView<const int32> ToeBoneIndices,
				const TArrayView<const FVector3f> KneeSideVectors,
				const TArrayView<const FVector3f> ToeForwardVectors,
				const float DeltaTime,
				const FContactPinningSettings& Settings = FContactPinningSettings());

			/** Removes Foot-Ground penetration without pinning the feet */
			UE_API void RemoveFootGroundPenetration(
				const FPoseGlobalBoneDataView& InOutPoseGlobalBoneData,
				const FPoseLocalBoneDataView& InOutPoseLocalBoneData,
				const FPoseRootDataConstView& InPoseRootData,
				const TArrayView<const int32> InBoneParents,
				const TArrayView<const int32> PelvisBoneIndices,
				const TArrayView<const int32> HipBoneIndices,
				const TArrayView<const int32> KneeBoneIndices,
				const TArrayView<const int32> AnkleBoneIndices,
				const TArrayView<const int32> ToeBoneIndices,
				const TArrayView<const FVector3f> KneeSideVectors,
				const TArrayView<const FVector3f> ToeForwardVectors,
				const FContactPinningSettings& Settings = FContactPinningSettings());
		}
	}

	//--------------------------------------------------
}

/**
 * Provides a simple Blueprint-accessible wrapper around Pose Data.
 *
 * This is a struct that can be used to provide access to pose data to blueprint code. Since it references a batch of pose data the PoseIdx variable
 * can be set to say which pose the blueprint code is accessing. This also records the DeltaTime and the previous pose data since this allows for the
 * detection of the occurrence (and prediction) of events in the attribute data.
 */
USTRUCT(BlueprintType)
struct FAnimDatabasePoseState
{
	GENERATED_BODY()

public:

	/** Check if the given pose state is valid */
	UE_API bool IsValid() const;

	/** Initialize the pose state */
	UE_API void Init(const UE::AnimDatabase::FPoseDataConstView& PoseData, const TLearningArrayView<1, const int32> InBoneParents);

public:

	int32 PoseIdx = 0;
	TLearningArray<1, int32> BoneParents;
	TSharedPtr<UE::AnimDatabase::FPoseData, ESPMode::ThreadSafe> CurrPoseData;
	TSharedPtr<UE::AnimDatabase::FPoseGlobalBoneData, ESPMode::ThreadSafe> PoseGlobalBoneData;
};

/** This provides a blueprint interface to the pose state. */
UCLASS(BlueprintType, meta = (BlueprintThreadSafe))
class UAnimDatabasePoseStateLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/** Construct a new Empty Pose State */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FAnimDatabasePoseState MakePoseState();

	/** Copy a PoseState */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase", meta=(NonBlueprintThreadSafe))
	static UE_API void PoseStateCopy(UPARAM(ref) FAnimDatabasePoseState& OutPoseState, const FAnimDatabasePoseState& InPoseState);

	/** Get the root transform of the pose state */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FTransform PoseStateRootTransform(const FAnimDatabasePoseState& PoseState);

	/** Get the root location of the pose state */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FVector PoseStateRootLocation(const FAnimDatabasePoseState& PoseState);

	/** Get the root rotation of the pose state */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FRotator PoseStateRootRotation(const FAnimDatabasePoseState& PoseState);

	/** Get the root location and rotation of the pose state */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API void PoseStateRootLocationAndRotation(FVector& OutRootLocation, FRotator& OutRootRotation, const FAnimDatabasePoseState& PoseState);

	/** Get the root scale of the pose state */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FVector PoseStateRootScale3D(const FAnimDatabasePoseState& PoseState);

	/** Get the root direction of the pose state */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FVector PoseStateRootDirection(const FAnimDatabasePoseState& PoseState, const FVector ForwardVector = FVector(0.0f, 1.0f, 0.0f));

	/** Get the root linear velocity of the pose state */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FVector PoseStateRootLinearVelocity(const FAnimDatabasePoseState& PoseState);

	/** Get the root angular velocity of the pose state */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FVector PoseStateRootAngularVelocity(const FAnimDatabasePoseState& PoseState);

	/** Get the root scalar velocity of the pose state */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FVector PoseStateRootScalarVelocity(const FAnimDatabasePoseState& PoseState);

	/** Get the world location of a given bone in the pose state */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FVector PoseStateBoneWorldLocation(const FAnimDatabasePoseState& PoseState, const int32 BoneIndex);

	/** Get the world rotation of a given bone in the pose state */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FRotator PoseStateBoneWorldRotation(const FAnimDatabasePoseState& PoseState, const int32 BoneIndex);

	/** Get the world linear velocity of a given bone in the pose state */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FVector PoseStateBoneWorldLinearVelocity(const FAnimDatabasePoseState& PoseState, const int32 BoneIndex);

	/** Get the world transform of a given bone in the pose state */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FTransform PoseStateBoneWorldTransform(const FAnimDatabasePoseState& PoseState, const int32 BoneIndex);

	/** Get if the number of attributes in the pose state */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API int32 PoseStateAttributeNum(const FAnimDatabasePoseState& PoseState);

	/** Get if the given attribute is active in the pose state */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API bool PoseStateIsAttributeActive(const FAnimDatabasePoseState& PoseState, const int32 AttributeIndex);

	/** Get if the given attribute name in the pose state */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FName PoseStateAttributeName(const FAnimDatabasePoseState& PoseState, const int32 AttributeIndex);

	/** Get if the given attribute type in the pose state */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API EAnimDatabaseAttributeType PoseStateAttributeType(const FAnimDatabasePoseState& PoseState, const int32 AttributeIndex);

	/** Gets a string representation of the given attribute in the pose state */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FString PoseStateAttributeString(const FAnimDatabasePoseState& PoseState, const int32 AttributeIndex);

	/** Get the given attribute as a bool */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API bool PoseStateBoolAttribute(const FAnimDatabasePoseState& PoseState, const int32 AttributeIndex);

	/** Get the given attribute as a float */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API float PoseStateFloatAttribute(const FAnimDatabasePoseState& PoseState, const int32 AttributeIndex);

	/** Get the given attribute as a angle in degrees */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API float PoseStateAngleAttributeDegrees(const FAnimDatabasePoseState& PoseState, const int32 AttributeIndex);

	/** Get the given attribute as a angle in radians */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API float PoseStateAngleAttributeRadians(const FAnimDatabasePoseState& PoseState, const int32 AttributeIndex);

	/** Get the given attribute as a location */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FVector PoseStateLocationAttribute(const FAnimDatabasePoseState& PoseState, const int32 AttributeIndex);

	/** Get the given attribute as a rotation */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FRotator PoseStateRotationAttribute(const FAnimDatabasePoseState& PoseState, const int32 AttributeIndex);
	static UE_API FQuat PoseStateRotationAttributeAsQuat(const FAnimDatabasePoseState& PoseState, const int32 AttributeIndex);

	/** Get the given attribute as a scale */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FVector PoseStateScaleAttribute(const FAnimDatabasePoseState& PoseState, const int32 AttributeIndex);

	/** Get the given attribute as a linear velocity */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static FVector PoseStateLinearVelocityAttribute(const FAnimDatabasePoseState& PoseState, const int32 AttributeIndex);

	/** Get the given attribute as an angular velocity */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FVector PoseStateAngularVelocityAttribute(const FAnimDatabasePoseState& PoseState, const int32 AttributeIndex);

	/** Get the given attribute as a scalar velocity */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FVector PoseStateScalarVelocityAttribute(const FAnimDatabasePoseState& PoseState, const int32 AttributeIndex);

	/** Get the given attribute as a direction */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FVector PoseStateDirectionAttribute(const FAnimDatabasePoseState& PoseState, const int32 AttributeIndex);

	/** Get the given attribute as a transform */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API FTransform PoseStateTransformAttribute(const FAnimDatabasePoseState& PoseState, const int32 AttributeIndex);

	/** Get the given attribute as an event */
	UFUNCTION(BlueprintPure, Category = "AnimDatabase")
	static UE_API void PoseStateEventAttribute(bool& bOutTimeUntilEventKnown, float& OutTimeUntilEvent, const FAnimDatabasePoseState& PoseState, const int32 AttributeIndex);
};

#undef UE_API