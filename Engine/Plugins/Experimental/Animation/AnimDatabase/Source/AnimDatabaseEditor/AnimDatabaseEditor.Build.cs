// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AnimDatabaseEditor : ModuleRules
{
	public AnimDatabaseEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicIncludePaths.AddRange(
			new string[] {
				// ... add public include paths required here ...
			}
			);
				
		
		PrivateIncludePaths.AddRange(
			new string[] {
				// ... add other private include paths required here ...
			}
			);
			
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"AnimDatabase",
				"Learning",
				// ... add other public dependencies that you statically link with here ...
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
				"InputCore",
				"EditorFramework",
				"UnrealEd",
				"LevelEditor",
				"DetailCustomizations",
				"EditorStyle",
				"Projects",
				"AssetDefinition",
				"EditorWidgets",
				"PropertyEditor",
				"KismetWidgets",
				"SequencerWidgets",
				"AdvancedPreviewScene",
				"GraphEditor",
				"ClassViewer",
				"RenderCore",
				"BlueprintGraph",
				"Learning",
				"Json",
				"JsonUtilities",
				"Blutility",
				"TimeManagement",
				"DrawDebugLibrary",
			}
			);
		
		
		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			}
			);
	}
}
