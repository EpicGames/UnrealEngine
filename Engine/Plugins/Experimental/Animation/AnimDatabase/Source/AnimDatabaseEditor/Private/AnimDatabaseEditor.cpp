// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimDatabaseEditor.h"

#include "AnimDatabaseEditorMode.h"
#include "AnimDatabaseEditorToolkit.h"

#include "PropertyEditorModule.h"

#define LOCTEXT_NAMESPACE "FAnimDatabaseEditorModule"

void FAnimDatabaseEditorModule::StartupModule()
{
	// Register Ed Mode used to interact with the preview scene
	FEditorModeRegistry::Get().RegisterMode<UE::AnimDatabase::Editor::FDatabaseMode>(UE::AnimDatabase::Editor::FDatabaseMode::EditorModeId, LOCTEXT("AnimDatabaseEditorModeName", "AnimDatabase"));

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomClassLayout("AnimDatabaseQuery", FOnGetDetailCustomizationInstance::CreateStatic(&UE::AnimDatabase::Editor::FQueryDetails::MakeInstance));
}

void FAnimDatabaseEditorModule::ShutdownModule()
{
	// Unregister Ed Mode
	FEditorModeRegistry::Get().UnregisterMode(UE::AnimDatabase::Editor::FDatabaseMode::EditorModeId);

	if (FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.UnregisterCustomClassLayout("AnimDatabaseQuery");
	}
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FAnimDatabaseEditorModule, AnimDatabaseEditor)