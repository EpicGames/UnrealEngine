// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimDatabaseEditorMode.h"
#include "AnimDatabaseEditorToolkit.h"
#include "AnimDatabaseEditorViewportClient.h"
#include "AnimDatabaseEditorPreviewScene.h"
#include "AnimDatabaseEditorTimeline.h"

#include "AnimDatabase.h"
#include "AnimDatabaseFrameAttribute.h"
#include "AnimDatabaseFrameRanges.h"

#include "LearningNeuralNetwork.h"
#include "LearningFrameSet.h"
#include "LearningFrameRangeSet.h"
#include "LearningFrameAttribute.h"

#include "GameFramework/Character.h"
#include "Components/PoseableMeshComponent.h"
#include "Animation/AnimSequence.h"
#include "Animation/Skeleton.h"
#include "Engine/SkeletalMesh.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#include "Async/ParallelFor.h"
#include "EngineUtils.h"
#include "SceneView.h"
#include "DrawDebugLibrary.h"
#include "Preferences/PersonaOptions.h"

#define LOCTEXT_NAMESPACE "AnimDatabaseEditorMode"

namespace UE::AnimDatabase::Editor
{
	namespace Database::Private
	{
		/** Computes the spatial offset for a range given the character index and various settings */
		static inline FVector GetRangeOffset(
			const int32 CharacterIdx, 
			const int32 CharacterNum, 
			const float Spacing,
			const bool bOffsetRanges,
			const bool bOffsetRangesInGrid)
		{
			const int32 GridRows = FMath::CeilToInt(FMath::Sqrt((float)CharacterNum));

			return
				bOffsetRanges ?
				bOffsetRangesInGrid ?
				FVector(
					Spacing * (CharacterIdx % GridRows) - 0.5f * Spacing * (GridRows - 1),
					Spacing * (CharacterIdx / GridRows) - 0.5f * Spacing * (GridRows - 1), 0.0f) :
				FVector(
					Spacing * CharacterIdx - 0.5f * Spacing * (CharacterNum - 1), 0.0f, 0.0f) :
				FVector::ZeroVector;
		}

		/** Adjusts a character's location, rotation, and velocities based off some offset, relative translation and various settings */
		static inline void AdjustTransformRootTransform(
			FVector& InOutLocation,
			FQuat4f& InOutRotation,
			FVector3f& InOutLinearVelocity,
			FVector3f& InOutAngularVelocity,
			const FVector Offset,
			const FTransform RelativeTransform,
			const bool bDisableRootTranslation,
			const bool bDisableRootRotation)
		{
			const FTransform RootOffsetTransform = FTransform(FQuat::Identity, Offset, FVector::OneVector);
			const FTransform RootAdjustmenTransform = RelativeTransform.Inverse() * RootOffsetTransform;

			if (bDisableRootTranslation && bDisableRootRotation)
			{
				const FQuat4f RootRotationOffset = InOutRotation;
				InOutLocation = RootOffsetTransform.GetLocation();
				InOutRotation = FQuat4f::Identity;
				InOutLinearVelocity = RootRotationOffset.UnrotateVector(InOutLinearVelocity);
				InOutAngularVelocity = RootRotationOffset.UnrotateVector(InOutAngularVelocity);
			}
			else if (bDisableRootTranslation)
			{
				InOutLocation = RootOffsetTransform.GetLocation();
				InOutRotation = ((FQuat4f)RootAdjustmenTransform.GetRotation()) * InOutRotation;
				InOutLinearVelocity = (FVector3f)RootAdjustmenTransform.TransformVectorNoScale((FVector)InOutLinearVelocity);
				InOutAngularVelocity = (FVector3f)RootAdjustmenTransform.TransformVectorNoScale((FVector)InOutAngularVelocity);
			}
			else 
			{
				InOutLocation = RootAdjustmenTransform.TransformPosition(InOutLocation);
				InOutRotation = ((FQuat4f)RootAdjustmenTransform.GetRotation()) * InOutRotation;
				InOutLinearVelocity = (FVector3f)RootAdjustmenTransform.TransformVectorNoScale((FVector)InOutLinearVelocity);
				InOutAngularVelocity = (FVector3f)RootAdjustmenTransform.TransformVectorNoScale((FVector)InOutAngularVelocity);
			}
		}
	}

	const FEditorModeID FDatabaseMode::EditorModeId = TEXT("AnimDatabaseEditorDatabaseMode");

	void FDatabaseMode::Tick(FEditorViewportClient* EditorViewportClient, float DeltaTime)
	{
		FEdMode::Tick(EditorViewportClient, DeltaTime);

		// Get the viewport client

		FViewportClient* ViewportClient = static_cast<FViewportClient*>(EditorViewportClient);
		check(ViewportClient);
		ViewportClient->ClearWarningMessage();

		// Grab the Editor Toolkit

		TSharedPtr<FDatabaseToolkit> EditorToolKit = StaticCastSharedPtr<FDatabaseToolkit>(ViewportClient->GetAssetEditorToolkit().Pin());
		if (!EditorToolKit.IsValid()) { return; }

		WeakToolkit = EditorToolKit.ToWeakPtr();

		TSharedPtr<FPreviewScene> PreviewScene = ViewportClient->GetPreviewScene().Pin();
		if (!PreviewScene.IsValid()) { return; }

		// Get the database

		UAnimDatabase* Database = EditorToolKit->Database;
		check(Database);

		if (!Database->GetSkeleton())
		{
			ViewportClient->SetWarningMessage(LOCTEXT("NoSkeletonSelected", "Warning: No skeleton selected for database."));
			return;
		}

		// Add the required number of characters to the scene

		while (PreviewScene->GetCharacterNum() < MaxCharacterNum)
		{
			const int32 CharacterIdx = PreviewScene->AddCharacter();
			PreviewScene->SetCharacterVisibility(CharacterIdx, false);
		}

		// Update the Desired Skeletal Meshes

		USkinnedAsset* DesiredSkeletalMesh = Database->GetSkeleton() ? Database->GetSkeleton()->GetPreviewMesh() : nullptr;
		DesiredSkeletalMesh = Database->ViewportSettings->PreviewMesh ? Database->ViewportSettings->PreviewMesh : DesiredSkeletalMesh;

		for (int32 CharacterIdx = 0; CharacterIdx < MaxCharacterNum; CharacterIdx++)
		{
			USkinnedAsset* CurrentSkeletalMesh = PreviewScene->GetPoseableMeshComponent(CharacterIdx)->GetSkinnedAsset();

			if (DesiredSkeletalMesh && DesiredSkeletalMesh != CurrentSkeletalMesh)
			{
				PreviewScene->GetPoseableMeshComponent(CharacterIdx)->SetSkinnedAssetAndUpdate(DesiredSkeletalMesh, true);
			}
		}

		// Update Query

		Database->Query->RangeIdentifierColorSeed = Database->ViewportSettings->RangeIdentifierColorSeed;
		const bool bQueryUpdated = Database->Query->Update();

		if (bQueryUpdated)
		{
			ActiveRanges.Reset();
			EditorToolKit->QueryWidget->ForceRefresh();
		}

		// Compute all range trajectories

		if (bQueryUpdated || RangeTrajectoryLocations.IsEmpty())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FDatabaseMode::Tick::Trajectories);

			const int32 QueryRangeNum = Database->Query->QueryRanges.IsValid() ?
				Database->Query->QueryRanges.FrameRangeSet->GetTotalRangeNum() : 0;

			RangeTrajectoryLocations.SetNum(QueryRangeNum);
			RangeTrajectoryRotations.SetNum(QueryRangeNum);

			if (Database->Query->QueryRanges.IsValid())
			{
				Database->WaitForCompressionOnAnimSequencesFromArrayView(Database->Query->QueryRanges.FrameRangeSet->GetEntrySequences());
			}

			ParallelFor(QueryRangeNum, [this, Database](int32 RangeIdx)
				{
					int32 RangeEntry = INDEX_NONE, RangeRange = INDEX_NONE;
					const bool bFound = Database->Query->QueryRanges.FrameRangeSet->FindTotalRange(RangeEntry, RangeRange, RangeIdx);
					check(bFound);

					const int32 RangeFrameNum = Database->Query->QueryRanges.FrameRangeSet->GetEntryRangeLength(RangeEntry, RangeRange);
					const int32 RangeSequence = Database->Query->QueryRanges.FrameRangeSet->GetEntrySequence(RangeEntry);
					const int32 RangeStart = Database->Query->QueryRanges.FrameRangeSet->GetEntryRangeStart(RangeEntry, RangeRange);

					RangeTrajectoryLocations[RangeIdx].SetNumUninitialized({ RangeFrameNum });
					RangeTrajectoryRotations[RangeIdx].SetNumUninitialized({ RangeFrameNum });

					Database->GetRootLocation(RangeTrajectoryLocations[RangeIdx], RangeSequence, RangeStart);
					Database->GetRootRotation(RangeTrajectoryRotations[RangeIdx], RangeSequence, RangeStart);
				});
		}

		// Adjust Number of visible characters in the scene

		const int32 CharacterNum = FMath::Min(Database->Query->SelectedRanges.Num(), MaxCharacterNum);

		for (int32 CharacterIdx = 0; CharacterIdx < MaxCharacterNum; CharacterIdx++)
		{
			PreviewScene->SetCharacterVisibility(CharacterIdx, CharacterIdx < CharacterNum);
		}

		// If selection has changed adjust view ranges

		bool bSelectedRangesUpdated = false;

		if (bQueryUpdated || ActiveRanges != Database->Query->SelectedRanges)
		{
			bSelectedRangesUpdated = true;
			ActiveRanges = Database->Query->SelectedRanges;

			// Add error message if too many ranges selected

			if (ActiveRanges.Num() > MaxCharacterNum)
			{
				ViewportClient->SetWarningMessage(FText::Format(
					LOCTEXT("AnimDatabaseTooManyRangesWarning", "Warning: Rendering characters for {0} for {1} selected ranges."),
					FText::AsNumber(MaxCharacterNum),
					FText::AsNumber(ActiveRanges.Num())));
			}

			// Adjust the timeline for the newly selected ranges


			if (ActiveRanges.Num() == 0)
			{
				EditorToolKit->TimelineModel->SetTimelineFrameRate(Database->GetFrameRate());
				EditorToolKit->TimelineModel->SetTimelinePlaybackRange(TRange<FFrameNumber>(0, 0));
				EditorToolKit->TimelineModel->SetTimelineWorkingRange(TRange<FFrameNumber>(0, 0));
				EditorToolKit->TimelineModel->SetTimelineViewRange(TRange<FFrameNumber>(0, 0));
				EditorToolKit->TimelineModel->SetTimelineFrameTime(FFrameTime::FromDecimal(0));
			}
			else if (ActiveRanges.Num() == 1)
			{
				const int32 RangeStart = Database->Query->QueryRanges.FrameRangeSet->GetAllRangeStarts()[ActiveRanges[0]];
				const int32 RangeLength = Database->Query->QueryRanges.FrameRangeSet->GetAllRangeLengths()[ActiveRanges[0]];

				EditorToolKit->TimelineModel->SetTimelineFrameRate(Database->GetFrameRate());
				EditorToolKit->TimelineModel->SetTimelinePlaybackRange(TRange<FFrameNumber>(RangeStart, RangeStart + RangeLength - 1));
				EditorToolKit->TimelineModel->SetTimelineWorkingRange(TRange<FFrameNumber>(RangeStart, RangeStart + RangeLength - 1));
				EditorToolKit->TimelineModel->SetTimelineViewRange(TRange<FFrameNumber>(RangeStart, RangeStart + RangeLength - 1));
				EditorToolKit->TimelineModel->SetTimelineFrameTime(FFrameTime::FromDecimal(RangeStart));
			}
			else
			{
				const FFrameTime CurrentTimelineTime = EditorToolKit->TimelineModel->GetTimelineFrameTime();

				int32 RangeLength = 1;
				for (const int32 ActiveRangeIdx : ActiveRanges)
				{
					RangeLength = FMath::Max(Database->Query->QueryRanges.FrameRangeSet->GetAllRangeLengths()[ActiveRangeIdx], RangeLength);
				}

				EditorToolKit->TimelineModel->SetTimelineFrameRate(Database->GetFrameRate());
				EditorToolKit->TimelineModel->SetTimelinePlaybackRange(TRange<FFrameNumber>(0, RangeLength - 1));
				EditorToolKit->TimelineModel->SetTimelineWorkingRange(TRange<FFrameNumber>(0, RangeLength - 1));
				EditorToolKit->TimelineModel->SetTimelineViewRange(TRange<FFrameNumber>(0, RangeLength - 1));
				EditorToolKit->TimelineModel->SetTimelineFrameTime(FFrameTime::FromDecimal(0));
			}
		}

		// Re-build frames / frame ranges / frame attributes

		if (bQueryUpdated)
		{
			const int32 FramesNum = Database->Query->AdditionalFrames.Num();
			FramesObjects.SetNum(FramesNum);
			FramesNames.SetNum(FramesNum);

			const int32 FrameRangesNum = Database->Query->AdditionalFrameRanges.Num();
			FrameRangesObjects.SetNum(FrameRangesNum);
			FrameRangeNames.SetNum(FrameRangesNum);

			const int32 FrameAttributesNum = Database->Query->AdditionalFrameAttributes.Num();
			FrameAttributeObjects.SetNum(FrameAttributesNum);
			FrameAttributeTypes.SetNum(FrameAttributesNum);
			FrameAttributeNames.SetNum(FrameAttributesNum);

			if (FramesNum > 0 || FrameRangesNum > 0 || FrameAttributesNum > 0)
			{
				FScopedSlowTask SlowTask(0.0f, LOCTEXT("RebuildingFrameAttributes", "Rebuilding Additional Frames / Frame Ranges / Frame Attributes in Database"));
				SlowTask.MakeDialog();

				for (int32 FramesIdx = 0; FramesIdx < FramesNum; FramesIdx++)
				{
					FramesObjects[FramesIdx] = UAnimDatabaseFramesLibrary::MakeFramesFromFunction(
						Database, Database->Query->QueryRanges, Database->Query->AdditionalFrames[FramesIdx].Frames);

					if (!FramesObjects[FramesIdx].IsValid())
					{
						FramesObjects[FramesIdx] = UAnimDatabaseFramesLibrary::MakeEmptyFrames();
					}

					FramesNames[FramesIdx] = Database->Query->AdditionalFrames[FramesIdx].Name;
				}

				for (int32 FrameRangesIdx = 0; FrameRangesIdx < FrameRangesNum; FrameRangesIdx++)
				{
					FrameRangesObjects[FrameRangesIdx] = UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromFunction(
						Database, Database->Query->QueryRanges, Database->Query->AdditionalFrameRanges[FrameRangesIdx].FrameRanges);

					if (!FrameRangesObjects[FrameRangesIdx].IsValid())
					{
						FrameRangesObjects[FrameRangesIdx] = UAnimDatabaseFrameRangesLibrary::MakeEmptyFrameRanges();
					}

					FrameRangeNames[FrameRangesIdx] = Database->Query->AdditionalFrameRanges[FrameRangesIdx].Name;
				}

				for (int32 FrameAttributeIdx = 0; FrameAttributeIdx < FrameAttributesNum; FrameAttributeIdx++)
				{
					FrameAttributeObjects[FrameAttributeIdx] = UAnimDatabaseFrameAttributeLibrary::MakeFrameAttributeFromFunction(
						Database, Database->Query->QueryRanges, Database->Query->AdditionalFrameAttributes[FrameAttributeIdx].FrameAttribute);

					if (!FrameAttributeObjects[FrameAttributeIdx].IsValid())
					{
						FrameAttributeObjects[FrameAttributeIdx] = UAnimDatabaseFrameAttributeLibrary::MakeEmptyFrameAttribute();
					}

					FrameAttributeTypes[FrameAttributeIdx] = FrameAttributeObjects[FrameAttributeIdx].Type;
					FrameAttributeNames[FrameAttributeIdx] = Database->Query->AdditionalFrameAttributes[FrameAttributeIdx].Name;
				}
			}
		}

		// If selection has changed or additional frames/frame ranges has changed update tracks

		if (bSelectedRangesUpdated)
		{
			EditorToolKit->TracksModel->ResetTracks();

			TArray<TSubclassOf<UAnimNotify>> TrackAnimNotifies;
			TArray<TSubclassOf<UAnimNotifyState>> TrackAnimNotifyStates;
			TArray<FName> TrackSyncMarkers;
			TArray<FName> TrackAnimCurves;
			TArray<FName, TInlineAllocator<32>> TrackFrames;
			TArray<FName, TInlineAllocator<32>> TrackFrameRanges;
			TArray<FName, TInlineAllocator<32>> TrackFrameAttributes;
			TArray<FAnimDatabaseFrames, TInlineAllocator<32>> TrackFramesStruct;
			TArray<FAnimDatabaseFrameRanges, TInlineAllocator<32>> TrackFrameRangesStruct;
			TArray<FAnimDatabaseFrameAttribute, TInlineAllocator<32>> TrackFrameAttributesStruct;

			for (const int32 ActiveRangeIdx : ActiveRanges)
			{
				const int32 ViewOffset = ActiveRanges.Num() > 1 ? Database->Query->QueryRanges.FrameRangeSet->GetAllRangeStarts()[ActiveRangeIdx] : 0;
				int32 EntryIdx = INDEX_NONE, EntryRangeIdx = INDEX_NONE;
				Database->Query->QueryRanges.FrameRangeSet->FindTotalRange(EntryIdx, EntryRangeIdx, ActiveRangeIdx);
				const int32 SequenceIdx = Database->Query->QueryRanges.FrameRangeSet->GetEntrySequence(EntryIdx);
				const int32 RangeStart = Database->Query->QueryRanges.FrameRangeSet->GetEntryRangeStart(EntryIdx, EntryRangeIdx);
				const int32 RangeLength = Database->Query->QueryRanges.FrameRangeSet->GetEntryRangeLength(EntryIdx, EntryRangeIdx);

				const FAnimDatabaseFrameRanges SequenceRanges =
					ActiveRanges.Num() > 1 ?
					UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromSequenceRange(Database, SequenceIdx, RangeStart, RangeLength) :
					UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromSequenceIndex(Database, SequenceIdx);

				// Make Frames

				UAnimDatabaseFrameRangesLibrary::FindAnimNotifyClassesInFrameRanges(
					TrackAnimNotifies,
					Database,
					SequenceRanges);

				UAnimDatabaseFrameRangesLibrary::FindSyncMarkersInFrameRanges(
					TrackSyncMarkers,
					Database,
					SequenceRanges);

				const int32 AnimNotifyNum = TrackAnimNotifies.Num();
				TrackFrames.Reset();
				TrackFramesStruct.Reset();
				for (int32 AnimNotifyIdx = 0; AnimNotifyIdx < AnimNotifyNum; AnimNotifyIdx++)
				{
					const FAnimDatabaseFrames NotifyFrames = UAnimDatabaseFramesLibrary::MakeFramesFromAnimNotify(Database, SequenceRanges, TrackAnimNotifies[AnimNotifyIdx]);

					if (NotifyFrames.IsValid() && !NotifyFrames.FrameSet->IsEmpty())
					{
						TrackFrames.Add(TrackAnimNotifies[AnimNotifyIdx]->GetFName());
						TrackFramesStruct.Add(NotifyFrames);
					}
				}

				const int32 SyncMarkerNum = TrackSyncMarkers.Num();
				for (int32 SyncMarkerIdx = 0; SyncMarkerIdx < SyncMarkerNum; SyncMarkerIdx++)
				{
					const FAnimDatabaseFrames SyncMarkerFramess = UAnimDatabaseFramesLibrary::MakeFramesFromSyncMarker(Database, SequenceRanges, TrackSyncMarkers[SyncMarkerIdx]);

					if (SyncMarkerFramess.IsValid() && !SyncMarkerFramess.FrameSet->IsEmpty())
					{
						TrackFrames.Add(TrackSyncMarkers[SyncMarkerIdx]);
						TrackFramesStruct.Add(SyncMarkerFramess);
					}
				}

				const int32 AdditionalFramesNum = FramesObjects.Num();
				for (int32 AdditionalIdx = 0; AdditionalIdx < AdditionalFramesNum; AdditionalIdx++)
				{
					if (FramesObjects[AdditionalIdx].IsValid())
					{
						const FAnimDatabaseFrames SequenceAdditionalFrames = UAnimDatabaseFrameRangesLibrary::FramesFrameRangesIntersection(FramesObjects[AdditionalIdx], SequenceRanges);

						if (SequenceAdditionalFrames.IsValid() && !SequenceAdditionalFrames.FrameSet->IsEmpty())
						{
							TrackFrames.Add(FramesNames[AdditionalIdx]);
							TrackFramesStruct.Add(SequenceAdditionalFrames);
						}
					}
				}

				// Make Frame Ranges

				UAnimDatabaseFrameRangesLibrary::FindAnimNotifyStateClassesInFrameRanges(
					TrackAnimNotifyStates,
					Database,
					SequenceRanges);

				const int32 AnimNotifyStateNum = TrackAnimNotifyStates.Num();
				TrackFrameRanges.Reset();
				TrackFrameRangesStruct.Reset();
				for (int32 AnimNotifyStateIdx = 0; AnimNotifyStateIdx < AnimNotifyStateNum; AnimNotifyStateIdx++)
				{
					const FAnimDatabaseFrameRanges NotifyStateFrameRanges = UAnimDatabaseFrameRangesLibrary::MakeFrameRangesFromAnimNotifyState(Database, SequenceRanges, TrackAnimNotifyStates[AnimNotifyStateIdx]);

					if (NotifyStateFrameRanges.IsValid() && !NotifyStateFrameRanges.FrameRangeSet->IsEmpty())
					{
						TrackFrameRanges.Add(TrackAnimNotifyStates[AnimNotifyStateIdx]->GetFName());
						TrackFrameRangesStruct.Add(NotifyStateFrameRanges);
					}
				}

				const int32 AdditionalFrameRangesNum = FrameRangesObjects.Num();
				for (int32 AdditionalIdx = 0; AdditionalIdx < AdditionalFrameRangesNum; AdditionalIdx++)
				{
					if (FrameRangesObjects[AdditionalIdx].IsValid())
					{
						const FAnimDatabaseFrameRanges SequenceAdditionalFrameRanges = UAnimDatabaseFrameRangesLibrary::FrameRangesIntersection(FrameRangesObjects[AdditionalIdx], SequenceRanges);

						if (SequenceAdditionalFrameRanges.IsValid() && !SequenceAdditionalFrameRanges.FrameRangeSet->IsEmpty())
						{
							TrackFrameRanges.Add(FrameRangeNames[AdditionalIdx]);
							TrackFrameRangesStruct.Add(SequenceAdditionalFrameRanges);
						}
					}
				}

				// Make Frame Attributes

				UAnimDatabaseFrameRangesLibrary::FindCurvesInFrameRanges(
					TrackAnimCurves,
					Database,
					SequenceRanges);

				const int32 CurveNum = TrackAnimCurves.Num();
				TrackFrameAttributes.Reset();
				TrackFrameAttributesStruct.Reset();
				for (int32 CurveIdx = 0; CurveIdx < CurveNum; CurveIdx++)
				{
					const FAnimDatabaseFrameAttribute CurveFrameAttribute = UAnimDatabaseFrameAttributeLibrary::MakeFloatFrameAttributeFromCurveWhenActive(Database, SequenceRanges, TrackAnimCurves[CurveIdx]);

					if (CurveFrameAttribute.IsValid() && !CurveFrameAttribute.FrameAttribute->IsEmpty())
					{
						TrackFrameAttributes.Add(TrackAnimCurves[CurveIdx]);
						TrackFrameAttributesStruct.Add(CurveFrameAttribute);
					}
				}

				const int32 AdditionalFrameAttributeNum = FrameAttributeObjects.Num();
				for (int32 AdditionalIdx = 0; AdditionalIdx < AdditionalFrameAttributeNum; AdditionalIdx++)
				{
					if (FrameAttributeObjects[AdditionalIdx].IsValid())
					{
						const FAnimDatabaseFrameAttribute SequenceAdditionalFrameAttribute = UAnimDatabaseFrameAttributeLibrary::FrameAttributeIntersection(FrameAttributeObjects[AdditionalIdx], SequenceRanges);

						if (SequenceAdditionalFrameAttribute.IsValid() && !SequenceAdditionalFrameAttribute.FrameAttribute->IsEmpty())
						{
							TrackFrameAttributes.Add(FrameAttributeNames[AdditionalIdx]);
							TrackFrameAttributesStruct.Add(SequenceAdditionalFrameAttribute);
						}
					}
				}

				// Add Track

				EditorToolKit->TracksModel->AddTrack(
					Database->Query->QueryEntries[ActiveRangeIdx]->Sequence,
					Database->Query->QueryEntries[ActiveRangeIdx]->Color,
					TrackFrames,
					TrackFrameRanges,
					TrackFrameAttributes,
					TrackFramesStruct,
					TrackFrameRangesStruct,
					TrackFrameAttributesStruct,
					ViewOffset);
			}

			// Reset Accumulated Root Motion
			AccumulatedRootLocations.SetNumUninitialized({ CharacterNum });
			AccumulatedRootRotations.SetNumUninitialized({ CharacterNum });
			Learning::Array::Set(AccumulatedRootLocations, FVector::ZeroVector);
			Learning::Array::Set(AccumulatedRootRotations, FQuat4f::Identity);

			// Reconstruct the timeline widget for the current selection
			EditorToolKit->ReconstructTimelineWidget();
		}
	
		// Update Character Range Data

		CharacterSequenceIndices.SetNumUninitialized(CharacterNum);
		CharacterRangeTimes.SetNumUninitialized(CharacterNum);
		CharacterRangeStarts.SetNumUninitialized(CharacterNum);
		CharacterRangeLengths.SetNumUninitialized(CharacterNum);

		for (int32 CharacterIdx = 0; CharacterIdx < CharacterNum; CharacterIdx++)
		{
			int32 TestEntryIdx = INDEX_NONE, TestEntryRangeIdx = INDEX_NONE;
			const bool bFound = Database->Query->QueryRanges.FrameRangeSet->FindTotalRange(TestEntryIdx, TestEntryRangeIdx, ActiveRanges[CharacterIdx]);
			check(bFound);

			CharacterSequenceIndices[CharacterIdx] = Database->Query->QueryRanges.FrameRangeSet->GetEntrySequence(TestEntryIdx);
			CharacterRangeStarts[CharacterIdx] = Database->Query->QueryRanges.FrameRangeSet->GetAllRangeStarts()[ActiveRanges[CharacterIdx]];
			CharacterRangeLengths[CharacterIdx] = Database->Query->QueryRanges.FrameRangeSet->GetAllRangeLengths()[ActiveRanges[CharacterIdx]];

			CharacterRangeTimes[CharacterIdx] = 0.0;
			if (ActiveRanges.Num() == 1)
			{
				CharacterRangeTimes[CharacterIdx] = EditorToolKit->TimelineModel->GetTimelineFrameTime().AsDecimal() / Database->GetFrameRate().AsDecimal();
			}
			else
			{
				CharacterRangeTimes[CharacterIdx] = FMath::Clamp(
					EditorToolKit->TimelineModel->GetTimelineFrameTime().AsDecimal() + (float)CharacterRangeStarts[CharacterIdx],
					(float)CharacterRangeStarts[CharacterIdx],
					(float)(CharacterRangeStarts[CharacterIdx] + CharacterRangeLengths[CharacterIdx] - 1)) / Database->GetFrameRate().AsDecimal();
			}

			// Clamp some small epsilon within the duration to help with sampling of frame attributes.
			CharacterRangeTimes[CharacterIdx] = FMath::Clamp(CharacterRangeTimes[CharacterIdx],
				0.0f, FMath::Max(Database->GetSequenceDuration(CharacterSequenceIndices[CharacterIdx]) - UE_KINDA_SMALL_NUMBER, 0.0f));
		}

		// Evaluate Poses
		
		if (!PoseState.IsValid())
		{
			PoseState = UAnimDatabasePoseStateLibrary::MakePoseState();
		}

		Database->GetBoneParents(BoneParents);
		const int32 BoneNum = Database->GetBoneNum();
		PoseState.CurrPoseData->Resize(CharacterNum, BoneNum, FrameAttributeTypes, FrameAttributeNames);
		PoseState.PoseGlobalBoneData->Resize(CharacterNum, BoneNum);

		Database->WaitForCompressionOnAnimSequences(CharacterSequenceIndices);

		if (Database->ViewportSettings->bAccumulateRootMotion && EditorToolKit->TimelineModel->GetTransportPlaybackMode() == EPlaybackMode::Type::PlayingForward)
		{
			Database->SamplePoseData(
				PoseState.CurrPoseData->View(),
				CharacterSequenceIndices,
				CharacterRangeTimes,
				FrameAttributeObjects);

			for (int32 CharacterIdx = 0; CharacterIdx < CharacterNum; CharacterIdx++)
			{
				const FVector3f LocalLinearVelocity = PoseState.CurrPoseData->RootData.RootRotations[CharacterIdx].UnrotateVector(PoseState.CurrPoseData->RootData.RootLinearVelocities[CharacterIdx]);
				const FVector3f LocalAngularVelocity = PoseState.CurrPoseData->RootData.RootRotations[CharacterIdx].UnrotateVector(PoseState.CurrPoseData->RootData.RootAngularVelocities[CharacterIdx]);

				const float PlaybackDeltaTime = DeltaTime * EditorToolKit->TimelineModel->GetTimelinePlaybackRate();
				PoseState.CurrPoseData->RootData.RootLocations[CharacterIdx] = PlaybackDeltaTime * (FVector)AccumulatedRootRotations[CharacterIdx].RotateVector(LocalLinearVelocity) + AccumulatedRootLocations[CharacterIdx];
				PoseState.CurrPoseData->RootData.RootRotations[CharacterIdx] = FQuat4f::MakeFromRotationVector(PlaybackDeltaTime * AccumulatedRootRotations[CharacterIdx].RotateVector(LocalAngularVelocity)) * AccumulatedRootRotations[CharacterIdx];
			}
		}
		else
		{
			Database->SamplePoseData(
				PoseState.CurrPoseData->View(),
				CharacterSequenceIndices,
				CharacterRangeTimes,
				FrameAttributeObjects);
		}

		AccumulatedRootLocations = PoseState.CurrPoseData->RootData.RootLocations;
		AccumulatedRootRotations = PoseState.CurrPoseData->RootData.RootRotations;

		// Adjust root based on range settings

		if (Database->ViewportSettings->bRangesStartAtOrigin)
		{
			for (int32 CharacterIdx = 0; CharacterIdx < CharacterNum; CharacterIdx++)
			{
				const FVector RootOffset = Database::Private::GetRangeOffset(
					CharacterIdx, CharacterNum,
					Database->ViewportSettings->RangeOffsetSpacing,
					Database->ViewportSettings->bOffsetRanges,
					Database->ViewportSettings->bOffsetRangesInGrid);

				FTransform RangeStartRootTransform = FTransform::Identity;
				Database->GetRootTransform(TLearningArrayView<1, FTransform>(&RangeStartRootTransform, 1), 
					CharacterSequenceIndices[CharacterIdx], 
					CharacterRangeStarts[CharacterIdx]);

				Database::Private::AdjustTransformRootTransform(
					PoseState.CurrPoseData->RootData.RootLocations[CharacterIdx],
					PoseState.CurrPoseData->RootData.RootRotations[CharacterIdx],
					PoseState.CurrPoseData->RootData.RootLinearVelocities[CharacterIdx],
					PoseState.CurrPoseData->RootData.RootAngularVelocities[CharacterIdx],
					RootOffset,
					RangeStartRootTransform,
					Database->ViewportSettings->bDisableRootTranslation,
					Database->ViewportSettings->bDisableRootRotation);
			}
		}

		// Forward Kinematics

		PoseData::ForwardKinematics(
			PoseState.PoseGlobalBoneData->View(),
			PoseState.CurrPoseData->LocalBoneData.ConstView(),
			PoseState.CurrPoseData->RootData.ConstView(),
			BoneParents);

		// Update Pose

		for (int32 CharacterIdx = 0; CharacterIdx < CharacterNum; CharacterIdx++)
		{
			if (PreviewScene->GetPoseableMeshComponent(CharacterIdx)->GetSkinnedAsset())
			{
				for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
				{
					const FTransform BoneTransform = FTransform(
						((FQuat)PoseState.PoseGlobalBoneData->BoneRotations[CharacterIdx][BoneIdx]).GetNormalized(),
						(FVector)PoseState.PoseGlobalBoneData->BoneLocations[CharacterIdx][BoneIdx],
						(FVector)PoseState.PoseGlobalBoneData->BoneScales[CharacterIdx][BoneIdx]);

					PreviewScene->GetPoseableMeshComponent(CharacterIdx)->SetBoneTransformByName(Database->GetBoneName(BoneIdx), BoneTransform, EBoneSpaces::WorldSpace);
				}

				PreviewScene->GetPoseableMeshComponent(CharacterIdx)->RefreshBoneTransforms();
			}
		}

		// Update test range time if playing animation

		EditorToolKit->TimelineModel->UpdateTimelineFrameTime(DeltaTime * Database->GetFrameRate().AsDecimal());
	}

	void FDatabaseMode::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
	{
		if (TSharedPtr<FDatabaseToolkit> ToolKit = WeakToolkit.Pin())
		{
			if (!ToolKit->Database->Skeleton) { return; }

			const int32 CharacterNum = FMath::Min(ActiveRanges.Num(), MaxCharacterNum);

			// Draw Skeleton

			if (ToolKit->Database->ViewportSettings->bDrawSkeleton)
			{
				// Find bone indices to render

				const int32 BoneNum = ToolKit->Database->GetBoneNum();

				TArray<int32> BoneIndices;

				if (ToolKit->Database->ViewportSettings->bDrawMeshBonesOnly)
				{
					USkinnedAsset* DesiredSkeletalMesh = ToolKit->Database->GetSkeleton() ? ToolKit->Database->GetSkeleton()->GetPreviewMesh() : nullptr;
					DesiredSkeletalMesh = ToolKit->Database->ViewportSettings->PreviewMesh ? ToolKit->Database->ViewportSettings->PreviewMesh : DesiredSkeletalMesh;

					if (DesiredSkeletalMesh)
					{
						const int32 LODNum = DesiredSkeletalMesh->GetResourceForRendering()->LODRenderData.Num();
						const int32 LODIndex = FMath::Clamp(ToolKit->Database->ViewportSettings->DrawMeshBonesLOD, 0, LODNum - 1);

						const FSkeletalMeshLODRenderData& LODRenderData = DesiredSkeletalMesh->GetResourceForRendering()->LODRenderData[LODIndex];
						const int32 LODBoneNum = LODRenderData.ActiveBoneIndices.Num();

						BoneIndices.SetNumUninitialized(LODBoneNum);
						for (int32 LODBoneIdx = 0; LODBoneIdx < LODBoneNum; LODBoneIdx++)
						{
							BoneIndices[LODBoneIdx] = ToolKit->Database->GetSkeleton()->GetSkeletonBoneIndexFromMeshBoneIndex(
								DesiredSkeletalMesh, LODRenderData.ActiveBoneIndices[LODBoneIdx]);
						}
					}
				}
				else
				{
					BoneIndices.SetNumUninitialized(BoneNum);
					for (int32 BoneIdx = 0; BoneIdx < BoneNum; BoneIdx++)
					{
						BoneIndices[BoneIdx] = BoneIdx;
					}
				}

				const FLinearColor DefaultBoneColor = UDrawDebugLibrary::GetDefaultBoneColor();

				for (int32 CharacterIdx = 0; CharacterIdx < CharacterNum; CharacterIdx++)
				{
					const FLinearColor RangeIdentifierColor = ToolKit->Database->ViewportSettings->bOverrideRangeIdentifierColor ?
						ToolKit->Database->ViewportSettings->RangeIdentifierColorOverride : ToolKit->Database->Query->QueryEntries[ActiveRanges[CharacterIdx]]->Color;

					// Draw Skeleton

					FLinearColor BoneColor = ToolKit->Database->ViewportSettings->bColorSkeletonBonesUsingRangeIdentifier ? RangeIdentifierColor : DefaultBoneColor;
					BoneColor.A = ToolKit->Database->ViewportSettings->DrawSkeletonOpacity;

					FDrawDebugSkeletonSettings SkeletonSettings;
					SkeletonSettings.bDrawSimpleSkeleton = ToolKit->Database->ViewportSettings->bDrawSimpleSkeleton;
					SkeletonSettings.BoneRadius = ToolKit->Database->ViewportSettings->DrawSkeletonScale;

					UDrawDebugLibrary::DrawDebugSkeletonArrayView(
						FDebugDrawer::MakeDebugDrawer(PDI),
						PoseState.CurrPoseData->RootData.RootLocations[CharacterIdx],
						PoseState.CurrPoseData->RootData.RootRotations[CharacterIdx],
						PoseState.PoseGlobalBoneData->BoneLocations[CharacterIdx],
						PoseState.PoseGlobalBoneData->BoneRotations[CharacterIdx],
						BoneIndices,
						BoneParents,
						BoneColor,
						false,
						SkeletonSettings);

					// Draw Bone Names

					if (ToolKit->Database->ViewportSettings->bDrawBoneNamesAndIndices)
					{
						for (const int32 BoneIdx : BoneIndices)
						{
							const float OffsetBase = ToolKit->Database->ViewportSettings->DrawSkeletonScale;

							FDrawDebugStringSettings Settings;
							Settings.Height = 3.0f;

							FDrawDebugLineStyle LineStyle;
							LineStyle.Color = BoneColor;

							UDrawDebugLibrary::DrawDebugString(
								FDebugDrawer::MakeDebugDrawer(PDI),
								FString::Printf(TEXT("%3i: %s"), BoneIdx, *ToolKit->Database->GetBoneName(BoneIdx).ToString()),
								PoseState.PoseGlobalBoneData->BoneLocations[CharacterIdx][BoneIdx] +
								FVector(OffsetBase + 0.0f, 0.0f, OffsetBase + 3.0f),
								FRotator::ZeroRotator,
								LineStyle,
								false,
								Settings);
						}
					}

					// Draw Velocities

					FLinearColor VelocityBoneColor = BoneColor;
					VelocityBoneColor.A = ToolKit->Database->ViewportSettings->DrawLinearVelocitiesOpacity;

					if (ToolKit->Database->ViewportSettings->bDrawLinearVelocities)
					{
						UDrawDebugLibrary::DrawDebugBoneVelocitiesArrayView(
							FDebugDrawer::MakeDebugDrawer(PDI),
							PoseState.PoseGlobalBoneData->BoneLocations[CharacterIdx],
							PoseState.PoseGlobalBoneData->BoneLinearVelocities[CharacterIdx],
							BoneIndices,
							VelocityBoneColor,
							ToolKit->Database->ViewportSettings->DrawSkeletonScale * 0.5f,
							false,
							ToolKit->Database->ViewportSettings->DrawLinearVelocitiesScale);
					}

					if (ToolKit->Database->ViewportSettings->bDrawAngularVelocities)
					{
						UDrawDebugLibrary::DrawDebugBoneVelocitiesArrayView(
							FDebugDrawer::MakeDebugDrawer(PDI),
							PoseState.PoseGlobalBoneData->BoneLocations[CharacterIdx],
							PoseState.PoseGlobalBoneData->BoneAngularVelocities[CharacterIdx],
							BoneIndices,
							VelocityBoneColor,
							ToolKit->Database->ViewportSettings->DrawSkeletonScale * 0.5f,
							false,
							ToolKit->Database->ViewportSettings->DrawAngularVelocitiesScale);
					}

					// Draw Transforms

					if (ToolKit->Database->ViewportSettings->bDrawBoneTransforms)
					{
						UDrawDebugLibrary::DrawDebugTransformsArrayView(
							FDebugDrawer::MakeDebugDrawer(PDI),
							PoseState.PoseGlobalBoneData->BoneLocations[CharacterIdx],
							PoseState.PoseGlobalBoneData->BoneRotations[CharacterIdx],
							PoseState.PoseGlobalBoneData->BoneScales[CharacterIdx],
							UDrawDebugLibrary::MakeDrawDebugLineStyleFromColor(BoneColor),
							false,
							ToolKit->Database->ViewportSettings->DrawSkeletonScale * 5.0f);
					}
				}
			}

			// Draw Range Identifiers

			if (ToolKit->Database->ViewportSettings->bDrawRangeIdentifier)
			{
				for (int32 CharacterIdx = 0; CharacterIdx < CharacterNum; CharacterIdx++)
				{
					FLinearColor IdentifierColor = ToolKit->Database->ViewportSettings->bOverrideRangeIdentifierColor ?
						ToolKit->Database->ViewportSettings->RangeIdentifierColorOverride : ToolKit->Database->Query->QueryEntries[ActiveRanges[CharacterIdx]]->Color;
					IdentifierColor.A = ToolKit->Database->ViewportSettings->DrawRangeIdentifierOpacity;

					const FVector IdentifierLocation = (FVector)PoseState.CurrPoseData->RootData.RootLocations[CharacterIdx] +
						(FVector)PoseState.CurrPoseData->RootData.RootRotations[CharacterIdx].RotateVector(FVector3f(0.0f, 0.0f, ToolKit->Database->ViewportSettings->DrawRangeIdentifierHeight));

					FDrawDebugLineStyle LineStyle;
					LineStyle.Color = IdentifierColor;
					LineStyle.Thickness = ToolKit->Database->ViewportSettings->DrawRangeIdentifierThickness;

					UDrawDebugLibrary::DrawDebugSimpleSphere(
						FDebugDrawer::MakeDebugDrawer(PDI),
						IdentifierLocation,
						FRotator::ZeroRotator,
						LineStyle,
						true,
						ToolKit->Database->ViewportSettings->DrawRangeIdentifierRadius);
				}
			}

			// Draw Skeleton Roots

			if (ToolKit->Database->ViewportSettings->bDrawRoot && CharacterNum > 0)
			{
				FDrawDebugLineStyle LineStyle;
				LineStyle.Color = FLinearColor::White;
				LineStyle.Color.A = ToolKit->Database->ViewportSettings->DrawRootOpacity;
				LineStyle.Thickness = ToolKit->Database->ViewportSettings->DrawSkeletonScale;

				UDrawDebugLibrary::DrawDebugRotationsQuatArrayView(
					FDebugDrawer::MakeDebugDrawer(PDI),
					PoseState.CurrPoseData->RootData.RootLocations,
					PoseState.CurrPoseData->RootData.RootRotations,
					LineStyle,
					true,
					ToolKit->Database->ViewportSettings->DrawRootScale);
			}

			// Draw Origin

			if (ToolKit->Database->ViewportSettings->bDrawOrigin)
			{
				FDrawDebugLineStyle LineStyle;
				LineStyle.Color = FLinearColor::White;
				LineStyle.Color.A = ToolKit->Database->ViewportSettings->DrawOriginOpacity;
				LineStyle.Thickness = ToolKit->Database->ViewportSettings->DrawOriginLineThickness;

				UDrawDebugLibrary::DrawDebugTransform(
					FDebugDrawer::MakeDebugDrawer(PDI),
					FTransform::Identity,
					LineStyle,
					true,
					ToolKit->Database->ViewportSettings->DrawOriginScale);
			}

			// Draw Trajectory

			if (ToolKit->Database->ViewportSettings->bDrawTrajectories &&
				(!ToolKit->Database->ViewportSettings->bRangesStartAtOrigin || 
					!ToolKit->Database->ViewportSettings->bDisableRootTranslation))
			{
				FVector ForwardVector = FVector::ForwardVector;
				switch (ToolKit->Database->GetSkeleton()->GetPreviewForwardAxis())
				{
				case EAxis::X: ForwardVector = FVector::XAxisVector; break;
				case EAxis::Y: ForwardVector = FVector::YAxisVector; break;
				case EAxis::Z: ForwardVector = FVector::ZAxisVector; break;
				}

				for (int32 CharacterIdx = 0; CharacterIdx < CharacterNum; CharacterIdx++)
				{
					const FVector RootOffset = Database::Private::GetRangeOffset(
						CharacterIdx, CharacterNum,
						ToolKit->Database->ViewportSettings->RangeOffsetSpacing,
						ToolKit->Database->ViewportSettings->bOffsetRanges,
						ToolKit->Database->ViewportSettings->bOffsetRangesInGrid);

					UDrawDebugLibrary::DrawDebugRangeTrajectoryArrayView(
						FDebugDrawer::MakeDebugDrawer(PDI),
						RangeTrajectoryLocations[ActiveRanges[CharacterIdx]],
						RangeTrajectoryRotations[ActiveRanges[CharacterIdx]],
						true,
						ForwardVector,
						ToolKit->Database->ViewportSettings->bDrawTrajectoryOrientations,
						ToolKit->Database->ViewportSettings->bRangesStartAtOrigin,
						RootOffset);
				}
			}

			// Debug Draw

			if (ToolKit->Database->Query->DebugDrawer)
			{
				for (int32 CharacterIdx = 0; CharacterIdx < CharacterNum; CharacterIdx++)
				{
					FLinearColor IdentifierColor = ToolKit->Database->ViewportSettings->bOverrideRangeIdentifierColor ?
						ToolKit->Database->ViewportSettings->RangeIdentifierColorOverride : ToolKit->Database->Query->QueryEntries[ActiveRanges[CharacterIdx]]->Color;
					IdentifierColor.A = ToolKit->Database->ViewportSettings->DrawRangeIdentifierOpacity;

					const FVector RootOffset = Database::Private::GetRangeOffset(
						CharacterIdx, CharacterNum,
						ToolKit->Database->ViewportSettings->RangeOffsetSpacing,
						ToolKit->Database->ViewportSettings->bOffsetRanges,
						ToolKit->Database->ViewportSettings->bOffsetRangesInGrid);

					FTransform RangeStartRootTransform = FTransform::Identity;
					ToolKit->Database->GetRootTransform(TLearningArrayView<1, FTransform>(&RangeStartRootTransform, 1),
						CharacterSequenceIndices[CharacterIdx],
						CharacterRangeStarts[CharacterIdx]);

					const FTransform RangeViewportTransform = !ToolKit->Database->ViewportSettings->bRangesStartAtOrigin ? FTransform::Identity
						: RangeStartRootTransform.Inverse() * FTransform(FQuat::Identity, RootOffset, FVector::OneVector);

					PoseState.PoseIdx = CharacterIdx;

					ToolKit->Database->Query->DebugDrawer->DrawDebug(
						FDebugDrawer::MakeDebugDrawer(PDI),
						FDebugDrawer::MakeDebugDrawer(static_cast<FViewportClient*>(Viewport->GetClient())->GetOrMakeDebugDrawBuffer()),
						PoseState,
						ToolKit->Database,
						UAnimDatabaseFrameRangesLibrary::FrameRangesRangeAtIndex(ToolKit->Database->Query->QueryRanges, ActiveRanges[CharacterIdx]),
						RangeViewportTransform,
						CharacterIdx,
						CharacterSequenceIndices[CharacterIdx],
						CharacterRangeTimes[CharacterIdx],
						CharacterRangeStarts[CharacterIdx],
						CharacterRangeLengths[CharacterIdx],
						IdentifierColor);
				}
			}
		}

		FEdMode::Render(View, Viewport, PDI);
	}
}

#undef LOCTEXT_NAMESPACE
