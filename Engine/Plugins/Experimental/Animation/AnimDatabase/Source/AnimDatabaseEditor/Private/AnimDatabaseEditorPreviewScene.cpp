// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimDatabaseEditorPreviewScene.h"

#include "GameFramework/WorldSettings.h"
#include "GameFramework/Character.h"
#include "Camera/CameraComponent.h"
#include "Components/CapsuleComponent.h"
#include "Components/PoseableMeshComponent.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "EngineUtils.h"

namespace UE::AnimDatabase::Editor
{
	FPreviewScene::FPreviewScene(ConstructionValues CVS) : FAdvancedPreviewScene(CVS)
	{
		// Disable killing actors outside the world
		AWorldSettings* WorldSettings = GetWorld()->GetWorldSettings(true);
		WorldSettings->bEnableWorldBoundsChecks = false;
	}

	FPreviewScene::~FPreviewScene() = default;

	int32 FPreviewScene::GetCharacterNum() const
	{
		return CharacterActors.Num();
	}

	int32 FPreviewScene::AddCharacter()
	{
		// Spawn a Character

		FActorSpawnParameters Params;
		Params.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AdjustIfPossibleButAlwaysSpawn;

		ACharacter* CharacterActor = GetWorld()->SpawnActor<ACharacter>(ACharacter::StaticClass(), FTransform(), Params);
		check(CharacterActor != nullptr);

		// If Character has a Camera Component attached they destroy it

		if (UCameraComponent* CameraComp = CharacterActor->FindComponentByClass<UCameraComponent>())
		{
			CameraComp->DestroyComponent();
		}

		// Find the Capsule Component and disable the visibility

		UCapsuleComponent* CapsuleComponent = Cast<UCapsuleComponent>(CharacterActor->FindComponentByClass<UCapsuleComponent>());
		check(CapsuleComponent);
		CapsuleComponent->SetVisibility(false, true);

		// Attach a new Poseable Mesh Component

		FTransform RelativeMeshTransform;
		RelativeMeshTransform.SetTranslation(FVector3d(0.0f, 0.0f, -CapsuleComponent->GetScaledCapsuleHalfHeight()));

		UPoseableMeshComponent* PoseableMeshComponent = Cast<UPoseableMeshComponent>(CharacterActor->AddComponentByClass(UPoseableMeshComponent::StaticClass(), false, RelativeMeshTransform, false));
		check(PoseableMeshComponent);

		UDebugSkelMeshComponent* DebugSkelMeshComponent = Cast<UDebugSkelMeshComponent>(CharacterActor->AddComponentByClass(UDebugSkelMeshComponent::StaticClass(), false, RelativeMeshTransform, false));
		check(DebugSkelMeshComponent);

		// Add all the components to the lists

		CharacterActors.Add(CharacterActor);
		CapsuleComponents.Add(CapsuleComponent);
		PoseableMeshComponents.Add(PoseableMeshComponent);

		return CharacterActors.Num() - 1;
	}

	int32 FPreviewScene::RemoveCharacter()
	{
		if (!CharacterActors.IsEmpty())
		{
			GetWorld()->RemoveActor(CharacterActors.Last(), false);

			CharacterActors.Pop();
			CapsuleComponents.Pop();
			PoseableMeshComponents.Pop();

			return CharacterActors.Num() - 1;
		}
		else
		{
			return 0;
		}
	}

	void FPreviewScene::SetCharacterVisibility(const int32 CharacterIdx, bool bVisibility)
	{
		PoseableMeshComponents[CharacterIdx]->SetVisibility(bVisibility, true);
	}

	bool FPreviewScene::GetCharacterVisibility(const int32 CharacterIdx) const
	{
		return PoseableMeshComponents[CharacterIdx]->IsVisible();
	}

	ACharacter* FPreviewScene::GetCharacterActor(const int32 CharacterIdx) const
	{
		return CharacterActors[CharacterIdx];
	}

	UPoseableMeshComponent* FPreviewScene::GetPoseableMeshComponent(const int32 CharacterIdx) const
	{
		return PoseableMeshComponents[CharacterIdx];
	}

	void FPreviewScene::Tick(float InDeltaTime)
	{
		FAdvancedPreviewScene::Tick(InDeltaTime);

		GetWorld()->Tick(LEVELTICK_All, InDeltaTime);
	}

}
