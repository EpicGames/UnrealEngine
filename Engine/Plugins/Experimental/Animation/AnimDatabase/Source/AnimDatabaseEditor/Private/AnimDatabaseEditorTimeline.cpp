// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimDatabaseEditorTimeline.h"

#include "Animation/AnimSequence.h"

namespace UE::AnimDatabase::Editor
{

	FFrameRate FTimelineModel::GetTimelineFrameRate() const { return FrameRate; }
	FFrameTime FTimelineModel::GetTimelineFrameTime() const { return FrameTime; }
	
	TRange<FFrameNumber> FTimelineModel::GetTimelineViewRange() const
	{
		return ViewRange;
	}

	FAnimatedRange FTimelineModel::GetTimelineViewRangeTimes() const
	{
		return FAnimatedRange(
			ViewRange.GetLowerBoundValue().Value,
			ViewRange.GetUpperBoundValue().Value);
	}

	TRange<FFrameNumber> FTimelineModel::GetTimelineWorkingRange() const
	{
		return WorkingRange;
	}

	FAnimatedRange FTimelineModel::GetTimelineWorkingRangeTimes() const
	{
		return FAnimatedRange(
			WorkingRange.GetLowerBoundValue().Value,
			WorkingRange.GetUpperBoundValue().Value);
	}

	TRange<FFrameNumber> FTimelineModel::GetTimelinePlaybackRangeFrames() const
	{
		return PlaybackRange;
	}

	float FTimelineModel::GetTimelinePlaybackRate() const
	{
		return bUseCustomPlaybackRate ? CustomPlaybackRate : PlaybackRate;
	}

	float FTimelineModel::GetTimelineCustomPlaybackRate() const
	{
		return CustomPlaybackRate;
	}

	bool FTimelineModel::GetTimelineUseCustomPlaybackRate() const
	{
		return bUseCustomPlaybackRate;
	}
	
	bool FTimelineModel::GetTimelineSnapOnFrames() const
	{
		return bSnapOnFrames;
	}

	void FTimelineModel::SetTimelineFrameRate(const FFrameRate InFrameRate)
	{
		FrameRate = InFrameRate;
	}

	void FTimelineModel::SetTimelineFrameTime(const FFrameTime InFrameTime)
	{
		// Clamp the frame time to within the view range
		FrameTime = FFrameTime::FromDecimal(FMath::Clamp(
			InFrameTime.AsDecimal(), 
			(double)ViewRange.GetLowerBoundValue().Value,
			(double)ViewRange.GetUpperBoundValue().Value));

		// If snapping, snap the frametime to the nearest frame
		if (bSnapOnFrames) { FrameTime = FFrameTime::FromDecimal((double)FMath::RoundToInt(FrameTime.AsDecimal())); }
	}

	void FTimelineModel::SetTimelineViewRangeTimes(const FAnimatedRange InViewRange)
	{
		// Clamp the view range to within the working range
		const int32 ViewLower = (int32)FMath::RoundToInt(InViewRange.GetLowerBoundValue());
		const int32 ViewUpper = (int32)FMath::RoundToInt(InViewRange.GetUpperBoundValue());
		ViewRange.SetLowerBoundValue(FMath::Clamp(ViewLower, WorkingRange.GetLowerBoundValue().Value, WorkingRange.GetUpperBoundValue().Value));
		ViewRange.SetUpperBoundValue(FMath::Clamp(ViewUpper, WorkingRange.GetLowerBoundValue().Value, WorkingRange.GetUpperBoundValue().Value));
	}

	void FTimelineModel::SetTimelineWorkingRangeTimes(const FAnimatedRange InWorkingRange)
	{
		// Update the working range to the nearest frames
		WorkingRange.SetLowerBoundValue((int32)FMath::RoundToInt(InWorkingRange.GetLowerBoundValue()));
		WorkingRange.SetUpperBoundValue((int32)FMath::RoundToInt(InWorkingRange.GetUpperBoundValue()));

		// Clamp the view range to within the working range
		const int32 ViewLower = ViewRange.GetLowerBoundValue().Value;
		const int32 ViewUpper = ViewRange.GetUpperBoundValue().Value;
		ViewRange.SetLowerBoundValue(FMath::Clamp(ViewLower, WorkingRange.GetLowerBoundValue().Value, WorkingRange.GetUpperBoundValue().Value));
		ViewRange.SetUpperBoundValue(FMath::Clamp(ViewUpper, WorkingRange.GetLowerBoundValue().Value, WorkingRange.GetUpperBoundValue().Value));
	}

	void FTimelineModel::SetTimelinePlaybackRate(const float InPlaybackRate)
	{
		PlaybackRate = InPlaybackRate;
	}

	void FTimelineModel::SetTimelineCustomPlaybackRate(const float InCustomPlaybackRate)
	{
		CustomPlaybackRate = InCustomPlaybackRate;
	}

	void FTimelineModel::SetTimelineViewRange(const TRange<FFrameNumber> InViewRange)
	{
		ViewRange = InViewRange;
	}

	void FTimelineModel::SetTimelineWorkingRange(const TRange<FFrameNumber> InWorkingRange)
	{
		// Update the working range
		WorkingRange = InWorkingRange;

		// Clamp the view range to within the working range
		const int32 ViewLower = ViewRange.GetLowerBoundValue().Value;
		const int32 ViewUpper = ViewRange.GetUpperBoundValue().Value;
		ViewRange.SetLowerBoundValue(FMath::Clamp(ViewLower, WorkingRange.GetLowerBoundValue().Value, WorkingRange.GetUpperBoundValue().Value));
		ViewRange.SetUpperBoundValue(FMath::Clamp(ViewUpper, WorkingRange.GetLowerBoundValue().Value, WorkingRange.GetUpperBoundValue().Value));
	}

	void FTimelineModel::SetTimelinePlaybackRange(const TRange<FFrameNumber> InPlaybackRange)
	{
		PlaybackRange = InPlaybackRange;
	}

	void FTimelineModel::SetTransportPlaybackMode(const EPlaybackMode::Type InPlaybackMode)
	{
		PlaybackMode = InPlaybackMode;
	}

	void FTimelineModel::SetTimelineSnapOnFrames(const bool bInSnapOnFrames)
	{
		bSnapOnFrames = bInSnapOnFrames;
	}

	void FTimelineModel::SetLooping(const bool bInLooping)
	{
		bLooping = bInLooping;
	}

	void FTimelineModel::SetTimelineUseCustomPlaybackRate(const bool bInUseCustomPlaybackRate)
	{
		bUseCustomPlaybackRate = bInUseCustomPlaybackRate;
	}

	void FTimelineModel::OnTransportStepForward()
	{
		// Round to the nearest frame, step forward, and clamp to within the view range
		const int32 FrameNumber = FMath::Clamp(
			FMath::RoundToInt(FrameTime.AsDecimal()) + 1, 
			ViewRange.GetLowerBoundValue().Value, 
			ViewRange.GetUpperBoundValue().Value);

		FrameTime = FFrameTime::FromDecimal(FrameNumber);
	}

	void FTimelineModel::OnTransportStepBackward()
	{
		// Round to the nearest frame, step backward, and clamp to within the view range
		const int32 FrameNumber = FMath::Clamp(
			FMath::RoundToInt(FrameTime.AsDecimal()) - 1,
			ViewRange.GetLowerBoundValue().Value, 
			ViewRange.GetUpperBoundValue().Value);

		FrameTime = FFrameTime::FromDecimal(FrameNumber);
	}

	void FTimelineModel::OnTransportToStartFrame()
	{
		FrameTime = FFrameTime::FromDecimal(PlaybackRange.GetLowerBoundValue().Value);
	}

	void FTimelineModel::OnTransportToEndFrame()
	{
		FrameTime = FFrameTime::FromDecimal(PlaybackRange.GetUpperBoundValue().Value);
	}
	
	void FTimelineModel::OnTransportPlayPressed()
	{
		// Toggle Playback

		if (PlaybackMode == EPlaybackMode::PlayingForward)
		{
			PlaybackMode = EPlaybackMode::Stopped;
		}
		else
		{
			PlaybackMode = EPlaybackMode::PlayingForward;
		}
	}

	EPlaybackMode::Type FTimelineModel::GetTransportPlaybackMode() const { return PlaybackMode; }

	bool FTimelineModel::GetLooping() const { return bLooping; }

	void FTimelineModel::UpdateTimelineFrameTime(const float DeltaTime)
	{
		// Move forward FrameTime by DeltaTime multiplied by the PlaybackRate

		if (PlaybackMode == EPlaybackMode::PlayingForward)
		{
			if (bLooping)
			{
				FrameTime = FFrameTime::FromDecimal(FMath::Wrap(
					FrameTime.AsDecimal() + DeltaTime * GetTimelinePlaybackRate(),
					(double)ViewRange.GetLowerBoundValue().Value,
					(double)ViewRange.GetUpperBoundValue().Value));
			}
			else
			{
				FrameTime = FFrameTime::FromDecimal(FMath::Clamp(
					FrameTime.AsDecimal() + DeltaTime * GetTimelinePlaybackRate(),
					(double)ViewRange.GetLowerBoundValue().Value,
					(double)ViewRange.GetUpperBoundValue().Value));
			}
		}
	}

	void FTimelineTracksModel::ResetTracks()
	{
		RootTracks.Reset();
	}

	void FTimelineTracksModel::AddTrack(
		const UAnimSequence* Sequence,
		const FLinearColor& RangeColor,
		const TArrayView<const FName> FrameNames,
		const TArrayView<const FName> FrameRangeNames,
		const TArrayView<const FName> FrameAttributeNames,
		const TArrayView<const FAnimDatabaseFrames> Frames,
		const TArrayView<const FAnimDatabaseFrameRanges> FrameRanges,
		const TArrayView<const FAnimDatabaseFrameAttribute> FrameAttributes,
		const int32 ViewOffset)
	{
		check(FrameNames.Num() == Frames.Num());
		check(FrameRangeNames.Num() == FrameRanges.Num());
		check(FrameAttributeNames.Num() == FrameAttributes.Num());

		const int32 FrameNum = FrameNames.Num();
		const int32 FrameRangesNum = FrameRangeNames.Num();
		const int32 FrameAttributesNum = FrameAttributeNames.Num();

		if (FrameNum == 0 && FrameRangesNum == 0 && FrameAttributesNum == 0) { return; }

		TSharedRef<FTimelineTrack> RangeTrack = RootTracks.Add_GetRef(MakeShared<FTimelineTrack>());
		RangeTrack->Color = RangeColor;
		RangeTrack->SequenceName = Sequence->GetName();
		RangeTrack->SequencePath = Sequence;
		RangeTrack->bIsExpanded = true;
		RangeTrack->bIsRange = true;

		for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
		{
			check(Frames[FrameIdx].IsValid());

			TSharedRef<FTimelineTrack> FramesSubTrack = RangeTrack->Children.Add_GetRef(MakeShared<FTimelineTrack>());
			FramesSubTrack->DisplayName = FText::FromName(FrameNames[FrameIdx]);
			FramesSubTrack->Color = RangeColor;
			FramesSubTrack->FramesObject = Frames[FrameIdx];
			FramesSubTrack->ViewOffset = ViewOffset;
		}

		for (int32 FrameIdx = 0; FrameIdx < FrameRangesNum; FrameIdx++)
		{
			check(FrameRanges[FrameIdx].IsValid());

			TSharedRef<FTimelineTrack> FramesSubTrack = RangeTrack->Children.Add_GetRef(MakeShared<FTimelineTrack>());
			FramesSubTrack->DisplayName = FText::FromName(FrameRangeNames[FrameIdx]);
			FramesSubTrack->Color = RangeColor;
			FramesSubTrack->FrameRangesObject = FrameRanges[FrameIdx];
			FramesSubTrack->ViewOffset = ViewOffset;
		}

		for (int32 FrameIdx = 0; FrameIdx < FrameAttributesNum; FrameIdx++)
		{
			check(FrameAttributes[FrameIdx].IsValid());

			TSharedRef<FTimelineTrack> FramesSubTrack = RangeTrack->Children.Add_GetRef(MakeShared<FTimelineTrack>());
			FramesSubTrack->DisplayName = FText::FromName(FrameAttributeNames[FrameIdx]);
			FramesSubTrack->Color = RangeColor;
			FramesSubTrack->FrameAttributeObject = FrameAttributes[FrameIdx];
			FramesSubTrack->ViewOffset = ViewOffset;
		}
	}

	TArray<TSharedRef<FTimelineTrack>>& FTimelineTracksModel::GetRootTracks()
	{
		return RootTracks;
	}
}
