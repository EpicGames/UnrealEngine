// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimDatabaseEditorToolkit.h"

#include "AnimDatabaseEditorMode.h"
#include "AnimDatabaseEditorPreviewScene.h"
#include "AnimDatabaseEditorTimeline.h"
#include "AnimDatabaseEditorViewportClient.h"
#include "SAnimDatabaseEditorTimeline.h"
#include "SEditorViewport.h"
#include "SCommonEditorViewportToolbarBase.h"

#include "AnimDatabase.h"
#include "AnimDatabaseFrameRanges.h"

#include "AdvancedPreviewSceneModule.h"
#include "GameFramework/WorldSettings.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Docking/SDockTab.h"
#include "PropertyEditorModule.h"
#include "IContentBrowserSingleton.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "ContentBrowserDataSource.h"
#include "ContentBrowserModule.h"
#include "PreviewProfileController.h"
#include "Viewports.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailGroup.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Layout/SScrollBox.h"

#include "Animation/Skeleton.h"
#include "Animation/MirrorDataTable.h"
#include "Animation/AnimSequence.h"

#define LOCTEXT_NAMESPACE "AnimDatabaseEditorToolkit"

FText UAnimDatabaseEditorAssetDefinition::GetAssetDisplayName() const
{
	return LOCTEXT("AnimDatabaseEditorAssetDefinitionName", "Animation Database");
}

FLinearColor UAnimDatabaseEditorAssetDefinition::GetAssetColor() const
{
	return FColor(220, 149, 66);
}

TSoftClassPtr<UObject> UAnimDatabaseEditorAssetDefinition::GetAssetClass() const
{
	return UAnimDatabase::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAnimDatabaseEditorAssetDefinition::GetAssetCategories() const
{
	static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Animation) };
	return Categories;
}

EAssetCommandResult UAnimDatabaseEditorAssetDefinition::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	for (UAnimDatabase* Asset : OpenArgs.LoadObjects<UAnimDatabase>())
	{
		TSharedRef<UE::AnimDatabase::Editor::FDatabaseToolkit> NewEditor(new UE::AnimDatabase::Editor::FDatabaseToolkit());
		NewEditor->InitAssetEditor(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, Asset);
	}

	return EAssetCommandResult::Handled;
}

UAnimDatabaseEditorFactory::UAnimDatabaseEditorFactory()
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UAnimDatabase::StaticClass();
}

UObject* UAnimDatabaseEditorFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UAnimDatabase>(InParent, Name, Flags | RF_Transactional);
}

bool UAnimDatabaseEditorFactory::ShouldShowInNewMenu() const
{
	return true;
}

bool UAnimDatabaseEditorFactory::ConfigureProperties()
{
	return true;
}

FText UAnimDatabaseEditorFactory::GetDisplayName() const
{
	return LOCTEXT("AnimDatabaseEditorAsset_DisplayName", "Animation Database");
}

uint32 UAnimDatabaseEditorFactory::GetMenuCategories() const
{
	return EAssetTypeCategories::Animation;
}

FText UAnimDatabaseEditorFactory::GetToolTip() const
{
	return LOCTEXT("AnimDatabaseEditorAsset_Tooltip", "Asset for organization and processing large databases of animation.");
}

FString UAnimDatabaseEditorFactory::GetDefaultNewAssetName() const
{
	return FString(TEXT("ADB_NewAnimDatabase"));
}

namespace UE::AnimDatabase::Editor
{
	TSharedRef<IDetailCustomization> FQueryDetails::MakeInstance()
	{
		return MakeShared<FQueryDetails>();
	}

	void FQueryDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
	{
		// Get the UAnimDatabaseQuery object we are associated with

		TArray<TWeakObjectPtr<UObject>> Objects;
		DetailBuilder.GetObjectsBeingCustomized(Objects);
		if (Objects.Num() != 1) { return; }

		UAnimDatabaseQuery* Query = StaticCast<UAnimDatabaseQuery*>(Objects[0].Pin().Get());
		if (!Query) { return; }

		// Put the Query category first

		IDetailCategoryBuilder& QueryCategory = DetailBuilder.EditCategory("Query", LOCTEXT("QueryCategory", "Query"), ECategoryPriority::Important);
		QueryCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UAnimDatabaseQuery, FrameRanges), UAnimDatabaseQuery::StaticClass());

		// Put the frame ranges returned by the query next

		IDetailCategoryBuilder& RangesCategory = DetailBuilder.EditCategory("Ranges", LOCTEXT("RangesCategory", "Ranges"), ECategoryPriority::Important);

		// Add the ListView 

		if (Query->QueryEntries.Num() > 0)
		{
			// Make the ListView used to render the FQueryEntry array

			SAssignNew(ListView, SListView<TSharedPtr<FQueryEntry>>)
				.ListItemsSource(&Query->QueryEntries)
				.OnGenerateRow(this, &FQueryDetails::OnGenerateRow)
				.OnSelectionChanged(this, &FQueryDetails::OnSelectionChanged)
				.OnKeyDownHandler_Lambda([this, Query](const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) { return OnKeyDown(Query, InKeyEvent); })
				.HeaderRow(
					SNew(SHeaderRow)
					+ SHeaderRow::Column("Icon")
					.DefaultLabel(FText::GetEmpty())
					.FillWidth(0.05f)
					+ SHeaderRow::Column("Sequence")
					.DefaultLabel(LOCTEXT("ColumnLabelAnimSequence", "Anim Sequence"))
					.FillWidth(0.7f)
					+ SHeaderRow::Column("Start")
					.DefaultLabel(LOCTEXT("ColumnLabelStart", "Start"))
					.FillWidth(0.1f)
					+ SHeaderRow::Column("Stop")
					.DefaultLabel(LOCTEXT("ColumnLabelStop", "Stop"))
					.FillWidth(0.1f)
					+ SHeaderRow::Column("Mirrored")
					.DefaultLabel(LOCTEXT("ColumnLabelMirrored", "Mirrored"))
					.FillWidth(0.05f)
				);

			// Add to UI

			RangesCategory.AddCustomRow(LOCTEXT("RangesProperty", "Ranges"))
				.WholeRowContent()
				[
					SNew(SBox)
						// We need to set the list view to a fixed maximum height otherwise it will fill the whole details panel. Unfortunately the 
						// scroll bar of the details panel will get messed up if we make this too large and this is a wont-fix right now (UE-40387)
						// we therefore try to limit the size a bit even if this is annoying for users.
						.MaxDesiredHeight(500.0f)
						[
							ListView.ToSharedRef()
						]
				];

			// Re-set selection

			TArray<TSharedPtr<FQueryEntry>, TInlineAllocator<16>> ResetSelectedItems;
			for (const TSharedPtr<FQueryEntry>& Item : ListView->GetItems())
			{
				if (Item->bIsSelected)
				{
					ResetSelectedItems.Add(Item);
				}
			}
			ListView->SetItemSelection(ResetSelectedItems, true);
		}
		else
		{
			// Write out message instead of query returned no ranges

			RangesCategory.AddCustomRow(LOCTEXT("RangesProperty", "Ranges"))
				.WholeRowContent()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock).Text(LOCTEXT("EmptyRanges", "No ranges returned by Query"))
				];
		}

		IDetailCategoryBuilder& TimelineCategory = DetailBuilder.EditCategory("Timeline", LOCTEXT("TimelineCategory", "Timeline"), ECategoryPriority::Important);
		IDetailCategoryBuilder& DebugDrawCategory = DetailBuilder.EditCategory("Debug Draw", LOCTEXT("DebugDrawCategory", "Debug Draw"), ECategoryPriority::Important);
		IDetailCategoryBuilder& FunctionCategory = DetailBuilder.EditCategory("Function", LOCTEXT("FunctionCategory", "Function"), ECategoryPriority::Important);
	}

	FReply FQueryDetails::OnKeyDown(UAnimDatabaseQuery* Query, const FKeyEvent& InKeyEvent)
	{
		if (InKeyEvent.GetKey() == EKeys::Delete)
		{
			Query->DeleteSelectedFromDatabase();
			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

	void FQueryDetails::OnSelectionChanged(TSharedPtr<FQueryEntry> SelectedItem, ESelectInfo::Type SelectInfo)
	{
		ListView->GetSelectedItems(SelectedItems);
		for (const TSharedPtr<FQueryEntry>& Item : ListView->GetItems())
		{
			Item->bIsSelected = SelectedItems.Contains(Item);
		}
	}

	TSharedRef<ITableRow> FQueryDetails::OnGenerateRow(TSharedPtr<FQueryEntry> Item, const TSharedRef<STableViewBase>& OwnerTable)
	{
		class SFilterMultiColumnRow : public SMultiColumnTableRow<TSharedPtr<FQueryEntry>>
		{
		public:
			SLATE_BEGIN_ARGS(SFilterMultiColumnRow) {}
				SLATE_ARGUMENT(TSharedPtr<FQueryEntry>, Item)
			SLATE_END_ARGS()

			void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
			{
				Item = InArgs._Item;
				SMultiColumnTableRow<TSharedPtr<FQueryEntry>>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
			}

			virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
			{
				if (ColumnName == "Icon")
				{
					// Draw colored circle icon a.k.a "Range Identifier"
					return SNew(SBox)
						.HAlign(EHorizontalAlignment::HAlign_Center)
						.VAlign(EVerticalAlignment::VAlign_Center)
						.MaxDesiredWidth(10.0f)
						.MaxDesiredHeight(10.0f)
						.MaxAspectRatio(1.0f)
						.MinAspectRatio(1.0f)
						[
							SNew(SImage)
								.ColorAndOpacity(Item->Color)
								.Image(FCoreStyle::Get().GetBrush("GraphEditor.PinIcon"))
						];
				}
				else if (ColumnName == "Sequence")
				{
					return SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(EHorizontalAlignment::HAlign_Left)
						.VAlign(EVerticalAlignment::VAlign_Center)
						.Padding(0.0f, 1.0f, 0.0f, 1.0f)
						[
							Item->Sequence ?
							SNew(SHyperlink).Text(FText::FromString(Item->Sequence->GetName())).OnNavigate_Lambda([this]()
								{
									GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Item->Sequence);
								})
								:
							SNew(SHyperlink).Text(LOCTEXT("NullSequenceName", "null"))
						]
						+SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(EHorizontalAlignment::HAlign_Left)
						.VAlign(EVerticalAlignment::VAlign_Center)
						.Padding(0.0f, 1.0f, 0.0f, 1.0f)
						[
							SNew(STextBlock).Text(Item->bIsMirrored ? LOCTEXT("MirroredAddition", " (mirrored)") : FText::GetEmpty())
						];
				}
				else if (ColumnName == "Start")
				{
					return SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						//.AutoWidth()
						.HAlign(EHorizontalAlignment::HAlign_Right)
						.VAlign(EVerticalAlignment::VAlign_Center)
						.Padding(0.0f, 1.0f, 0.0f, 1.0f)
						[
							SNew(STextBlock).Text(FText::AsNumber(Item->StartFrame))
						];
				}
				else if (ColumnName == "Stop")
				{
					return SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						//.AutoWidth()
						.HAlign(EHorizontalAlignment::HAlign_Right)
						.VAlign(EVerticalAlignment::VAlign_Center)
						.Padding(0.0f, 1.0f, 0.0f, 1.0f)
						[
							SNew(STextBlock).Text(FText::AsNumber(Item->StopFrame))
						];
				}
				else if (ColumnName == "Mirrored")
				{
					return SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						//.AutoWidth()
						.HAlign(EHorizontalAlignment::HAlign_Center)
						.VAlign(EVerticalAlignment::VAlign_Center)
						.Padding(0.0f, 1.0f, 0.0f, 1.0f)
						[
							SNew(SImage).Image(FCoreStyle::Get().GetBrush("GraphEditor.AlignNodesCenter")).RenderOpacity(Item->bIsMirrored ? 1.0f : 0.0f)
						];
				}

				return SNullWidget::NullWidget;
			}

		private:
			TSharedPtr<FQueryEntry> Item;
		};

		return SNew(SFilterMultiColumnRow, OwnerTable).Item(Item);
	}

	class SDatabaseViewportToolBar;

	/** Custom Database Viewport class. Contains a custom show menu at the top which contains some toggles for the ViewportSettings */
	class SDatabaseViewport : public SEditorViewport, public ICommonEditorViewportToolbarInfoProvider
	{
	public:

		SLATE_BEGIN_ARGS(SDatabaseViewport) {}
		SLATE_END_ARGS();

		void Construct(
			const FArguments& InArgs,
			const TSharedRef<FDatabaseToolkit>& InAssetEditorToolkit,
			const TSharedRef<FPreviewScene>& InPreviewScene,
			const FEditorModeID InModeID);

		// ~ICommonEditorViewportToolbarInfoProvider interface
		virtual TSharedRef<SEditorViewport> GetViewportWidget() override;
		virtual TSharedPtr<FExtender> GetExtenders() const override;
		virtual void OnFloatingButtonClicked() override {}
		// ~End of ICommonEditorViewportToolbarInfoProvider interface

	protected:

		// ~SEditorViewport interface
		virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;
		virtual TSharedPtr<SWidget> BuildViewportToolbar() override;
		// ~End of SEditorViewport interface

		/** Viewport client */
		TSharedPtr<FViewportClient> ViewportClient;

		/** The preview scene that we are viewing */
		TWeakPtr<FPreviewScene> PreviewScenePtr;

		/** Database Toolkit */
		TWeakPtr<FDatabaseToolkit> DatabaseToolkit;

		/** Editor Mode */
		FEditorModeID ModeID;
	};

	/** Custom viewport toolbar which refers to some settings in database ViewportSettings */
	class SDatabaseViewportToolBar : public SCommonEditorViewportToolbarBase
	{
	public:
		SLATE_BEGIN_ARGS(SDatabaseViewportToolBar) {}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TSharedPtr<SDatabaseViewport> InViewport, TWeakPtr<FDatabaseToolkit> InDatabaseToolkit);
		virtual TSharedRef<SWidget> GenerateShowMenu() const override;

		TWeakPtr<FDatabaseToolkit> DatabaseToolkit;
	};

	void SDatabaseViewport::Construct(
		const FArguments& InArgs,
		const TSharedRef<FDatabaseToolkit>& InAssetEditorToolkit,
		const TSharedRef<FPreviewScene>& InPreviewScene,
		const FEditorModeID InModeID)
	{
		PreviewScenePtr = InPreviewScene;
		DatabaseToolkit = InAssetEditorToolkit.ToWeakPtr();
		ModeID = InModeID;

		SEditorViewport::Construct(
			SEditorViewport::FArguments()
			.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
			.AddMetaData<FTagMetaData>(TEXT("AnimationTools.Viewport"))
		);
	}

	TSharedRef<SEditorViewport> SDatabaseViewport::GetViewportWidget()
	{
		return SharedThis(this);
	}

	TSharedPtr<FExtender> SDatabaseViewport::GetExtenders() const
	{
		TSharedPtr<FExtender> Result(MakeShareable(new FExtender));
		return Result;
	}

	TSharedRef<FEditorViewportClient> SDatabaseViewport::MakeEditorViewportClient()
	{
		ViewportClient = MakeShared<FViewportClient>(
			PreviewScenePtr, 
			SharedThis(this), 
			StaticCastWeakPtr<FAssetEditorToolkit>(DatabaseToolkit),
			ModeID);

		ViewportClient->ViewportType = LVT_Perspective;
		ViewportClient->bSetListenerPosition = false;
		ViewportClient->SetViewLocation(EditorViewportDefs::DefaultPerspectiveViewLocation);
		ViewportClient->SetViewRotation(EditorViewportDefs::DefaultPerspectiveViewRotation);

		return ViewportClient.ToSharedRef();
	}

	TSharedPtr<SWidget> SDatabaseViewport::BuildViewportToolbar()
	{
		return SNew(SDatabaseViewportToolBar, SharedThis(this), DatabaseToolkit);
	}

	void SDatabaseViewportToolBar::Construct(const FArguments& InArgs, TSharedPtr<SDatabaseViewport> InViewport, TWeakPtr<FDatabaseToolkit> InDatabaseToolkit)
	{
		SCommonEditorViewportToolbarBase::Construct(
			SCommonEditorViewportToolbarBase::FArguments()
			.AddRealtimeButton(false)
			.PreviewProfileController(MakeShared<FPreviewProfileController>()),
			InViewport);

		DatabaseToolkit = InDatabaseToolkit;
	}

	TSharedRef<SWidget> SDatabaseViewportToolBar::GenerateShowMenu() const
	{
		GetInfoProvider().OnFloatingButtonClicked();

		TSharedRef<SEditorViewport> ViewportRef = GetInfoProvider().GetViewportWidget();

		const bool bInShouldCloseWindowAfterMenuSelection = true;
		FMenuBuilder ShowMenuBuilder(bInShouldCloseWindowAfterMenuSelection, ViewportRef->GetCommandList());
		{
			//-------------------------------

			ShowMenuBuilder.BeginSection("Skeleton", LOCTEXT("ShowMenu_SkeletonLabel", "Skeleton"));

			ShowMenuBuilder.AddMenuEntry(
				LOCTEXT("ShowMenu_SkeletonLabelSkeleton", "Skeleton"),
				LOCTEXT("ShowMenu_SkeletonLabelSkeletonToolTip", "Draw the character skeleton"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([this]() { if (DatabaseToolkit.IsValid()) {
				DatabaseToolkit.Pin()->Database->ViewportSettings->bDrawSkeleton = !DatabaseToolkit.Pin()->Database->ViewportSettings->bDrawSkeleton;
			} }),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([this]() { return DatabaseToolkit.IsValid() ? DatabaseToolkit.Pin()->Database->ViewportSettings->bDrawSkeleton : false; })),
				NAME_None,
				EUserInterfaceActionType::ToggleButton);

			ShowMenuBuilder.AddMenuEntry(
				LOCTEXT("ShowMenu_SkeletonLabelLinearVelocities", "Linear Velocities"),
				LOCTEXT("ShowMenu_SkeletonLabelLinearVelocitiesToolTip", "Draw linear velocities"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([this]() { if (DatabaseToolkit.IsValid()) {
				DatabaseToolkit.Pin()->Database->ViewportSettings->bDrawLinearVelocities = !DatabaseToolkit.Pin()->Database->ViewportSettings->bDrawLinearVelocities;
			} }),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([this]() { return DatabaseToolkit.IsValid() ? DatabaseToolkit.Pin()->Database->ViewportSettings->bDrawLinearVelocities : false; })),
				NAME_None,
				EUserInterfaceActionType::ToggleButton);

			ShowMenuBuilder.AddMenuEntry(
				LOCTEXT("ShowMenu_SkeletonLabelAngularVelocities", "Angular Velocities"),
				LOCTEXT("ShowMenu_SkeletonLabelAngularVelocitiesToolTip", "Draw Angular velocities"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([this]() { if (DatabaseToolkit.IsValid()) {
				DatabaseToolkit.Pin()->Database->ViewportSettings->bDrawAngularVelocities = !DatabaseToolkit.Pin()->Database->ViewportSettings->bDrawAngularVelocities;
			} }),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([this]() { return DatabaseToolkit.IsValid() ? DatabaseToolkit.Pin()->Database->ViewportSettings->bDrawAngularVelocities : false; })),
				NAME_None,
				EUserInterfaceActionType::ToggleButton);

			ShowMenuBuilder.EndSection();

			//-------------------------------

			ShowMenuBuilder.BeginSection("Root", LOCTEXT("ShowMenu_RootLabel", "Root"));

			ShowMenuBuilder.AddMenuEntry(
				LOCTEXT("ShowMenu_RootLabelRoot", "Root"),
				LOCTEXT("ShowMenu_RootLabelRootToolTip", "Draw Root"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([this]() { if (DatabaseToolkit.IsValid()) {
				DatabaseToolkit.Pin()->Database->ViewportSettings->bDrawRoot = !DatabaseToolkit.Pin()->Database->ViewportSettings->bDrawRoot;
			} }),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([this]() { return DatabaseToolkit.IsValid() ? DatabaseToolkit.Pin()->Database->ViewportSettings->bDrawRoot : false; })),
				NAME_None,
				EUserInterfaceActionType::ToggleButton);

			ShowMenuBuilder.EndSection();

			//-------------------------------

			ShowMenuBuilder.BeginSection("Trajectories", LOCTEXT("ShowMenu_TrajectoriesLabel", "Trajectories"));

			ShowMenuBuilder.AddMenuEntry(
				LOCTEXT("ShowMenu_TrajectoriesLabelTrajectories", "Trajectories"),
				LOCTEXT("ShowMenu_TrajectoriesLabelTrajectoriesToolTip", "Draw Trajectories"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([this]() { if (DatabaseToolkit.IsValid()) {
				DatabaseToolkit.Pin()->Database->ViewportSettings->bDrawTrajectories = !DatabaseToolkit.Pin()->Database->ViewportSettings->bDrawTrajectories;
			} }),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([this]() { return DatabaseToolkit.IsValid() ? DatabaseToolkit.Pin()->Database->ViewportSettings->bDrawTrajectories : false; })),
				NAME_None,
				EUserInterfaceActionType::ToggleButton);

			ShowMenuBuilder.AddMenuEntry(
				LOCTEXT("ShowMenu_TrajectoriesLabelTrajectoriesOrientations", "Trajectories Orientations"),
				LOCTEXT("ShowMenu_TrajectoriesLabelTrajectoriesOrientationsToolTip", "Draw Trajectories Orientations"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([this]() { if (DatabaseToolkit.IsValid()) {
				DatabaseToolkit.Pin()->Database->ViewportSettings->bDrawTrajectoryOrientations = !DatabaseToolkit.Pin()->Database->ViewportSettings->bDrawTrajectoryOrientations;
			} }),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([this]() { return DatabaseToolkit.IsValid() ? DatabaseToolkit.Pin()->Database->ViewportSettings->bDrawTrajectoryOrientations : false; })),
				NAME_None,
				EUserInterfaceActionType::ToggleButton);

			ShowMenuBuilder.EndSection();

			//-------------------------------

			ShowMenuBuilder.BeginSection("Ranges", LOCTEXT("ShowMenu_RangesLabel", "Ranges"));

			ShowMenuBuilder.AddMenuEntry(
				LOCTEXT("ShowMenu_RangesLabelRangeIdentifiers", "Range Identifiers"),
				LOCTEXT("ShowMenu_RangesLabelRangeIdentifiersToolTip", "Draw Range Identifiers"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([this]() { if (DatabaseToolkit.IsValid()) {
				DatabaseToolkit.Pin()->Database->ViewportSettings->bDrawRangeIdentifier = !DatabaseToolkit.Pin()->Database->ViewportSettings->bDrawRangeIdentifier;
			} }),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([this]() { return DatabaseToolkit.IsValid() ? DatabaseToolkit.Pin()->Database->ViewportSettings->bDrawRangeIdentifier : false; })),
				NAME_None,
				EUserInterfaceActionType::ToggleButton);

			ShowMenuBuilder.AddMenuEntry(
				LOCTEXT("ShowMenu_RangesLabelSkeletonBones", "Color Skeleton Bones"),
				LOCTEXT("ShowMenu_RangesLabelSkeletonBonesToolTip", "Color the drawn skeleton bones using the range identifier color"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([this]() { if (DatabaseToolkit.IsValid()) {
				DatabaseToolkit.Pin()->Database->ViewportSettings->bColorSkeletonBonesUsingRangeIdentifier = !DatabaseToolkit.Pin()->Database->ViewportSettings->bColorSkeletonBonesUsingRangeIdentifier;
			} }),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([this]() { return DatabaseToolkit.IsValid() ? DatabaseToolkit.Pin()->Database->ViewportSettings->bColorSkeletonBonesUsingRangeIdentifier : false; })),
				NAME_None,
				EUserInterfaceActionType::ToggleButton);

			ShowMenuBuilder.EndSection();

			//-------------------------------

			ShowMenuBuilder.BeginSection("Origin", LOCTEXT("ShowMenu_OriginLabel", "Origin"));

			ShowMenuBuilder.AddMenuEntry(
				LOCTEXT("ShowMenu_OriginLabelOrigin", "Origin"),
				LOCTEXT("ShowMenu_OriginLabelOriginToolTip", "Draw Origin"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([this]() { if (DatabaseToolkit.IsValid()) {
				DatabaseToolkit.Pin()->Database->ViewportSettings->bDrawOrigin = !DatabaseToolkit.Pin()->Database->ViewportSettings->bDrawOrigin;
			} }),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([this]() { return DatabaseToolkit.IsValid() ? DatabaseToolkit.Pin()->Database->ViewportSettings->bDrawOrigin : false; })),
				NAME_None,
				EUserInterfaceActionType::ToggleButton);

			ShowMenuBuilder.EndSection();

			//-------------------------------
		}

		return ShowMenuBuilder.MakeWidget();
	}

	/** Custom asset browser for database which filters assets which match the selected skeleton */
	class SDatabaseAssetBrowser : public SBox
	{
	public:

		SLATE_BEGIN_ARGS(SDatabaseAssetBrowser) {}
		SLATE_END_ARGS();

		void Construct(const FArguments& InArgs, TWeakPtr<FDatabaseToolkit> InDatabaseToolKit)
		{
			DatabaseToolkit = InDatabaseToolKit;

			ChildSlot
				[
					SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						[
							SAssignNew(AssetBrowserBox, SBox)
						]
				];

			RefreshView();

			/** Add callback to update view if skeleton changes */
			if (const TSharedPtr<FDatabaseToolkit> Toolkit = DatabaseToolkit.Pin())
			{
				OnSkeletonChangedDelegate = DatabaseToolkit.Pin()->Database->OnSkeletonChanged.AddLambda([this]() { RefreshAssetViewDelegate.ExecuteIfBound(true); });
			}
		}

		~SDatabaseAssetBrowser()
		{
			if (const TSharedPtr<FDatabaseToolkit> Toolkit = DatabaseToolkit.Pin())
			{
				Toolkit->Database->OnSkeletonChanged.Remove(OnSkeletonChangedDelegate);
			}
		}

		void RefreshView()
		{
			FAssetPickerConfig AssetPickerConfig;

			AssetPickerConfig.Filter.ClassPaths.Add(UAnimSequence::StaticClass()->GetClassPathName());
			AssetPickerConfig.Filter.bRecursiveClasses = true;

			AssetPickerConfig.bAddFilterUI = true;
			AssetPickerConfig.bAllowNullSelection = false;
			AssetPickerConfig.InitialAssetViewType = EAssetViewType::Column;
			AssetPickerConfig.bShowPathInColumnView = true;
			AssetPickerConfig.bShowTypeInColumnView = false;
			AssetPickerConfig.InitialThumbnailSize = EThumbnailSize::Small;
			AssetPickerConfig.RefreshAssetViewDelegates.Add(&RefreshAssetViewDelegate);
			AssetPickerConfig.OnShouldFilterAsset = FOnShouldFilterAsset::CreateSP(this, &SDatabaseAssetBrowser::OnShouldFilterAsset);
			AssetPickerConfig.OnAssetDoubleClicked = FOnAssetSelected::CreateSP(this, &SDatabaseAssetBrowser::OnAssetDoubleClicked);
			AssetPickerConfig.AssetShowWarningText = LOCTEXT("NoAssets_Warning", "No Assets found. No compatible assets with the database's skeleton where found. Ensure your assets' skeleton matches a skeleton from the database.");
			AssetPickerConfig.bCanShowDevelopersFolder = true;

			// Hide all asset registry columns by default (we only really want the name and path)
			const UObject* AnimSequenceDefaultObject = UAnimSequence::StaticClass()->GetDefaultObject();
			FAssetRegistryTagsContextData TagsContext(AnimSequenceDefaultObject, EAssetRegistryTagsCaller::Uncategorized);
			AnimSequenceDefaultObject->GetAssetRegistryTags(TagsContext);
			for (const TPair<FName, UObject::FAssetRegistryTag>& TagPair : TagsContext.Tags)
			{
				AssetPickerConfig.HiddenColumnNames.Add(TagPair.Key.ToString());
			}

			// Also hide the type column by default (but allow users to enable it, so don't use bShowTypeInColumnView)
			AssetPickerConfig.HiddenColumnNames.Add(TEXT("Class"));
			AssetPickerConfig.HiddenColumnNames.Add(ContentBrowserItemAttributes::ItemDiskSize.ToString());
			AssetPickerConfig.HiddenColumnNames.Add(ContentBrowserItemAttributes::VirtualizedData.ToString());

			FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
			AssetBrowserBox->SetContent(ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig));
		}

	private:

		TWeakPtr<FDatabaseToolkit> DatabaseToolkit;

		TSharedPtr<SBox> AssetBrowserBox;

		void OnAssetDoubleClicked(const FAssetData& AssetData)
		{
			if (const UObject* Asset = AssetData.GetAsset())
			{
				if (Cast<UAnimSequence>(Asset))
				{
					GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Asset);
				}
			}
		}

		bool OnShouldFilterAsset(const FAssetData& AssetData)
		{
			const bool bIsAssetCompatibleWithDatabase = 
				AssetData.GetClass()->IsChildOf(UAnimSequence::StaticClass()) &&
				DatabaseToolkit.IsValid() &&
				DatabaseToolkit.Pin()->Database->GetSkeleton() &&
				DatabaseToolkit.Pin()->Database->GetSkeleton()->IsCompatibleForEditor(AssetData);

			return !bIsAssetCompatibleWithDatabase;
		}

		/** Delegate for when the skeleton changes */
		FDelegateHandle OnSkeletonChangedDelegate;

		/* We need to be able to refresh the asset list if requested (i.e. database skeleton) */
		FRefreshAssetViewDelegate RefreshAssetViewDelegate;
	};

	static const FName DatabaseTabAssetDetails(TEXT("AnimDatabaseEditorAssetDetailsTabID"));
	static const FName DatabaseTabViewportSettings(TEXT("AnimDatabaseEditorViewportSettingsTabID"));
	static const FName DatabaseTabPreviewSettings(TEXT("AnimDatabaseEditorPreviewSettingsTabID"));
	static const FName DatabaseTabViewport(TEXT("AnimDatabaseEditorViewportTabID"));
	static const FName DatabaseTabTimeline(TEXT("AnimDatabaseEditorTimelineTabID"));
	static const FName DatabaseTabQuery(TEXT("AnimDatabaseEditorQueryTabID"));
	static const FName DatabaseTabAssetBrowser(TEXT("AnimDatabaseEditorAsserBrowserTabID"));

	void FDatabaseToolkit::InitAssetEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UAnimDatabase* InDatabase)
	{
		// Store Database Asset
		Database = InDatabase;
		check(InDatabase->ViewportSettings);

		// Create Query if for some reason it doesn't exist
		if (!Database->Query)
		{
			Database->Query = NewObject<UAnimDatabaseQuery>(Database, TEXT("Query"));
		}

		// Load all AnimSequences
		InDatabase->WaitForCompressionOnAll();

		// Reset all the transient properties and update when we are opening a new asset
		InDatabase->Query->Database = InDatabase;
		InDatabase->Query->Update();

		// Create Preview Scene
		PreviewScene = MakeShared<FPreviewScene>(FPreviewScene::ConstructionValues().ShouldSimulatePhysics(true).ForceUseMovementComponentInNonGameWorld(true));
		//Temporary fix for missing attached assets - MDW (Copied from FPersonaToolkit::CreatePreviewScene)
		PreviewScene->GetWorld()->GetWorldSettings()->SetIsTemporarilyHiddenInEditor(false);

		// Create Timeline Model
		TimelineModel = MakeShared<FTimelineModel>();

		// Create Tracks Model
		TracksModel = MakeShared<FTimelineTracksModel>();

		// Create Widgets
		ViewportWidget = SNew(SDatabaseViewport, StaticCastSharedRef<FDatabaseToolkit>(AsShared()), PreviewScene.ToSharedRef(), FDatabaseMode::EditorModeId);
		TimelineWidget = SNew(STimeline, TimelineModel.ToWeakPtr(), TracksModel);

		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

		FDetailsViewArgs Args;
		Args.bHideSelectionTip = true;

		EditingAssetWidget = PropertyModule.CreateDetailView(Args);
		EditingAssetWidget->SetObject(InDatabase);

		ViewportSettingsWidget = PropertyModule.CreateDetailView(Args);
		ViewportSettingsWidget->SetObject(InDatabase->ViewportSettings);

		FDetailsViewArgs QueryArgs;
		Args.bHideSelectionTip = true;
		Args.bAllowSearch = false;

		QueryWidget = PropertyModule.CreateDetailView(Args);
		QueryWidget->SetObject(InDatabase->Query);

		FAdvancedPreviewSceneModule& AdvancedPreviewSceneModule = FModuleManager::LoadModuleChecked<FAdvancedPreviewSceneModule>("AdvancedPreviewScene");
		PreviewSettingsWidget = AdvancedPreviewSceneModule.CreateAdvancedPreviewSceneSettingsWidget(PreviewScene.ToSharedRef());

		AssetBrowserWidget = SNew(SDatabaseAssetBrowser, StaticCastWeakPtr<FDatabaseToolkit>(AsWeak()));

		// Define Editor Layout
		const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_AnimDatabaseEditorDatabase_Layout_v0.14")
			->AddArea
			(
				FTabManager::NewPrimaryArea()->SetOrientation(Orient_Vertical)
				->Split
				(
					FTabManager::NewSplitter()->SetOrientation(Orient_Horizontal)
					->Split
					(
						FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)->SetSizeCoefficient(0.2f)
						->Split
						(
							FTabManager::NewStack()
							->AddTab(DatabaseTabQuery, ETabState::OpenedTab)
							->SetForegroundTab(DatabaseTabQuery)
						)
					)
					->Split
					(
						FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)->SetSizeCoefficient(0.6f)
						->Split
						(
							FTabManager::NewStack()
							->SetSizeCoefficient(0.9f)
							->AddTab(DatabaseTabViewport, ETabState::OpenedTab)->SetHideTabWell(true)
						)
						->Split
						(
							FTabManager::NewStack()
							->SetSizeCoefficient(0.1f)
							->AddTab(DatabaseTabTimeline, ETabState::OpenedTab)->SetHideTabWell(true)
						)
					)
					->Split(
						FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)->SetSizeCoefficient(0.2f)
						->Split
						(
							FTabManager::NewStack()	
							->SetSizeCoefficient(0.5f)
							->AddTab(DatabaseTabAssetDetails, ETabState::OpenedTab)
							->AddTab(DatabaseTabPreviewSettings, ETabState::OpenedTab)
							->SetForegroundTab(DatabaseTabAssetDetails)
						)
						->Split
						(
							FTabManager::NewStack()
							->SetSizeCoefficient(0.5f)
							->AddTab(DatabaseTabViewportSettings, ETabState::OpenedTab)
							->AddTab(DatabaseTabAssetBrowser, ETabState::OpenedTab)
							->SetForegroundTab(DatabaseTabViewportSettings)
						)
					)
				)
			);

		FAssetEditorToolkit::InitAssetEditor(
			Mode, InitToolkitHost,
			TEXT("AnimDatabaseEditorDatabaseApp"),
			StandaloneDefaultLayout,
			true, // CreateDefaultStandaloneMenu
			true, // CreateDefaultToolbar
			InDatabase);

		RegenerateMenusAndToolbars();
	}

	void FDatabaseToolkit::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
	{
		FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

		WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_AnimDatabaseEditorDatabase", "Animation Database Editor"));

		InTabManager->RegisterTabSpawner(DatabaseTabViewport, FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs&) {
			return SNew(SDockTab).Label(LOCTEXT("ViewportTab_Title", "Viewport"))[ ViewportWidget.ToSharedRef() ]; }))
			.SetDisplayName(LOCTEXT("ViewportTab", "Viewport"))
			.SetGroup(WorkspaceMenuCategory.ToSharedRef())
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.EventGraph_16x"));

		InTabManager->RegisterTabSpawner(DatabaseTabAssetDetails, FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs&) {
			return SNew(SDockTab).Label(LOCTEXT("AssetDetailsTab_Title", "Asset Details"))[EditingAssetWidget.ToSharedRef()]; }))
			.SetDisplayName(LOCTEXT("AssetDetailsTab", "Database Details"))
			.SetGroup(WorkspaceMenuCategory.ToSharedRef())
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

		InTabManager->RegisterTabSpawner(DatabaseTabViewportSettings, FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs&) {
			return SNew(SDockTab).Label(LOCTEXT("ViewportSettings_Title", "Viewport Settings"))[ViewportSettingsWidget.ToSharedRef()]; }))
			.SetDisplayName(LOCTEXT("ViewportSettingsTab", "Viewport Settings"))
			.SetGroup(WorkspaceMenuCategory.ToSharedRef())
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

		InTabManager->RegisterTabSpawner(DatabaseTabPreviewSettings, FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs&) {
			return SNew(SDockTab).Label(LOCTEXT("PreviewSceneSettingsTab", "Preview Scene Settings")) [ PreviewSettingsWidget.ToSharedRef() ]; }))
			.SetDisplayName(LOCTEXT("PreviewSettingsTab", "Preview Settings"))
			.SetGroup(WorkspaceMenuCategory.ToSharedRef())
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

		InTabManager->RegisterTabSpawner(DatabaseTabTimeline, FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs&) {
			return SNew(SDockTab).Label(LOCTEXT("Timeline_Title", "Timeline"))[ TimelineWidget.ToSharedRef() ]; }))
			.SetDisplayName(LOCTEXT("TimelineTab", "Timeline"))
			.SetGroup(WorkspaceMenuCategory.ToSharedRef())
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

		InTabManager->RegisterTabSpawner(DatabaseTabQuery, FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs&) {
			return SNew(SDockTab).Label(LOCTEXT("Query_Title", "Query"))[ QueryWidget.ToSharedRef() ]; }))
			.SetDisplayName(LOCTEXT("QueryTab", "Query"))
			.SetGroup(WorkspaceMenuCategory.ToSharedRef())
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "FoliageEditMode.Filter"));
		
		InTabManager->RegisterTabSpawner(DatabaseTabAssetBrowser, FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs&) {
			return SNew(SDockTab).Label(LOCTEXT("AssetBrowser_Title", "Asset Browser"))[ AssetBrowserWidget.ToSharedRef() ]; }))
			.SetDisplayName(LOCTEXT("AssetBrowserTab", "Asset Browser"))
			.SetGroup(WorkspaceMenuCategory.ToSharedRef())
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.ContentBrowser"));
	}

	void FDatabaseToolkit::UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
	{
		FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

		InTabManager->UnregisterTabSpawner(DatabaseTabViewport);
		InTabManager->UnregisterTabSpawner(DatabaseTabAssetDetails);
		InTabManager->UnregisterTabSpawner(DatabaseTabViewportSettings);
		InTabManager->UnregisterTabSpawner(DatabaseTabPreviewSettings);
		InTabManager->UnregisterTabSpawner(DatabaseTabTimeline);
		InTabManager->UnregisterTabSpawner(DatabaseTabQuery);
		InTabManager->UnregisterTabSpawner(DatabaseTabAssetBrowser);
	}

	FName FDatabaseToolkit::GetToolkitFName() const
	{
		return FName("AnimDatabaseEditorDatabase");
	}

	FText FDatabaseToolkit::GetBaseToolkitName() const
	{
		return LOCTEXT("AnimDatabaseEditorDatabaseAppLabel", "Animation Database Editor");
	}

	FText FDatabaseToolkit::GetToolkitName() const
	{
		return FText::Format(LOCTEXT("AnimDatabaseEditorDatabaseToolkitName", "{0}"), FText::FromString(Database->GetName()));
	}

	FLinearColor FDatabaseToolkit::GetWorldCentricTabColorScale() const
	{
		return FLinearColor::White;
	}

	FString FDatabaseToolkit::GetWorldCentricTabPrefix() const
	{
		return TEXT("AnimDatabaseEditorDatabase");
	}

	void FDatabaseToolkit::ReconstructTimelineWidget()
	{
		// Re-create the timeline widget
		TimelineWidget = SNew(STimeline, TimelineModel.ToWeakPtr(), TracksModel);

		// Switch out the content of the timeline tab to the new widget
		if (TSharedPtr<SDockTab> Tab = TabManager->FindExistingLiveTab(DatabaseTabTimeline))
		{
			Tab->SetContent(TimelineWidget.ToSharedRef());
		}
	}
}

#undef LOCTEXT_NAMESPACE

