// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimDatabaseEditorViewportClient.h"

#include "SAnimDatabaseEditorViewport.h"
#include "AnimDatabaseEditorPreviewScene.h"

#include "AssetEditorModeManager.h"
#include "UnrealWidget.h"
#include "CanvasItem.h"
#include "Engine/Canvas.h"

#include "DrawDebugLibrary.h"

namespace UE::AnimDatabase::Editor
{
	FViewportClient::FViewportClient(const TWeakPtr<FPreviewScene> InPreviewScene, const TSharedRef<SEditorViewport>& InViewport, const TWeakPtr<FAssetEditorToolkit> InAssetEditorToolkit, const FEditorModeID ModeID)
		: FEditorViewportClient(nullptr, InPreviewScene.Pin().Get(), InViewport)
		, PreviewScenePtr(InPreviewScene)
		, AssetEditorToolkitPtr(InAssetEditorToolkit)
	{
		Widget->SetUsesEditorModeTools(ModeTools.Get());
		StaticCastSharedPtr<FAssetEditorModeManager>(ModeTools)->SetPreviewScene(InPreviewScene.Pin().Get());
		ModeTools->SetDefaultMode(ModeID);

		SetRealtime(true);

		FEditorViewportClient::SetWidgetCoordSystemSpace(COORD_Local);
		ModeTools->SetWidgetMode(UE::Widget::WM_Translate);
	}

	FViewportClient::~FViewportClient() = default;

	void FViewportClient::ClearWarningMessage()
	{
		WarningMessage = FText();
	}

	TWeakPtr<FPreviewScene> FViewportClient::GetPreviewScene() const { return PreviewScenePtr; }

	TWeakPtr<FAssetEditorToolkit> FViewportClient::GetAssetEditorToolkit() const { return AssetEditorToolkitPtr; }

	void FViewportClient::SetWarningMessage(const FText& InWarningMessage)
	{
		WarningMessage = InWarningMessage;
	}

	const TSharedPtr<FDebugDrawerCanvasBuffer>& FViewportClient::GetOrMakeDebugDrawBuffer()
	{
		if (DebugDrawBuffer)
		{
			return DebugDrawBuffer;
		}
		else
		{
			DebugDrawBuffer = MakeShared<FDebugDrawerCanvasBuffer>();
			return DebugDrawBuffer;
		}
	}

	void FViewportClient::Draw(FViewport* InViewport, FCanvas* Canvas)
	{
		FEditorViewportClient::Draw(InViewport, Canvas);

		if (Canvas && !WarningMessage.IsEmpty())
		{
			UFont* Font = GEngine->GetSmallFont();
			FCanvasTextItem TextItem(FVector2D(50, 50), WarningMessage, Font, FLinearColor::Yellow);
			TextItem.EnableShadow(FLinearColor::Black);
			Canvas->DrawItem(TextItem);
		}

		if (DebugDrawBuffer)
		{
			DebugDrawBuffer->Flush(Canvas);
		}
	}

	void FViewportClient::TrackingStarted(const struct FInputEventState& InInputState, bool bIsDraggingWidget, bool bNudge)
	{
		ModeTools->StartTracking(this, Viewport);
	}

	void FViewportClient::TrackingStopped()
	{
		ModeTools->EndTracking(this, Viewport);
		Invalidate();
	}
}