// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAnimDatabaseEditorTimeline.h"
#include "AnimDatabaseEditorTimeline.h"

#include "LearningFrameSet.h"
#include "LearningFrameRangeSet.h"
#include "LearningFrameAttribute.h"

#include "Editor.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "EditorWidgetsModule.h"
#include "Fonts/FontMeasure.h"
#include "FrameNumberNumericInterface.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IPersonaPreviewScene.h"
#include "ISequencerWidgetsModule.h"
#include "Modules/ModuleManager.h"
#include "MovieSceneFwd.h"
#include "MovieSceneTimeHelpers.h"
#include "Preferences/PersonaOptions.h"
#include "Styling/AppStyle.h"
#include "TimeSliderArgs.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SWidget.h"
#include "Widgets/SWeakWidget.h"
#include "Widgets/SLeafWidget.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Views/SExpanderArrow.h"
#include "Layout/LayoutUtils.h"
#include "ITimeSlider.h"
#include "ITransportControl.h"

#define LOCTEXT_NAMESPACE "AnimDatabaseEditorTimeline"

namespace UE::AnimDatabase::Editor
{
	/** Splitter used on the anim timeline as an overlay. Input is disabled on all areas except the draggable positions */
	class STimelineSplitterOverlay : public SOverlay
	{
	public:
		typedef SSplitter::FArguments FArguments;

		void Construct(const FArguments& InArgs)
		{
			SetVisibility(EVisibility::SelfHitTestInvisible);

			Splitter = SArgumentNew(InArgs, SSplitter);
			Splitter->SetVisibility(EVisibility::HitTestInvisible);
			AddSlot()
				[
					Splitter.ToSharedRef()
				];

			for (int32 Index = 0; Index < Splitter->GetChildren()->Num() - 1; ++Index)
			{
				AddSlot()
					.Padding(TAttribute<FMargin>::Create(TAttribute<FMargin>::FGetter::CreateSP(this, &STimelineSplitterOverlay::GetSplitterHandlePadding, Index)))
					[
						SNew(SBox)
							.Visibility(EVisibility::Visible)
					];
			}
		}

		/** SWidget interface */
		virtual void OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const override
		{
			FArrangedChildren SplitterChildren(ArrangedChildren.GetFilter());
			Splitter->ArrangeChildren(AllottedGeometry, SplitterChildren);

			SlotPadding.Reset();

			for (int32 Index = 0; Index < SplitterChildren.Num() - 1; ++Index)
			{
				const auto& ThisGeometry = SplitterChildren[Index].Geometry;
				const auto& NextGeometry = SplitterChildren[Index + 1].Geometry;

				if (Splitter->GetOrientation() == EOrientation::Orient_Horizontal)
				{
					SlotPadding.Add(FMargin(
						ThisGeometry.Position.X + static_cast<float>(ThisGeometry.GetLocalSize().X),
						0,
						static_cast<float>(AllottedGeometry.Size.X) - NextGeometry.Position.X,
						0)
					);
				}
				else
				{
					SlotPadding.Add(FMargin(
						0,
						ThisGeometry.Position.Y + static_cast<float>(ThisGeometry.GetLocalSize().Y),
						0,
						static_cast<float>(AllottedGeometry.Size.Y) - NextGeometry.Position.Y)
					);
				}
			}

			SOverlay::OnArrangeChildren(AllottedGeometry, ArrangedChildren);
		}

		virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override
		{
			return Splitter->OnCursorQuery(MyGeometry, CursorEvent);
		}

		virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
		{
			FReply Reply = Splitter->OnMouseButtonDown(MyGeometry, MouseEvent);
			if (Reply.GetMouseCaptor().IsValid())
			{
				// Set us to be the mouse captor so we can forward events properly
				Reply.CaptureMouse(SharedThis(this));
				SetVisibility(EVisibility::Visible);
			}
			return Reply;
		}
		
		virtual void OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent) override
		{
			SetVisibility(EVisibility::SelfHitTestInvisible);
			SOverlay::OnMouseCaptureLost(CaptureLostEvent);
		}

		virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
		{
			FReply Reply = Splitter->OnMouseButtonUp(MyGeometry, MouseEvent);
			if (Reply.ShouldReleaseMouse())
			{
				SetVisibility(EVisibility::SelfHitTestInvisible);
			}
			return Reply;
		}

		virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
		{
			return Splitter->OnMouseMove(MyGeometry, MouseEvent);
		}

		virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override
		{
			return Splitter->OnMouseLeave(MouseEvent);
		}

	private:
		FMargin GetSplitterHandlePadding(int32 Index) const
		{
			if (SlotPadding.IsValidIndex(Index))
			{
				return SlotPadding[Index];
			}

			return 0.f;
		}

		TSharedPtr<SSplitter> Splitter;
		mutable TArray<FMargin> SlotPadding;
	};


	/**
	 * A time slider controller for the anim timeline. This is largely copy-pasted from the MLDeformer timeline controller.
	 */
	class FTimeSliderController : public ITimeSliderController
	{
	public:
		FTimeSliderController(const FTimeSliderArgs& InArgs, TWeakPtr<FTimelineModel> InWeakModel, TWeakPtr<STimeline> InWeakTimeline);

		void SetModel(TWeakPtr<FTimelineModel> InModel);
		virtual int32 OnPaintTimeSlider(bool bMirrorLabels, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
		virtual int32 OnPaintViewArea(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, bool bEnabled, const FPaintViewAreaArgs& Args) const override;
		virtual FReply OnMouseButtonDown(SWidget& WidgetOwner, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
		virtual FReply OnMouseButtonUp(SWidget& WidgetOwner, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
		virtual FReply OnMouseMove(SWidget& WidgetOwner, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
		virtual FReply OnMouseWheel(SWidget& WidgetOwner, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
		virtual FCursorReply OnCursorQuery(const SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;
		virtual void SetViewRange(double NewRangeMin, double NewRangeMax, EViewRangeInterpolation Interpolation) override;
		virtual void SetClampRange(double NewRangeMin, double NewRangeMax) override;
		virtual void SetPlayRange(FFrameNumber RangeStart, int32 RangeDuration) override;
		virtual FFrameRate GetDisplayRate() const override { return TimeSliderArgs.DisplayRate.Get(); }
		virtual FFrameRate GetTickResolution() const override { return TimeSliderArgs.TickResolution.Get(); }
		virtual FAnimatedRange GetViewRange() const override { return TimeSliderArgs.ViewRange.Get(); }
		virtual FAnimatedRange GetClampRange() const override { return TimeSliderArgs.ClampRange.Get(); }
		virtual TRange<FFrameNumber> GetPlayRange() const override { return TimeSliderArgs.PlaybackRange.Get(TRange<FFrameNumber>()); }
		virtual FFrameTime GetScrubPosition() const override { return TimeSliderArgs.ScrubPosition.Get(); }
		virtual void SetScrubPosition(FFrameTime InTime, bool bEvaluate) override { TimeSliderArgs.OnScrubPositionChanged.ExecuteIfBound(InTime, false, bEvaluate); }

		/**
		 * Clamp the given range to the clamp range.
		 * @param NewRangeMin The new lower bound of the range.
		 * @param NewRangeMax The new upper bound of the range.
		 */
		void ClampViewRange(const double NewRangeMin, const double NewRangeMax);

		/**
		 * Zoom the range by a given delta.
		 * @param InDelta The total amount to zoom by (+ve = zoom out, -ve = zoom in).
		 * @param ZoomBias Bias to apply to lower/upper extents of the range (0 = lower, 0.5 = equal, 1 = upper).
		 */
		bool ZoomByDelta(float InDelta, float ZoomBias = 0.5f);

		/**
		 * Pan the range by a given delta.
		 * @param InDelta The total amount to pan by (+ve = pan forwards in time, -ve = pan backwards in time).
		 */
		void PanByDelta(float InDelta);

		/** Determine frame time from a mouse position. */
		FFrameTime GetFrameTimeFromMouse(const FGeometry& Geometry, FVector2D ScreenSpacePosition) const;

	private:
		// Forward declared as class members to prevent name collision with similar types defined in other units.
		struct FDrawTickArgs;
		struct FScrubRangeToScreen;

		/**
		 * Call this method when the user's interaction has changed the scrub position.
		 * @param NewValue Value resulting from the user's interaction.
		 * @param bIsScrubbing True if done via scrubbing, false if just releasing scrubbing.
		 */
		void CommitScrubPosition(FFrameTime NewValue, bool bIsScrubbing);

		/**
		 * Draw time tick marks.
		 * @param OutDrawElements List to add draw elements to.
		 * @param ViewRange The currently visible time range in seconds.
		 * @param RangeToScreen Time range to screen space converter.
		 * @param InArgs Parameters for drawing the tick lines.
		 */
		void DrawTicks(FSlateWindowElementList& OutDrawElements, const TRange<double>& ViewRange, const FScrubRangeToScreen& RangeToScreen, FDrawTickArgs& InArgs) const;

		/**
		 * Draw the selection range.
		 * @return The new layer ID.
		 */
		int32 DrawSelectionRange(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FScrubRangeToScreen& RangeToScreen, const FPaintPlaybackRangeArgs& Args) const;

		/**
		 * Draw the playback range.
		 * @return the new layer ID.
		 */
		int32 DrawPlaybackRange(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FScrubRangeToScreen& RangeToScreen, const FPaintPlaybackRangeArgs& Args) const;

	private:
		/**
		 * Hit test the lower bound of a range.
		 */
		bool HitTestRangeStart(const FScrubRangeToScreen& RangeToScreen, const TRange<double>& Range, float HitPixel) const;

		/**
		 * Hit test the upper bound of a range.
		 */
		bool HitTestRangeEnd(const FScrubRangeToScreen& RangeToScreen, const TRange<double>& Range, float HitPixel) const;

		void SetPlaybackRangeStart(FFrameNumber NewStart) const;
		void SetPlaybackRangeEnd(FFrameNumber NewEnd) const;
		void SetSelectionRangeStart(FFrameNumber NewStart) const;
		void SetSelectionRangeEnd(FFrameNumber NewEnd) const;

		FFrameTime ComputeFrameTimeFromMouse(const FGeometry& Geometry, FVector2D ScreenSpacePosition, FScrubRangeToScreen RangeToScreen, bool CheckSnapping = true) const;

	private:
		struct FScrubPixelRange
		{
			TRange<float> Range;
			TRange<float> HandleRange;
			bool bClamped;
		};

		FScrubPixelRange GetHitTestScrubberPixelRange(FFrameTime ScrubTime, const FScrubRangeToScreen& RangeToScreen) const;
		FScrubPixelRange GetScrubberPixelRange(FFrameTime ScrubTime, const FScrubRangeToScreen& RangeToScreen) const;
		FScrubPixelRange GetScrubberPixelRange(FFrameTime ScrubTime, FFrameRate Resolution, FFrameRate PlayRate, const FScrubRangeToScreen& RangeToScreen, float DilationPixels = 0.f) const;

	private:
		/** Pointer back to the model object. */
		TWeakPtr<FTimelineModel> WeakModel;

		/** Pointer back to the timeline. */
		TWeakPtr<STimeline> WeakTimeline;

		/** Arguments used for constructing the time slider */
		FTimeSliderArgs TimeSliderArgs;

		/** Brush for drawing the fill area on the scrubber. */
		const FSlateBrush* ScrubFillBrush;

		/** Brush for drawing an upwards facing scrub handles. */
		const FSlateBrush* ScrubHandleUpBrush;

		/** Brush for drawing a downwards facing scrub handle. */
		const FSlateBrush* ScrubHandleDownBrush;

		/** Brush for drawing an editable time. */
		const FSlateBrush* EditableTimeBrush;

		/** Total mouse delta during dragging. **/
		float DistanceDragged;

		/** If we are dragging a scrubber or dragging to set the time range. */
		enum DragType
		{
			DRAG_SCRUBBING_TIME,
			DRAG_PLAYBACK_START,
			DRAG_PLAYBACK_END,
			DRAG_SELECTION_START,
			DRAG_SELECTION_END,
			DRAG_TIME,
			DRAG_NONE
		};

		DragType MouseDragType;

		/** If we are currently panning the panel. */
		bool bPanning;

		/** Index of the current dragged time. */
		int32 DraggedTimeIndex;

		/** Mouse down position range. */
		FVector2D MouseDownPosition = FVector2D::ZeroVector;
	};

	//----------------------------------------------------

	class SFramesTrack : public SLeafWidget
	{

	public:
		SLATE_BEGIN_ARGS(SFramesTrack) {}
			SLATE_ATTRIBUTE(TRange<FFrameNumber>, ViewRange)
			SLATE_ARGUMENT(FLinearColor, Color)
			SLATE_ARGUMENT(int32, ViewOffset)
		SLATE_END_ARGS()

		/** Construct this widget */
		void Construct(const FArguments& InArgs, const FAnimDatabaseFrames& InFramesObject)
		{
			ViewRange = InArgs._ViewRange;
			ViewOffset = InArgs._ViewOffset;
			Color = InArgs._Color;
			FramesObject = InFramesObject;
		}

		virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override
		{
			const float TrackWidth = AllottedGeometry.GetLocalSize().X;
			FVector2D FrameIconSize = FVector2D(12.0f, 12.0f);

			TRange<FFrameNumber> CurrViewRange = ViewRange.Get();

			const int32 ViewRangeMin = CurrViewRange.GetLowerBoundValue().Value;
			const int32 ViewRangeMax = CurrViewRange.GetUpperBoundValue().Value;
			const int32 ViewRangeLength = ViewRangeMax - ViewRangeMin;

			const float RangeSpaceToTrackSpace = TrackWidth / ViewRangeLength;

			check(FramesObject.FrameSet->GetEntryNum() == 1);

			const int32 FrameNum = FramesObject.FrameSet->GetEntryFrameNum(0);

			for (int32 FrameIdx = 0; FrameIdx < FrameNum; FrameIdx++)
			{
				const int32 Frame = FramesObject.FrameSet->GetEntryFrame(0, FrameIdx) - ViewOffset - ViewRangeMin;
				const float TrackFrame = RangeSpaceToTrackSpace * Frame;

				if (TrackFrame - FrameIconSize.X / 2.0f > TrackWidth || TrackFrame + FrameIconSize.X / 2.0f < 0.0f)
				{
					continue;
				}

				FVector2D ScrubHandlePosition(TrackFrame - FrameIconSize.X / 2.0f, (FTimelineTrack::Height - FrameIconSize.Y) / 2.f);
				FSlateDrawElement::MakeBox(
					OutDrawElements,
					LayerId,
					AllottedGeometry.ToPaintGeometry(FrameIconSize, FSlateLayoutTransform(ScrubHandlePosition)),
					FAppStyle::GetBrush(TEXT("Sequencer.KeyDiamond")),
					ESlateDrawEffect::None,
					Color.CopyWithNewOpacity(0.67f));

				FSlateDrawElement::MakeBox(
					OutDrawElements,
					LayerId,
					AllottedGeometry.ToPaintGeometry(FrameIconSize, FSlateLayoutTransform(ScrubHandlePosition)),
					FAppStyle::GetBrush(TEXT("Sequencer.KeyDiamondBorder")),
					ESlateDrawEffect::None,
					FLinearColor::Black);
			}

			return LayerId + 1;
		}

		virtual FVector2D ComputeDesiredSize(float LayoutScale) const override
		{
			return FVector2D(100.f, FTimelineTrack::Height);
		}

		TAttribute<TRange<FFrameNumber>> ViewRange;
		int32 ViewOffset = 0;
		FLinearColor Color = FLinearColor::Black;
		FAnimDatabaseFrames FramesObject;
	};

	//----------------------------------------------------

	class SFrameRangesTrack : public SLeafWidget
	{

	public:
		SLATE_BEGIN_ARGS(SFrameRangesTrack) {}
			SLATE_ATTRIBUTE(TRange<FFrameNumber>, ViewRange)
			SLATE_ARGUMENT(FLinearColor, Color)
			SLATE_ARGUMENT(int32, ViewOffset)
		SLATE_END_ARGS()

		/** Construct this widget */
		void Construct(const FArguments& InArgs, const FAnimDatabaseFrameRanges& InFrameRangesObject)
		{
			ViewRange = InArgs._ViewRange;
			ViewOffset = InArgs._ViewOffset;
			Color = InArgs._Color;
			FrameRangesObject = InFrameRangesObject;
		}

		virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override
		{
			const float PaddingY = 4.0f;
			const float TrackWidth = AllottedGeometry.GetLocalSize().X;

			TRange<FFrameNumber> CurrViewRange = ViewRange.Get();

			const int32 ViewRangeMin = CurrViewRange.GetLowerBoundValue().Value;
			const int32 ViewRangeMax = CurrViewRange.GetUpperBoundValue().Value;
			const int32 ViewRangeLength = ViewRangeMax - ViewRangeMin;

			const float RangeSpaceToTrackSpace = TrackWidth / ViewRangeLength;

			if (!ensure(FrameRangesObject.FrameRangeSet->GetEntryNum() == 1)) { return LayerId + 1; }

			const int32 RangeNum = FrameRangesObject.FrameRangeSet->GetEntryRangeNum(0);

			for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
			{
				const int32 Start = FrameRangesObject.FrameRangeSet->GetEntryRangeStart(0, RangeIdx) - ViewOffset - ViewRangeMin;
				const int32 Length = FrameRangesObject.FrameRangeSet->GetEntryRangeLength(0, RangeIdx);
				const float RangeStart = RangeSpaceToTrackSpace * Start;
				const float RangeWidth = FMath::Max(RangeSpaceToTrackSpace * (Length - 1), 1.0f);

				FVector2D DurationBoxSize = FVector2D(RangeWidth, FTimelineTrack::Height - 2.0f * PaddingY);
				FVector2D DurationBoxPosition = FVector2D(RangeStart, PaddingY);

				if (DurationBoxPosition.X > TrackWidth ||
					DurationBoxPosition.X + DurationBoxSize.X < 0.0f)
				{
					continue;
				}

				FPaintGeometry BoxGeometry = AllottedGeometry.ToPaintGeometry(DurationBoxSize, FSlateLayoutTransform(DurationBoxPosition));

				FSlateDrawElement::MakeBox(
					OutDrawElements,
					LayerId,
					BoxGeometry,
					FAppStyle::GetBrush(TEXT("SpecialEditableTextImageNormal")),
					ESlateDrawEffect::None,
					Color.CopyWithNewOpacity(0.67f));
			}

			return LayerId + 1;
		}

		virtual FVector2D ComputeDesiredSize(float LayoutScale) const override
		{
			return FVector2D(100.f, FTimelineTrack::Height);
		}

		TAttribute<TRange<FFrameNumber>> ViewRange;
		int32 ViewOffset = 0;
		FLinearColor Color = FLinearColor::Black;
		FAnimDatabaseFrameRanges FrameRangesObject;
	};


	//----------------------------------------------------

	class SFrameAttributeTrack : public SLeafWidget
	{

	public:
		SLATE_BEGIN_ARGS(SFrameAttributeTrack) {}
			SLATE_ATTRIBUTE(TRange<FFrameNumber>, ViewRange)
			SLATE_ARGUMENT(FLinearColor, Color)
			SLATE_ARGUMENT(int32, ViewOffset)
		SLATE_END_ARGS()

		/** Construct this widget */
		void Construct(const FArguments& InArgs, const FAnimDatabaseFrameAttribute& InFrameAttributeObject)
		{
			ViewRange = InArgs._ViewRange;
			ViewOffset = InArgs._ViewOffset;
			Color = InArgs._Color;
			FrameAttributeObject = InFrameAttributeObject;
		}

		virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override
		{
			const float TrackWidth = AllottedGeometry.GetLocalSize().X;
			FVector2D FrameIconSize = FVector2D(12.0f, 12.0f);

			TRange<FFrameNumber> CurrViewRange = ViewRange.Get();

			const int32 ViewRangeMin = CurrViewRange.GetLowerBoundValue().Value;
			const int32 ViewRangeMax = CurrViewRange.GetUpperBoundValue().Value;
			const int32 ViewRangeLength = ViewRangeMax - ViewRangeMin;
			
			const float RangeSpaceToTrackSpace = TrackWidth / ViewRangeLength;

			if (!ensure(FrameAttributeObject.FrameAttribute->FrameRangeSet.GetEntryNum() == 1)) { return LayerId + 1; }

			const int32 RangeNum = FrameAttributeObject.FrameAttribute->FrameRangeSet.GetEntryRangeNum(0);

			if (FrameAttributeObject.Type == EAnimDatabaseAttributeType::Float ||
				FrameAttributeObject.Type == EAnimDatabaseAttributeType::Bool)
			{
				const float Thickness = 1.0f;
				TArray<FVector2f> Points;

				for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
				{
					const int32 Start = FrameAttributeObject.FrameAttribute->FrameRangeSet.GetEntryRangeStart(0, RangeIdx) - ViewOffset - ViewRangeMin;
					const int32 Length = FrameAttributeObject.FrameAttribute->FrameRangeSet.GetEntryRangeLength(0, RangeIdx);
					const int32 Offset = FrameAttributeObject.FrameAttribute->FrameRangeSet.GetEntryRangeOffset(0, RangeIdx);

					if (Length <= 1) continue;

					const float RangeStart = RangeSpaceToTrackSpace * Start;
					const float RangeWidth = FMath::Max(RangeSpaceToTrackSpace * (Length - 1), 1.0f);

					FVector2D DurationBoxSize = FVector2D(RangeWidth, FTimelineTrack::Height);
					FVector2D DurationBoxPosition = FVector2D(RangeStart, 0.0f);

					if (DurationBoxPosition.X > TrackWidth ||
						DurationBoxPosition.X + DurationBoxSize.X < 0.0f)
					{
						continue;
					}

					FPaintGeometry BoxGeometry = AllottedGeometry.ToPaintGeometry(DurationBoxSize, FSlateLayoutTransform(DurationBoxPosition));

					Points.Reset();
					float MinValue = +UE_MAX_FLT;
					float MaxValue = -UE_MAX_FLT;
					for (int32 FrameIdx = 0; FrameIdx < Length; FrameIdx++)
					{
						const float PositionX = RangeWidth * ((float)FrameIdx / (Length - 1));
						const float PositionY = FrameAttributeObject.GetAsFloat(Offset + FrameIdx);
						Points.Add(FVector2f(PositionX, PositionY));
						MinValue = FMath::Min(MinValue, PositionY);
						MaxValue = FMath::Max(MaxValue, PositionY);
					}

					for (int32 FrameIdx = 0; FrameIdx < Length; FrameIdx++)
					{
						const float NormalizedValue = ((Points[FrameIdx].Y - MinValue) / (MaxValue - MinValue + UE_KINDA_SMALL_NUMBER));
						Points[FrameIdx].Y = FTimelineTrack::Height - Thickness - ((FTimelineTrack::Height - 2.0f * Thickness) * NormalizedValue);
					}

					FSlateDrawElement::MakeLines(
						OutDrawElements,
						LayerId,
						BoxGeometry,
						Points,
						ESlateDrawEffect::None,
						Color.CopyWithNewOpacity(0.67f),
						true,
						Thickness);
				}
			}
			else if (FrameAttributeObject.Type == EAnimDatabaseAttributeType::Event)
			{
				for (int32 RangeIdx = 0; RangeIdx < RangeNum; RangeIdx++)
				{
					const int32 Start = FrameAttributeObject.FrameAttribute->FrameRangeSet.GetEntryRangeStart(0, RangeIdx) - ViewOffset - ViewRangeMin;
					const int32 Length = FrameAttributeObject.FrameAttribute->FrameRangeSet.GetEntryRangeLength(0, RangeIdx);
					const int32 Offset = FrameAttributeObject.FrameAttribute->FrameRangeSet.GetEntryRangeOffset(0, RangeIdx);

					const float RangeStart = RangeSpaceToTrackSpace * Start;
					const float RangeWidth = FMath::Max(RangeSpaceToTrackSpace * (Length - 1), 1.0f);

					FVector2D DurationBoxSize = FVector2D(RangeWidth, FTimelineTrack::Height);
					FVector2D DurationBoxPosition = FVector2D(RangeStart, 0.0f);

					if (DurationBoxPosition.X > TrackWidth ||
						DurationBoxPosition.X + DurationBoxSize.X < 0.0f)
					{
						continue;
					}

					FPaintGeometry BoxGeometry = AllottedGeometry.ToPaintGeometry(DurationBoxSize, FSlateLayoutTransform(DurationBoxPosition));

					for (int32 FrameIdx = 0; FrameIdx < Length; FrameIdx++)
					{
						bool bTimeUntilEventKnown = false;
						float TimeUntilEvent = UE_MAX_FLT;
						FrameAttributeObject.GetAsEvent(bTimeUntilEventKnown, TimeUntilEvent, Offset + FrameIdx);

						if (bTimeUntilEventKnown && FMath::Abs(TimeUntilEvent) < UE_KINDA_SMALL_NUMBER)
						{
							const float TrackFrame = RangeSpaceToTrackSpace * (Start + FrameIdx);

							if (TrackFrame - FrameIconSize.X / 2.0f > TrackWidth || TrackFrame + FrameIconSize.X / 2.0f < 0.0f)
							{
								continue;
							}

							FVector2D ScrubHandlePosition(TrackFrame - FrameIconSize.X / 2.0f, (FTimelineTrack::Height - FrameIconSize.Y) / 2.f);
							FSlateDrawElement::MakeBox(
								OutDrawElements,
								LayerId,
								AllottedGeometry.ToPaintGeometry(FrameIconSize, FSlateLayoutTransform(ScrubHandlePosition)),
								FAppStyle::GetBrush(TEXT("Sequencer.KeyDiamond")),
								ESlateDrawEffect::None,
								Color.CopyWithNewOpacity(0.67f));

							FSlateDrawElement::MakeBox(
								OutDrawElements,
								LayerId,
								AllottedGeometry.ToPaintGeometry(FrameIconSize, FSlateLayoutTransform(ScrubHandlePosition)),
								FAppStyle::GetBrush(TEXT("Sequencer.KeyDiamondBorder")),
								ESlateDrawEffect::None,
								FLinearColor::Black);
						}
					}
				}
			}


			return LayerId + 1;
		}

		virtual FVector2D ComputeDesiredSize(float LayoutScale) const override
		{
			return FVector2D(100.f, FTimelineTrack::Height);
		}

		TAttribute<TRange<FFrameNumber>> ViewRange;
		int32 ViewOffset = 0;
		FLinearColor Color = FLinearColor::Black;
		FAnimDatabaseFrameAttribute FrameAttributeObject;
	};

	//----------------------------------------------------

	class SOutliner;
	class STrackArea;

	class STrack : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(STrack) {}

			SLATE_DEFAULT_SLOT(FArguments, Content)

		SLATE_END_ARGS()

		/** Construct this widget */
		void Construct(const FArguments& InArgs, const TSharedPtr<FTimelineTrack>& InTrack);

		virtual FVector2D ComputeDesiredSize(float LayoutScale) const override;

		virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

	public:

		TWeakPtr<FTimelineTrack> WeakTrack;
	};

	//----------------------------------------------------

	/**
	 * Structure representing a slot in the track area.
	 */
	class FTrackAreaSlot : public TSlotBase<FTrackAreaSlot>, public TAlignmentWidgetSlotMixin<FTrackAreaSlot>
	{
	public:

		/** Construction from a track lane */
		FTrackAreaSlot(const TSharedPtr<STrack>& InSlotContent)
			: TAlignmentWidgetSlotMixin<FTrackAreaSlot>(HAlign_Fill, VAlign_Top)
		{
			TrackWidget = InSlotContent;

			AttachWidget(
				SNew(SWeakWidget)
				.Clipping(EWidgetClipping::ClipToBounds)
				.PossiblyNullContent(InSlotContent)
			);
		}

		/** Get the vertical position of this slot inside its parent. */
		float GetVerticalOffset() const
		{
			if (TSharedPtr<STrack> Track = TrackWidget.Pin())
			{
				return Track->WeakTrack.IsValid() ? Track->WeakTrack.Pin()->Offset : 0.0f;
			}
			else
			{
				return 0.0f;
			}
		}

		/** The track that we represent. */
		TWeakPtr<STrack> TrackWidget;
	};

	class STrackArea : public SPanel
	{
	public:

		SLATE_BEGIN_ARGS(STrackArea)
			{
				_Clipping = EWidgetClipping::ClipToBounds;
			}
		SLATE_END_ARGS()

		STrackArea()
			: Children(this)
		{}

		/** Construct this widget */
		void Construct(const FArguments& InArgs,
			const TSharedRef<FTimeSliderController>& InTimeSliderController);

		void SetOutliner(const TSharedPtr<SOutliner>& InOutliner);

	public:

		/** Empty the track area */
		void Empty();

		void AddTrack(const TSharedRef<STrack>& Track);

	public:

		/** SWidget interface */
		virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
		virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
		virtual void OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const override;
		virtual FVector2D ComputeDesiredSize(float) const override;
		virtual FChildren* GetChildren() override;

		void UpdateHoverStates(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
		FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
		FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
		FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
		FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
		void OnMouseLeave(const FPointerEvent& MouseEvent);
		FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const;

	private:

		/** The track area's children. */
		TPanelChildren<FTrackAreaSlot> Children;

	private:

		/** Weak pointer to the time slider (used for scrubbing interactions). */
		TWeakPtr<FTimeSliderController> WeakTimeSliderController;

		/** Weak pointer to the outliner (used for scrolling interactions). */
		TWeakPtr<SOutliner> WeakOutliner;
	};

	class SOutliner : public STreeView<TSharedRef<FTimelineTrack>>
	{
	public:
		SLATE_BEGIN_ARGS(SOutliner) {}

			/** Externally supplied scroll bar */
			SLATE_ARGUMENT(TSharedPtr<SScrollBar>, ExternalScrollbar)

		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, 
			const TWeakPtr<FTimelineModel>& InTimelineModel, 
			const TWeakPtr<FTimelineTracksModel>& InTracksModel, 
			const TWeakPtr<STrackArea>& InTrackArea);
		
		TSharedRef<ITableRow> HandleGenerateRow(TSharedRef<FTimelineTrack> InTrack, const TSharedRef<STableViewBase>& OwnerTable);
		void HandleGetChildren(TSharedRef<FTimelineTrack> Item, TArray<TSharedRef<FTimelineTrack>>& OutChildren);
		void HandleExpansionChanged(TSharedRef<FTimelineTrack> InTrack, bool bIsExpanded);

		void UpdateTrackOffsets(const TArray<TSharedRef<FTimelineTrack>>& Tracks, float& InOutOffset);
		void ScrollByDelta(float DeltaInSlateUnits);

	private:

		/** The header row */
		TSharedPtr<SHeaderRow> HeaderRow;

		TWeakPtr<STrackArea> TrackArea;

		TWeakPtr<FTimelineModel> TimelineModel;

		TWeakPtr<FTimelineTracksModel> TracksModel;
	};

	class SOutlinerMultiColumnRow : public SMultiColumnTableRow<TSharedPtr<FTimelineTrack>>
	{
	public:
		SLATE_BEGIN_ARGS(SOutlinerMultiColumnRow) {}
			SLATE_ARGUMENT(TSharedPtr<FTimelineTrack>, Item)
			SLATE_ARGUMENT(TWeakPtr<FTimelineModel>, TimelineModel)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
		{
			Item = InArgs._Item;
			TimelineModel = InArgs._TimelineModel;
			SMultiColumnTableRow<TSharedPtr<FTimelineTrack>>::Construct(FSuperRowType::FArguments(), InOwnerTableView);

			if (Item->FramesObject.IsValid())
			{
				TrackItem =
					SNew(STrack, Item.ToSharedRef())
					[
						SNew(SFramesTrack, Item->FramesObject)
							.Color(Item->Color)
							.ViewRange_Lambda([this]() { return TimelineModel.Pin()->GetTimelineViewRange(); })
							.ViewOffset(Item->ViewOffset)
					];
			}
			else if (Item->FrameRangesObject.IsValid())
			{
				TrackItem =
					SNew(STrack, Item.ToSharedRef())
					[
						SNew(SFrameRangesTrack, Item->FrameRangesObject)
							.Color(Item->Color)
							.ViewRange_Lambda([this]() { return TimelineModel.Pin()->GetTimelineViewRange(); })
							.ViewOffset(Item->ViewOffset)
					];
			}
			else if (Item->FrameAttributeObject.IsValid())
			{
				TrackItem =
					SNew(STrack, Item.ToSharedRef())
					[
						SNew(SFrameAttributeTrack, Item->FrameAttributeObject)
							.Color(Item->Color)
							.ViewRange_Lambda([this]() { return TimelineModel.Pin()->GetTimelineViewRange(); })
							.ViewOffset(Item->ViewOffset)
					];
			}
			else
			{
				TrackItem = SNew(STrack, Item.ToSharedRef());
			}
		}

		virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
		{
			if (Item->bIsRange)
			{
				return
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("Sequencer.Section.BackgroundTint"))
					.BorderBackgroundColor(FAppStyle::GetColor("AnimTimeline.Outliner.ItemColor"))
					[
						SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							.AutoWidth()
							.Padding(4.0f, 1.0f)
							[
								SNew(SExpanderArrow, SharedThis(this))
							]
							+ SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							.HAlign(HAlign_Left)
							.AutoWidth()
							[
								SNew(SBox)
									.HAlign(EHorizontalAlignment::HAlign_Center)
									.VAlign(EVerticalAlignment::VAlign_Center)
									.MaxDesiredWidth(10.0f)
									.MaxDesiredHeight(10.0f)
									.MaxAspectRatio(1.0f)
									.MinAspectRatio(1.0f)
									[
										SNew(SImage)
											.ColorAndOpacity(MakeAttributeLambda([this]() { return (FSlateColor)Item->Color; }))
											.Image(FCoreStyle::Get().GetBrush("GraphEditor.PinIcon"))
									]
							]
							+ SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							.HAlign(HAlign_Left)
							.Padding(4.0f, 1.0f)
							.AutoWidth()
							[
								Item->SequencePath.IsValid() ?
									SNew(SHyperlink).Text(FText::FromString(Item->SequenceName)).OnNavigate_Lambda([this]()
									{
										GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Item->SequencePath);
									}) :
									SNew(SHyperlink).Text(LOCTEXT("NullSequenceName", "null"))
							]
					];
			}
			else
			{
				return
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("Sequencer.Section.BackgroundTint"))
					.BorderBackgroundColor(FAppStyle::GetColor("AnimTimeline.Outliner.ItemColor"))
					[
						SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							.AutoWidth()
							.Padding(4.0f, 1.0f)
							[
								SNew(SExpanderArrow, SharedThis(this))
							]
							+ SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							.HAlign(HAlign_Left)
							.Padding(2.0f, 1.0f)
							.FillWidth(1.0f)
							[
								SNew(STextBlock)
									.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("AnimTimeline.Outliner.Label"))
									.Text_Lambda([this]() { return Item->DisplayName; })
							]
					];
			}
		}

		TSharedPtr<STrack> TrackItem;

	private:

		TWeakPtr<FTimelineModel> TimelineModel;
		TSharedPtr<FTimelineTrack> Item;
	};

	/** Construct this widget */
	void STrack::Construct(const FArguments& InArgs, const TSharedPtr<FTimelineTrack>& InTrack)
	{
		WeakTrack = InTrack;

		ChildSlot
			.HAlign(HAlign_Fill)
			.Padding(0)
			[
				SNew(SOverlay)
					+ SOverlay::Slot()
					[
						InArgs._Content.Widget
					]
			];
	}

	FVector2D STrack::ComputeDesiredSize(float LayoutScale) const
	{
		return FVector2D(100.f, FTimelineTrack::Height);
	}

	int32 STrack::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
	{
		static const FName BorderName("AnimTimeline.Outliner.DefaultBorder");
		static const FName SelectionColorName("SelectionColor");

		TSharedPtr<FTimelineTrack> Track = WeakTrack.Pin();
		if (Track.IsValid())
		{
			// Draw track bottom border
			FSlateDrawElement::MakeLines(
				OutDrawElements,
				LayerId++,
				AllottedGeometry.ToPaintGeometry(),
				TArray<FVector2D>({ FVector2D(0.0f, FTimelineTrack::Height), FVector2D(AllottedGeometry.GetLocalSize().X, FTimelineTrack::Height) }),
				ESlateDrawEffect::None,
				FLinearColor(0.1f, 0.1f, 0.1f, 0.3f)
			);
		}

		return SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId + 1, InWidgetStyle, bParentEnabled);
	}

	/** Construct this widget */
	void STrackArea::Construct(const FArguments& InArgs,
		const TSharedRef<FTimeSliderController>& InTimeSliderController)
	{
		WeakTimeSliderController = InTimeSliderController;
	}

	void STrackArea::SetOutliner(const TSharedPtr<SOutliner>& InOutliner)
	{
		WeakOutliner = InOutliner;
	}

	/** Empty the track area */
	void STrackArea::Empty()
	{
		Children.Empty();
	}

	void STrackArea::AddTrack(const TSharedRef<STrack>& Track)
	{
		Children.AddSlot(FTrackAreaSlot::FSlotArguments(MakeUnique<FTrackAreaSlot>(Track)));
	}

	/** SWidget interface */
	int32 STrackArea::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
	{
		// paint the child widgets
		FArrangedChildren ArrangedChildren(EVisibility::Visible);
		ArrangeChildren(AllottedGeometry, ArrangedChildren);

		const FPaintArgs NewArgs = Args.WithNewParent(this);

		for (int32 ChildIndex = 0; ChildIndex < ArrangedChildren.Num(); ++ChildIndex)
		{
			FArrangedWidget& CurWidget = ArrangedChildren[ChildIndex];
			FSlateRect ChildClipRect = MyCullingRect.IntersectionWith(CurWidget.Geometry.GetLayoutBoundingRect());
			const int32 ThisWidgetLayerId = CurWidget.Widget->Paint(NewArgs, CurWidget.Geometry, ChildClipRect, OutDrawElements, LayerId + 2, InWidgetStyle, bParentEnabled);

			LayerId = FMath::Max(LayerId, ThisWidgetLayerId);
		}

		return LayerId;
	}
		
	void STrackArea::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
	{
		for (int32 Index = 0; Index < Children.Num();)
		{
			if (!StaticCastSharedRef<SWeakWidget>(Children[Index].GetWidget())->ChildWidgetIsValid())
			{
				Children.RemoveAt(Index);
			}
			else
			{
				++Index;
			}
		}
	}

	void STrackArea::OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const
	{
		const float Offset = WeakOutliner.IsValid() ? -FTimelineTrack::Height * WeakOutliner.Pin()->GetScrollOffset() : 0.0f;

		for (int32 ChildIndex = 0; ChildIndex < Children.Num(); ++ChildIndex)
		{
			const FTrackAreaSlot& CurChild = Children[ChildIndex];

			const EVisibility ChildVisibility = CurChild.GetWidget()->GetVisibility();
			if (!ArrangedChildren.Accepts(ChildVisibility))
			{
				continue;
			}

			const FMargin Padding(0, CurChild.GetVerticalOffset() + Offset, 0, 0);

			const AlignmentArrangeResult XResult = AlignChild<Orient_Horizontal>(static_cast<float>(AllottedGeometry.GetLocalSize().X), CurChild, Padding, 1.0f, false);
			const AlignmentArrangeResult YResult = AlignChild<Orient_Vertical>(static_cast<float>(AllottedGeometry.GetLocalSize().Y), CurChild, Padding, 1.0f, false);

			ArrangedChildren.AddWidget(ChildVisibility,
				AllottedGeometry.MakeChild(
					CurChild.GetWidget(),
					FVector2D(XResult.Offset, YResult.Offset),
					FVector2D(XResult.Size, YResult.Size)
				)
			);
		}
	}
		
	FVector2D STrackArea::ComputeDesiredSize(float) const
	{
		FVector2D MaxSize(0.0f, 0.0f);
		for (int32 ChildIndex = 0; ChildIndex < Children.Num(); ++ChildIndex)
		{
			const FTrackAreaSlot& CurChild = Children[ChildIndex];

			const EVisibility ChildVisibilty = CurChild.GetWidget()->GetVisibility();
			if (ChildVisibilty != EVisibility::Collapsed)
			{
				FVector2D ChildDesiredSize = CurChild.GetWidget()->GetDesiredSize();
				MaxSize.X = FMath::Max(MaxSize.X, ChildDesiredSize.X);
				MaxSize.Y = FMath::Max(MaxSize.Y, ChildDesiredSize.Y);
			}
		}

		return MaxSize;
	}

	FChildren* STrackArea::GetChildren()
	{
		return &Children;
	}

	void STrackArea::UpdateHoverStates(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{

	}

	FReply STrackArea::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		TSharedPtr<FTimeSliderController> TimeSliderController = WeakTimeSliderController.Pin();
		if (TimeSliderController.IsValid())
		{
			if (TSharedPtr<SOutliner> Outliner = WeakOutliner.Pin()) { Outliner->ClearSelection(); }
			TimeSliderController->OnMouseButtonDown(*this, MyGeometry, MouseEvent);
		}

		return FReply::Unhandled();
	}

	FReply STrackArea::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		TSharedPtr<FTimeSliderController> TimeSliderController = WeakTimeSliderController.Pin();
		if (TimeSliderController.IsValid())
		{

			return WeakTimeSliderController.Pin()->OnMouseButtonUp(*this, MyGeometry, MouseEvent);
		}

		return FReply::Unhandled();
	}

	FReply STrackArea::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		UpdateHoverStates(MyGeometry, MouseEvent);

		const TSharedPtr<FTimeSliderController> TimeSliderController = WeakTimeSliderController.Pin();
		if (TimeSliderController.IsValid())
		{
			FReply Reply = WeakTimeSliderController.Pin()->OnMouseMove(*this, MyGeometry, MouseEvent);

			// Handle right click scrolling on the track area
			if (Reply.IsEventHandled())
			{
				if (MouseEvent.IsMouseButtonDown(EKeys::RightMouseButton) && HasMouseCapture())
				{
					if (TSharedPtr<SOutliner> Outliner = WeakOutliner.Pin()) { Outliner->ScrollByDelta(static_cast<float>(-MouseEvent.GetCursorDelta().Y)); }
				}
			}

			return Reply;
		}

		return FReply::Unhandled();
	}

	FReply STrackArea::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		TSharedPtr<FTimeSliderController> TimeSliderController = WeakTimeSliderController.Pin();
		if (TimeSliderController.IsValid())
		{
			FReply Reply = WeakTimeSliderController.Pin()->OnMouseWheel(*this, MyGeometry, MouseEvent);
			if (Reply.IsEventHandled())
			{
				return Reply;
			}

			const float ScrollAmount = -MouseEvent.GetWheelDelta() * GetGlobalScrollAmount();
			if (TSharedPtr<SOutliner> Outliner = WeakOutliner.Pin()) { Outliner->ScrollByDelta(ScrollAmount); }

			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

	void STrackArea::OnMouseLeave(const FPointerEvent& MouseEvent)
	{
	}

	FCursorReply STrackArea::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
	{
		if (CursorEvent.IsMouseButtonDown(EKeys::RightMouseButton) && HasMouseCapture())
		{
			return FCursorReply::Cursor(EMouseCursor::GrabHandClosed);
		}
		else
		{
			TSharedPtr<FTimeSliderController> TimeSliderController = WeakTimeSliderController.Pin();
			if (TimeSliderController.IsValid())
			{
				return TimeSliderController->OnCursorQuery(*this, MyGeometry, CursorEvent);
			}
		}

		return FCursorReply::Unhandled();
	}
	
	void SOutliner::Construct(
		const FArguments& InArgs, 
		const TWeakPtr<FTimelineModel>& InTimelineModel,
		const TWeakPtr<FTimelineTracksModel>& InTracksModel,
		const TWeakPtr<STrackArea>& InTrackArea)
	{
		HeaderRow = SNew(SHeaderRow)
			.Visibility(EVisibility::Collapsed);

		HeaderRow->AddColumn(
			SHeaderRow::Column(TEXT("Outliner"))
			.FillWidth(1.0f)
		);

		TimelineModel = InTimelineModel;
		TracksModel = InTracksModel;
		check(TracksModel.IsValid());

		TrackArea = InTrackArea;
		TrackArea.Pin()->Empty();

		STreeView::Construct
		(
			STreeView::FArguments()
			.TreeItemsSource(&TracksModel.Pin()->GetRootTracks())
			.SelectionMode(ESelectionMode::None)
			.OnGenerateRow(this, &SOutliner::HandleGenerateRow)
			.OnGetChildren(this, &SOutliner::HandleGetChildren)
			.OnExpansionChanged(this, &SOutliner::HandleExpansionChanged)
			.HeaderRow(HeaderRow)
			.ExternalScrollbar(InArgs._ExternalScrollbar)
			.AllowOverscroll(EAllowOverscroll::No)
		);

		// Expand all
		for (const TSharedRef<FTimelineTrack>& Track : TracksModel.Pin()->GetRootTracks())
		{
			SetItemExpansion(Track, true);
		}

		// Recompute Offsets
		float Offset = 0.0f;
		UpdateTrackOffsets(TracksModel.Pin()->GetRootTracks(), Offset);
	}

	TSharedRef<ITableRow> SOutliner::HandleGenerateRow(TSharedRef<FTimelineTrack> InTrack, const TSharedRef<STableViewBase>& OwnerTable)
	{
		TSharedRef<SOutlinerMultiColumnRow> Row = SNew(SOutlinerMultiColumnRow, OwnerTable).Item(InTrack).TimelineModel(TimelineModel);
		TrackArea.Pin()->AddTrack(Row->TrackItem.ToSharedRef());
		return Row;
	}

	void SOutliner::HandleGetChildren(TSharedRef<FTimelineTrack> Item, TArray<TSharedRef<FTimelineTrack>>& OutChildren)
	{
		OutChildren.Append(Item->Children);
	}

	void SOutliner::UpdateTrackOffsets(const TArray<TSharedRef<FTimelineTrack>>& Tracks, float& InOutOffset)
	{
		const int32 TrackNum = Tracks.Num();
		for (int32 TrackIdx = 0; TrackIdx < TrackNum; TrackIdx++)
		{
			Tracks[TrackIdx]->Offset = InOutOffset;
			InOutOffset += FTimelineTrack::Height;
			float Offset = InOutOffset;
			UpdateTrackOffsets(Tracks[TrackIdx]->Children, Offset);
			if (Tracks[TrackIdx]->bIsExpanded)
			{
				InOutOffset = Offset;
			}
		}
	}

	void SOutliner::HandleExpansionChanged(TSharedRef<FTimelineTrack> InTrack, bool bIsExpanded)
	{
		InTrack->bIsExpanded = bIsExpanded;

		// Expand any children that are also expanded
		for (const TSharedRef<FTimelineTrack>& Child : InTrack->Children)
		{
			if (Child->bIsExpanded)
			{
				SetItemExpansion(Child, true);
			}
		}

		// Recompute Offsets
		float Offset = 0.0f;
		UpdateTrackOffsets(TracksModel.Pin()->GetRootTracks(), Offset);
	}

	void SOutliner::ScrollByDelta(float DeltaInSlateUnits)
	{
		ScrollBy(GetCachedGeometry(), DeltaInSlateUnits, EAllowOverscroll::No);
	}

	/** Small and simple widget made to contain the custom playback rate setting box. */
	class SCustomPlaybackRateSetting : public SCompoundWidget
	{
	public:
		DECLARE_DELEGATE_OneParam(FOnCustomPlaybackRateChanged, float);

		SLATE_BEGIN_ARGS(SCustomPlaybackRateSetting) {}
			SLATE_ATTRIBUTE(float, CustomPlaybackRate)
			SLATE_EVENT(FOnCustomPlaybackRateChanged, OnCustomPlaybackRateChanged)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs)
		{
			PlaybackRate = InArgs._CustomPlaybackRate;
			OnCustomPlaybackRateChanged = InArgs._OnCustomPlaybackRateChanged;

			ChildSlot
				[
					SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.FillWidth(1.0)
						[
							SNew(STextBlock).Text(LOCTEXT("AnimationCustomPlaybackRateLabel", "Custom"))
						]
						+ SHorizontalBox::Slot()
						.HAlign(HAlign_Right)
						.AutoWidth()
						[
							SNew(SBox)
								.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
								.WidthOverride(100.0f)
								[
									SNew(SSpinBox<float>)
										.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
										.ToolTipText(LOCTEXT("AnimationCustomPlaybackRate", "Set Custom Playback Rate."))
										.MinValue(0.f)
										.MaxSliderValue(10.f)
										.SupportDynamicSliderMaxValue(true)
										.Value(PlaybackRate)
										.OnValueChanged(OnCustomPlaybackRateChanged)
								]
						]
				];
		}

	protected:
		TAttribute<float> PlaybackRate = 1.0f;
		FOnCustomPlaybackRateChanged OnCustomPlaybackRateChanged;
	};

	/**
	 * Simple transport controls class used by the timeline. This is a slight customization on the EditorWidgets TransportControls since we don't 
	 * include all the buttons and have some custom code to handle playback rates and the SpinBox used for the frame number. 
	 * 
	 * The reason we need custom controls for the playback rate is that the existing playback rate controls are coupled with the global UI settings 
	 * for all Persona preview viewports and cannot be used in other custom viewports. This is therefore designed to emulate the same UI but to 
	 * provide something stand-alone (using FTimelineModel).
	 */
	class STimelineTransportControls : public SCompoundWidget
	{
	public:

		SLATE_BEGIN_ARGS(STimelineTransportControls) {}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TWeakPtr<FTimelineModel> InModel)
		{
			Model = InModel;

			// The names of the possible playback rate settings

			static TStaticArray<FText, 8> PlaybackRateStrings
			{
				LOCTEXT("PlaybackRateOneTenth", "x0.1"),
				LOCTEXT("PlaybackRateOneQuarter", "x0.25"),
				LOCTEXT("PlaybackRateHalf", "x0.5"),
				LOCTEXT("PlaybackRateThreeQuarters", "x0.75"),
				LOCTEXT("PlaybackRateNormal", "x1.0"),
				LOCTEXT("PlaybackRateDouble", "x2.0"),
				LOCTEXT("PlaybackRateFiveTimes", "x5.0"),
				LOCTEXT("PlaybackRateTenTimes", "x10.0"),
			};

			// The values of the possible playback rate settings 

			static TStaticArray<float, 8> PlaybackRateValues = { 0.1f, 0.25f, 0.5f, 0.75f, 1.0f, 2.0f, 5.0f, 10.0f };

			// Start building the playback rate menu

			FMenuBuilder PlaybackMenuBuilder(true, nullptr, TSharedPtr<FExtender>(), false, &FCoreStyle::Get());
			{
				PlaybackMenuBuilder.BeginSection(TEXT("PlaybackRateControlSection"), LOCTEXT("PlaybackSectionTitle", "Playback Rate"));

				const int32 PlaybackRateNum = PlaybackRateStrings.Num();

				for (int32 PlaybackRateIdx = 0; PlaybackRateIdx < PlaybackRateNum; PlaybackRateIdx++)
				{
					PlaybackMenuBuilder.AddMenuEntry(
						PlaybackRateStrings[PlaybackRateIdx],
						FText(),
						FSlateIcon(),
						FUIAction(FExecuteAction::CreateLambda([PlaybackRateIdx, InModel]() {

							if (InModel.IsValid())
							{
								InModel.Pin()->SetTimelinePlaybackRate(PlaybackRateValues[PlaybackRateIdx]);
								InModel.Pin()->SetTimelineUseCustomPlaybackRate(false);
							}
							}),
							FCanExecuteAction(),
							FIsActionChecked::CreateLambda([PlaybackRateIdx, InModel]() { return InModel.IsValid() ? (!InModel.Pin()->GetTimelineUseCustomPlaybackRate() && InModel.Pin()->GetTimelinePlaybackRate() == PlaybackRateValues[PlaybackRateIdx]) : false; })),
						NAME_None,
						EUserInterfaceActionType::RadioButton);
				}

				TSharedPtr<SWidget> AnimPlaybackRateWidget =
					SNew(SCustomPlaybackRateSetting)
					.CustomPlaybackRate_Lambda(
						[InModel]()
						{
							return InModel.IsValid() ? InModel.Pin()->GetTimelineCustomPlaybackRate() : 1.0f;
						}
					)
					.OnCustomPlaybackRateChanged_Lambda(
						[InModel](float InCustomPlaybackRate)
						{
							if (InModel.IsValid()) { InModel.Pin()->SetTimelineCustomPlaybackRate(InCustomPlaybackRate); }
						}
					);

				PlaybackMenuBuilder.AddMenuEntry(
					FUIAction(FExecuteAction::CreateLambda([InModel]() {

						if (InModel.IsValid())
						{
							InModel.Pin()->SetTimelineUseCustomPlaybackRate(true);
						}

						}),
						FCanExecuteAction(),
						FIsActionChecked::CreateLambda([InModel]() { return InModel.IsValid() ? InModel.Pin()->GetTimelineUseCustomPlaybackRate() : false; })),
					AnimPlaybackRateWidget.ToSharedRef(),
					NAME_None,
					FText(),
					EUserInterfaceActionType::RadioButton);

				PlaybackMenuBuilder.EndSection();
			}

			// Combo button used to pop-up menu

			const TAttribute<FText> ComboLabel = TAttribute<FText>::CreateLambda([InModel]()
				{
					const float PlaybackRate = InModel.IsValid() ? InModel.Pin()->GetTimelinePlaybackRate() : 1.0f;

					const int32 PlaybackRateNum = PlaybackRateStrings.Num();

					for (int32 PlaybackRateIdx = 0; PlaybackRateIdx < PlaybackRateNum - 1; PlaybackRateIdx++)
					{
						if (PlaybackRate == PlaybackRateValues[PlaybackRateIdx])
						{
							return PlaybackRateStrings[PlaybackRateIdx];
						}
					}

					const FNumberFormattingOptions FormatOptions = FNumberFormattingOptions().SetMinimumFractionalDigits(2).SetMaximumFractionalDigits(2);
					return FText::Format(LOCTEXT("AnimDatabaseDatabasePlaybackMenuLabelCustom", "x{0}"), FText::AsNumber(PlaybackRate, &FormatOptions));
				}
			);

			// Transport controls arguments

			FEditorWidgetsModule& EditorWidgetsModule = FModuleManager::LoadModuleChecked<FEditorWidgetsModule>("EditorWidgets");

			FTransportControlArgs TransportControlArgs;
			TransportControlArgs.OnForwardPlay = FOnClicked::CreateSP(this, &STimelineTransportControls::OnClick_Forward);
			TransportControlArgs.OnForwardStep = FOnClicked::CreateSP(this, &STimelineTransportControls::OnClick_Forward_Step);
			TransportControlArgs.OnBackwardStep = FOnClicked::CreateSP(this, &STimelineTransportControls::OnClick_Backward_Step);
			TransportControlArgs.OnForwardEnd = FOnClicked::CreateSP(this, &STimelineTransportControls::OnClick_Forward_End);
			TransportControlArgs.OnBackwardEnd = FOnClicked::CreateSP(this, &STimelineTransportControls::OnClick_Backward_End);
			TransportControlArgs.OnGetPlaybackMode = FOnGetPlaybackMode::CreateSP(this, &STimelineTransportControls::GetPlaybackMode);
			TransportControlArgs.OnGetLooping = FOnGetLooping::CreateSP(this, &STimelineTransportControls::GetLooping);
			TransportControlArgs.OnToggleLooping = FOnClicked::CreateSP(this, &STimelineTransportControls::OnClick_ToggleLooping);

			ChildSlot
				[
					SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.FillWidth(1.0)
						[
							EditorWidgetsModule.CreateTransportControl(TransportControlArgs)
						]
						+ SHorizontalBox::Slot()
						.HAlign(HAlign_Right)
						.VAlign(VAlign_Center)
						.AutoWidth()
						.Padding(2.0f, 0.0f, 2.0f, 0.0f)
						.MinWidth(100.0f)
						[
							SNew(SComboButton)
								.ButtonContent()
								[
									SNew(STextBlock).Text(ComboLabel)
								]
								.MenuContent()
								[
									PlaybackMenuBuilder.MakeWidget()
								]
						]
				];
		}

		void SetModel(TWeakPtr<FTimelineModel> InModel) { Model = InModel; }

	private:

		FReply OnClick_Forward_Step() const
		{
			if (Model.IsValid())
			{
				Model.Pin()->OnTransportStepForward();
				return FReply::Handled();
			}
			return FReply::Unhandled();
		}

		FReply OnClick_Forward_End() const
		{
			if (Model.IsValid())
			{
				Model.Pin()->OnTransportToEndFrame();
				return FReply::Handled();
			}
			return FReply::Unhandled();
		}

		FReply OnClick_Backward_Step() const
		{
			if (Model.IsValid())
			{
				Model.Pin()->OnTransportStepBackward();
				return FReply::Handled();
			}
			return FReply::Unhandled();
		}

		FReply OnClick_Backward_End() const
		{
			if (Model.IsValid())
			{
				Model.Pin()->OnTransportToStartFrame();
				return FReply::Handled();
			}
			return FReply::Unhandled();
		}

		FReply OnClick_Forward() const
		{
			if (Model.IsValid())
			{
				Model.Pin()->OnTransportPlayPressed();
				return FReply::Handled();
			}
			return FReply::Unhandled();
		}

		FReply OnClick_ToggleLooping() const
		{
			if (Model.IsValid())
			{
				Model.Pin()->SetLooping(!Model.Pin()->GetLooping());
				return FReply::Handled();
			}
			return FReply::Unhandled();
		}

		EPlaybackMode::Type GetPlaybackMode() const
		{
			if (Model.IsValid())
			{
				return Model.Pin()->GetTransportPlaybackMode();
			}
			return EPlaybackMode::Stopped;
		}

		bool GetLooping() const
		{
			if (Model.IsValid())
			{
				return Model.Pin()->GetLooping();
			}
			return false;
		}

	private:

		TWeakPtr<FTimelineModel> Model;
	};

	/** Utility struct for converting between scrub range space and local/absolute screen space. */
	struct FTimeSliderController::FScrubRangeToScreen
	{
		double ViewStart;
		float PixelsPerInput;

		FScrubRangeToScreen(const TRange<double>& InViewInput, const FVector2D& InWidgetSize)
		{
			const float ViewInputRange = InViewInput.Size<double>();
			ViewStart = InViewInput.GetLowerBoundValue();
			PixelsPerInput = ViewInputRange > 0 ? (InWidgetSize.X / ViewInputRange) : 0;
		}

		/** Local Widget Space -> Curve Input domain. */
		double LocalXToInput(float ScreenX) const
		{
			return PixelsPerInput > 0 ? (ScreenX / PixelsPerInput) + ViewStart : ViewStart;
		}

		/** Curve Input domain -> local Widget Space. */
		float InputToLocalX(double Input) const
		{
			return (Input - ViewStart) * PixelsPerInput;
		}
	};

	FTimeSliderController::FTimeSliderController(const FTimeSliderArgs& InArgs, TWeakPtr<FTimelineModel> InWeakModel, TWeakPtr<STimeline> InWeakTimeline)
		: WeakModel(InWeakModel)
		, WeakTimeline(InWeakTimeline)
		, TimeSliderArgs(InArgs)
		, DistanceDragged(0.0f)
		, MouseDragType(DRAG_NONE)
		, bPanning(false)
		, DraggedTimeIndex(INDEX_NONE)
	{
		ScrubFillBrush = FAppStyle::GetBrush(TEXT("Sequencer.Timeline.ScrubFill"));
		ScrubHandleUpBrush = FAppStyle::GetBrush(TEXT("Sequencer.Timeline.VanillaScrubHandleUp"));
		ScrubHandleDownBrush = FAppStyle::GetBrush(TEXT("Sequencer.Timeline.VanillaScrubHandleDown"));
		EditableTimeBrush = FAppStyle::GetBrush(TEXT("AnimTimeline.SectionMarker"));
	}

	void FTimeSliderController::SetModel(TWeakPtr<FTimelineModel> InModel)
	{
		WeakModel = InModel;
		if (WeakTimeline.IsValid())
		{
			TimeSliderArgs.ScrubPosition = MakeAttributeLambda([this]() { return WeakModel.IsValid() ? WeakModel.Pin()->GetTimelineFrameTime() : FFrameTime(0); });
			TimeSliderArgs.PlaybackRange = MakeAttributeLambda([this]() { return WeakModel.IsValid() ? WeakModel.Pin()->GetTimelinePlaybackRangeFrames() : TRange<FFrameNumber>(0, 0); });
			TimeSliderArgs.ClampRange = MakeAttributeLambda([this]() { return WeakModel.IsValid() ? WeakModel.Pin()->GetTimelineWorkingRangeTimes() : FAnimatedRange(0.0, 0.0); });
		}
	}

	FFrameTime FTimeSliderController::ComputeFrameTimeFromMouse(const FGeometry& Geometry, FVector2D ScreenSpacePosition, FScrubRangeToScreen RangeToScreen, bool CheckSnapping) const
	{
		const FVector2D CursorPos = Geometry.AbsoluteToLocal(ScreenSpacePosition);
		const double MouseValue = RangeToScreen.LocalXToInput(CursorPos.X);
		return MouseValue * GetTickResolution();
	}

	FTimeSliderController::FScrubPixelRange FTimeSliderController::GetHitTestScrubberPixelRange(FFrameTime ScrubTime, const FScrubRangeToScreen& RangeToScreen) const
	{
		static const float DragToleranceSlateUnits = 2.0f, MouseTolerance = 2.0f;
		return GetScrubberPixelRange(ScrubTime, GetTickResolution(), GetDisplayRate(), RangeToScreen, DragToleranceSlateUnits + MouseTolerance);
	}

	FTimeSliderController::FScrubPixelRange FTimeSliderController::GetScrubberPixelRange(FFrameTime ScrubTime, const FScrubRangeToScreen& RangeToScreen) const
	{
		return GetScrubberPixelRange(ScrubTime, GetTickResolution(), GetDisplayRate(), RangeToScreen);
	}

	FTimeSliderController::FScrubPixelRange FTimeSliderController::GetScrubberPixelRange(FFrameTime ScrubTime, FFrameRate Resolution, FFrameRate PlayRate, const FScrubRangeToScreen& RangeToScreen, float DilationPixels) const
	{
		const FFrameNumber Frame = ScrubTime.FloorToFrame();
		float StartPixel = RangeToScreen.InputToLocalX(Frame / Resolution);
		float EndPixel = RangeToScreen.InputToLocalX((Frame + 1) / Resolution);

		const float RoundedStartPixel = FMath::RoundToInt(StartPixel);
		EndPixel -= (StartPixel - RoundedStartPixel);

		StartPixel = RoundedStartPixel;
		EndPixel = FMath::Max(EndPixel, StartPixel + 1);

		const float MinScrubSize = 14.0f;
		FScrubPixelRange Range;
		Range.bClamped = EndPixel - StartPixel < MinScrubSize;
		Range.Range = TRange<float>(StartPixel, EndPixel);
		if (Range.bClamped)
		{
			Range.HandleRange = TRange<float>(
				(StartPixel + EndPixel - MinScrubSize) * 0.5f,
				(StartPixel + EndPixel + MinScrubSize) * 0.5f);
		}
		else
		{
			Range.HandleRange = Range.Range;
		}

		return Range;
	}

	struct FTimeSliderController::FDrawTickArgs
	{
		/** Geometry of the area. */
		FGeometry AllottedGeometry;
		/** Culling rect of the area. */
		FSlateRect CullingRect;
		/** Color of each tick. */
		FLinearColor TickColor;
		/** Offset in Y where to start the tick. */
		float TickOffset;
		/** Height in of major ticks. */
		float MajorTickHeight;
		/** Start layer for elements. */
		int32 StartLayer;
		/** Draw effects to apply. */
		ESlateDrawEffect DrawEffects;
		/** Whether or not to only draw major ticks. */
		bool bOnlyDrawMajorTicks;
		/** Whether or not to mirror labels. */
		bool bMirrorLabels;
	};

	void FTimeSliderController::DrawTicks(FSlateWindowElementList& OutDrawElements, const TRange<double>& ViewRange, const FScrubRangeToScreen& RangeToScreen, FDrawTickArgs& InArgs) const
	{
		TSharedPtr<STimeline> Timeline = WeakTimeline.Pin();
		if (!Timeline.IsValid())
		{
			return;
		}

		if (!FMath::IsFinite(ViewRange.GetLowerBoundValue()) || !FMath::IsFinite(ViewRange.GetUpperBoundValue()))
		{
			return;
		}

		const FFrameRate     FrameResolution = GetTickResolution();
		const FPaintGeometry PaintGeometry = InArgs.AllottedGeometry.ToPaintGeometry();
		const FSlateFontInfo SmallLayoutFont = FCoreStyle::GetDefaultFontStyle("Regular", 8);

		double MajorGridStep = 0.0;
		int32 MinorDivisions = 0;
		if (!Timeline->GetGridMetrics(InArgs.AllottedGeometry.Size.X, MajorGridStep, MinorDivisions))
		{
			return;
		}

		if (InArgs.bOnlyDrawMajorTicks)
		{
			MinorDivisions = 0;
		}

		TArray<FVector2D> LinePoints;
		LinePoints.SetNumUninitialized(2);

		const bool bAntiAliasLines = false;
		const double FirstMajorLine = FMath::FloorToDouble(ViewRange.GetLowerBoundValue() / MajorGridStep) * MajorGridStep;
		const double LastMajorLine = FMath::CeilToDouble(ViewRange.GetUpperBoundValue() / MajorGridStep) * MajorGridStep;

		FString FrameString;
		for (double CurrentMajorLine = FirstMajorLine; CurrentMajorLine < LastMajorLine; CurrentMajorLine += MajorGridStep)
		{
			const float MajorLinePx = RangeToScreen.InputToLocalX(CurrentMajorLine);

			LinePoints[0] = FVector2D(MajorLinePx, InArgs.TickOffset);
			LinePoints[1] = FVector2D(MajorLinePx, InArgs.TickOffset + InArgs.MajorTickHeight);

			// Draw each tick mark.
			FSlateDrawElement::MakeLines(
				OutDrawElements,
				InArgs.StartLayer,
				PaintGeometry,
				LinePoints,
				InArgs.DrawEffects,
				InArgs.TickColor,
				bAntiAliasLines
			);

			if (!InArgs.bOnlyDrawMajorTicks)
			{
				FrameString = TimeSliderArgs.NumericTypeInterface->ToString((CurrentMajorLine * FrameResolution).RoundToFrame().Value);

				// Space the text between the tick mark but slightly above.
				const FVector2D TextOffset(MajorLinePx + 5.0f, InArgs.bMirrorLabels ? 3.0f : FMath::Abs(InArgs.AllottedGeometry.Size.Y - (InArgs.MajorTickHeight + 3.0f)));
				FSlateDrawElement::MakeText(
					OutDrawElements,
					InArgs.StartLayer + 1,
					InArgs.AllottedGeometry.ToPaintGeometry(InArgs.AllottedGeometry.Size, FSlateLayoutTransform(TextOffset)),
					FrameString,
					SmallLayoutFont,
					InArgs.DrawEffects,
					InArgs.TickColor * 0.65f
				);
			}

			for (int32 Step = 1; Step < MinorDivisions; ++Step)
			{
				// Compute the size of each tick mark.  If we are half way between to visible values display a slightly larger tick mark.
				const float MinorTickHeight = ((MinorDivisions % 2 == 0) && (Step % (MinorDivisions / 2)) == 0) ? 6.0f : 2.0f;
				const float MinorLinePx = RangeToScreen.InputToLocalX(CurrentMajorLine + Step * MajorGridStep / MinorDivisions);

				LinePoints[0] = FVector2D(MinorLinePx, InArgs.bMirrorLabels ? 0.0f : FMath::Abs(InArgs.AllottedGeometry.Size.Y - MinorTickHeight));
				LinePoints[1] = FVector2D(MinorLinePx, LinePoints[0].Y + MinorTickHeight);

				// Draw each sub mark.
				FSlateDrawElement::MakeLines(
					OutDrawElements,
					InArgs.StartLayer,
					PaintGeometry,
					LinePoints,
					InArgs.DrawEffects,
					InArgs.TickColor,
					bAntiAliasLines);
			}
		}
	}

	int32 FTimeSliderController::OnPaintTimeSlider(bool bMirrorLabels, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
	{
		const bool bEnabled = bParentEnabled;
		const ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;
		const TRange<double> LocalViewRange = TimeSliderArgs.ViewRange.Get();
		const TRange<FFrameNumber> LocalPlaybackRange = TimeSliderArgs.PlaybackRange.Get();
		const float LocalViewRangeMin = LocalViewRange.GetLowerBoundValue();
		const float LocalViewRangeMax = LocalViewRange.GetUpperBoundValue();
		const float LocalSequenceLength = LocalViewRangeMax - LocalViewRangeMin;

		if (LocalSequenceLength > 0)
		{
			FScrubRangeToScreen RangeToScreen(LocalViewRange, AllottedGeometry.Size);

			// Draw the ticks.
			const float MajorTickHeight = 9.0f;
			FDrawTickArgs Args;
			{
				Args.AllottedGeometry = AllottedGeometry;
				Args.bMirrorLabels = bMirrorLabels;
				Args.bOnlyDrawMajorTicks = false;
				Args.TickColor = FLinearColor::White;
				Args.CullingRect = MyCullingRect;
				Args.DrawEffects = DrawEffects;
				Args.StartLayer = LayerId;
				Args.TickOffset = bMirrorLabels ? 0.0f : FMath::Abs(AllottedGeometry.Size.Y - MajorTickHeight);
				Args.MajorTickHeight = MajorTickHeight;
			}
			DrawTicks(OutDrawElements, LocalViewRange, RangeToScreen, Args);

			// Draw playback & selection range.
			FPaintPlaybackRangeArgs PlaybackRangeArgs(
				bMirrorLabels ? FAppStyle::GetBrush("Sequencer.Timeline.PlayRange_Bottom_L") : FAppStyle::GetBrush("Sequencer.Timeline.PlayRange_Top_L"),
				bMirrorLabels ? FAppStyle::GetBrush("Sequencer.Timeline.PlayRange_Bottom_R") : FAppStyle::GetBrush("Sequencer.Timeline.PlayRange_Top_R"),
				6.0f);

			LayerId = DrawPlaybackRange(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, RangeToScreen, PlaybackRangeArgs);

			PlaybackRangeArgs.SolidFillOpacity = 0.05f;
			LayerId = DrawSelectionRange(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, RangeToScreen, PlaybackRangeArgs);

			// Draw the scrub handle.
			const float HandleStart = RangeToScreen.InputToLocalX(TimeSliderArgs.ScrubPosition.Get().AsDecimal() / GetTickResolution().AsDecimal()) - 7.0f;
			const float HandleEnd = HandleStart + 13.0f;

			const int32 ArrowLayer = LayerId + 2;
			FPaintGeometry MyGeometry = AllottedGeometry.ToPaintGeometry(FVector2f(HandleEnd - HandleStart, AllottedGeometry.Size.Y), FSlateLayoutTransform(FVector2f(HandleStart, 0)));
			FLinearColor ScrubColor = InWidgetStyle.GetColorAndOpacityTint();
			{
				ScrubColor.A = ScrubColor.A * 0.75f;
				ScrubColor.B *= 0.1f;
				ScrubColor.G *= 0.2f;
			}

			const FSlateBrush* Brush = (bMirrorLabels ? ScrubHandleUpBrush : ScrubHandleDownBrush);

			FSlateDrawElement::MakeBox(
				OutDrawElements,
				ArrowLayer,
				MyGeometry,
				Brush,
				DrawEffects,
				ScrubColor);

			{
				// Draw the current time next to the scrub handle.
				FString FrameString = TimeSliderArgs.NumericTypeInterface->ToString(TimeSliderArgs.ScrubPosition.Get().AsDecimal());

				if (GetDefault<UPersonaOptions>()->bTimelineDisplayPercentage)
				{
					const double FrameTime = TimeSliderArgs.ScrubPosition.Get().AsDecimal();
					const int32 FrameRangeMin = LocalPlaybackRange.GetLowerBoundValue().Value;
					const int32 FrameRangeMax = LocalPlaybackRange.GetUpperBoundValue().Value;
					const double Percentage = FMath::Clamp((FrameTime - FrameRangeMin) / (FrameRangeMax - FrameRangeMin), 0.0, 1.0);
					FNumberFormattingOptions Options;
					Options.MaximumFractionalDigits = 2;
					FrameString += TEXT(" (") + FText::AsPercent(Percentage, &Options).ToString() + TEXT(")");
				}

				const FSlateFontInfo SmallLayoutFont = FCoreStyle::GetDefaultFontStyle("Regular", 10);
				const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
				const FVector2D TextSize = FontMeasureService->Measure(FrameString, SmallLayoutFont);

				// Flip the text position if getting near the end of the view range.
				static const float TextOffsetPx = 2.0f;
				const bool bDrawLeft = (AllottedGeometry.Size.X - HandleEnd) < (TextSize.X + 14.0f) - TextOffsetPx;
				const float TextPosition = bDrawLeft ? HandleStart - TextSize.X - TextOffsetPx : HandleEnd + TextOffsetPx;
				const FVector2D TextOffset(TextPosition, Args.bMirrorLabels ? TextSize.Y - 6.f : Args.AllottedGeometry.Size.Y - (Args.MajorTickHeight + TextSize.Y));

				FSlateDrawElement::MakeText(
					OutDrawElements,
					Args.StartLayer + 1,
					Args.AllottedGeometry.ToPaintGeometry(TextSize, FSlateLayoutTransform(TextOffset)),
					FrameString,
					SmallLayoutFont,
					Args.DrawEffects,
					Args.TickColor);
			}

			return ArrowLayer;
		}

		return LayerId;
	}

	int32 FTimeSliderController::DrawSelectionRange(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FScrubRangeToScreen& RangeToScreen, const FPaintPlaybackRangeArgs& Args) const
	{
		TRange<double> SelectionRange = TimeSliderArgs.SelectionRange.Get() / GetTickResolution();

		if (!SelectionRange.IsEmpty() && SelectionRange.HasLowerBound() && SelectionRange.HasUpperBound())
		{
			const float SelectionRangeL = RangeToScreen.InputToLocalX(SelectionRange.GetLowerBoundValue()) - 1;
			const float SelectionRangeR = RangeToScreen.InputToLocalX(SelectionRange.GetUpperBoundValue()) + 1;
			const auto DrawColor = FAppStyle::GetSlateColor("SelectionColor").GetColor(FWidgetStyle());

			if (Args.SolidFillOpacity > 0.f)
			{
				FSlateDrawElement::MakeBox(
					OutDrawElements,
					LayerId + 1,
					AllottedGeometry.ToPaintGeometry(FVector2f(SelectionRangeR - SelectionRangeL, AllottedGeometry.Size.Y), FSlateLayoutTransform(FVector2f(SelectionRangeL, 0.f))),
					FAppStyle::GetBrush("WhiteBrush"),
					ESlateDrawEffect::None,
					DrawColor.CopyWithNewOpacity(Args.SolidFillOpacity));
			}

			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId + 1,
				AllottedGeometry.ToPaintGeometry(FVector2f(Args.BrushWidth, AllottedGeometry.Size.Y), FSlateLayoutTransform(FVector2f(SelectionRangeL, 0.f))),
				Args.StartBrush,
				ESlateDrawEffect::None,
				DrawColor);

			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId + 1,
				AllottedGeometry.ToPaintGeometry(FVector2f(Args.BrushWidth, AllottedGeometry.Size.Y), FSlateLayoutTransform(FVector2f(SelectionRangeR - Args.BrushWidth, 0.f))),
				Args.EndBrush,
				ESlateDrawEffect::None,
				DrawColor);
		}

		return LayerId + 1;
	}

	int32 FTimeSliderController::DrawPlaybackRange(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FScrubRangeToScreen& RangeToScreen, const FPaintPlaybackRangeArgs& Args) const
	{
		if (!TimeSliderArgs.PlaybackRange.IsSet())
		{
			return LayerId;
		}

		const TRange<FFrameNumber> PlaybackRange = TimeSliderArgs.PlaybackRange.Get();
		const FFrameRate TickResolution = GetTickResolution();
		const float PlaybackRangeL = RangeToScreen.InputToLocalX(PlaybackRange.GetLowerBoundValue() / TickResolution);
		const float PlaybackRangeR = RangeToScreen.InputToLocalX(PlaybackRange.GetUpperBoundValue() / TickResolution) - 1;

		const uint8 OpacityBlend = TimeSliderArgs.SubSequenceRange.Get().IsSet() ? 128 : 255;
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId + 1,
			AllottedGeometry.ToPaintGeometry(FVector2f(Args.BrushWidth, AllottedGeometry.Size.Y), FSlateLayoutTransform(FVector2f(PlaybackRangeL, 0.0f))),
			Args.StartBrush,
			ESlateDrawEffect::None,
			FColor(32, 128, 32, OpacityBlend)	// 120, 75, 50 (HSV)
		);

		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId + 1,
			AllottedGeometry.ToPaintGeometry(FVector2f(Args.BrushWidth, AllottedGeometry.Size.Y), FSlateLayoutTransform(FVector2f(PlaybackRangeR - Args.BrushWidth, 0.0f))),
			Args.EndBrush,
			ESlateDrawEffect::None,
			FColor(128, 32, 32, OpacityBlend)	// 0, 75, 50 (HSV)
		);

		// Black tint for excluded regions
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId + 1,
			AllottedGeometry.ToPaintGeometry(FVector2f(PlaybackRangeL, AllottedGeometry.Size.Y), FSlateLayoutTransform(FVector2f(0.f, 0.f))),
			FAppStyle::GetBrush("WhiteBrush"),
			ESlateDrawEffect::None,
			FLinearColor::Black.CopyWithNewOpacity(0.3f * OpacityBlend / 255.0f)
		);

		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId + 1,
			AllottedGeometry.ToPaintGeometry(FVector2f(AllottedGeometry.Size.X - PlaybackRangeR, AllottedGeometry.Size.Y), FSlateLayoutTransform(FVector2f(PlaybackRangeR, 0.0f))),
			FAppStyle::GetBrush("WhiteBrush"),
			ESlateDrawEffect::None,
			FLinearColor::Black.CopyWithNewOpacity(0.3f * OpacityBlend / 255.0f)
		);

		return LayerId + 1;
	}

	FReply FTimeSliderController::OnMouseButtonDown(SWidget& WidgetOwner, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		DistanceDragged = 0;
		MouseDownPosition = MouseEvent.GetScreenSpacePosition();
		return FReply::Unhandled();
	}

	FReply FTimeSliderController::OnMouseButtonUp(SWidget& WidgetOwner, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		const bool bHandleLeftMouseButton = (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton) && WidgetOwner.HasMouseCapture();
		const bool bHandleRightMouseButton = (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton) && WidgetOwner.HasMouseCapture() && TimeSliderArgs.AllowZoom;

		const FScrubRangeToScreen RangeToScreen = FScrubRangeToScreen(TimeSliderArgs.ViewRange.Get(), MyGeometry.Size);
		const FFrameTime MouseTime = ComputeFrameTimeFromMouse(MyGeometry, MouseEvent.GetScreenSpacePosition(), RangeToScreen);

		if (bHandleRightMouseButton)
		{
			if (!bPanning && DistanceDragged == 0.0f)
			{
				return WeakTimeline.Pin()->OnMouseButtonUp(MyGeometry, MouseEvent).ReleaseMouseCapture();
			}

			bPanning = false;
			DistanceDragged = 0.0f;

			return FReply::Handled().ReleaseMouseCapture();
		}
		else if (bHandleLeftMouseButton)
		{
			if (MouseDragType == DRAG_PLAYBACK_START)
			{
				TimeSliderArgs.OnPlaybackRangeEndDrag.ExecuteIfBound();
			}
			else if (MouseDragType == DRAG_PLAYBACK_END)
			{
				TimeSliderArgs.OnPlaybackRangeEndDrag.ExecuteIfBound();
			}
			else if (MouseDragType == DRAG_SELECTION_START)
			{
				TimeSliderArgs.OnSelectionRangeEndDrag.ExecuteIfBound();
			}
			else if (MouseDragType == DRAG_SELECTION_END)
			{
				TimeSliderArgs.OnSelectionRangeEndDrag.ExecuteIfBound();
			}
			else
			{
				TimeSliderArgs.OnEndScrubberMovement.ExecuteIfBound();

				CommitScrubPosition(MouseTime, /*bIsScrubbing=*/false);
			}

			MouseDragType = DRAG_NONE;
			DistanceDragged = 0.0f;
			return FReply::Handled().ReleaseMouseCapture();
		}

		return FReply::Unhandled();
	}

	FReply FTimeSliderController::OnMouseMove(SWidget& WidgetOwner, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		const bool bHandleLeftMouseButton = MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton);
		const bool bHandleRightMouseButton = MouseEvent.IsMouseButtonDown(EKeys::RightMouseButton) && TimeSliderArgs.AllowZoom;

		if (bHandleRightMouseButton)
		{
			if (!bPanning)
			{
				DistanceDragged += FMath::Abs(MouseEvent.GetCursorDelta().X);
				if (DistanceDragged > FSlateApplication::Get().GetDragTriggerDistance())
				{
					bPanning = true;
				}
			}
			else
			{
				const TRange<double> LocalViewRange = TimeSliderArgs.ViewRange.Get();
				const double LocalViewRangeMin = LocalViewRange.GetLowerBoundValue();
				const double LocalViewRangeMax = LocalViewRange.GetUpperBoundValue();

				const FScrubRangeToScreen ScaleInfo(LocalViewRange, MyGeometry.Size);
				const FVector2D ScreenDelta = MouseEvent.GetCursorDelta();
				const double InputDeltaX = ScreenDelta.X / ScaleInfo.PixelsPerInput;
				double NewViewOutputMin = LocalViewRangeMin - InputDeltaX;
				double NewViewOutputMax = LocalViewRangeMax - InputDeltaX;

				ClampViewRange(NewViewOutputMin, NewViewOutputMax);
				SetViewRange(NewViewOutputMin, NewViewOutputMax, EViewRangeInterpolation::Immediate);
			}
		}
		else if (bHandleLeftMouseButton)
		{
			const TRange<double> LocalViewRange = TimeSliderArgs.ViewRange.Get();
			const FScrubRangeToScreen RangeToScreen(LocalViewRange, MyGeometry.Size);
			DistanceDragged += FMath::Abs(MouseEvent.GetCursorDelta().X);

			if (MouseDragType == DRAG_NONE)
			{
				if (DistanceDragged > FSlateApplication::Get().GetDragTriggerDistance())
				{
					const FFrameTime MouseDownFree = ComputeFrameTimeFromMouse(MyGeometry, MouseDownPosition, RangeToScreen, false);

					const FFrameRate FrameResolution = GetTickResolution();
					const bool       bLockedPlayRange = TimeSliderArgs.IsPlaybackRangeLocked.Get();
					const float      MouseDownPixel = RangeToScreen.InputToLocalX(MouseDownFree / FrameResolution);
					const bool       bHitScrubber = GetHitTestScrubberPixelRange(TimeSliderArgs.ScrubPosition.Get(), RangeToScreen).HandleRange.Contains(MouseDownPixel);

					TRange<double>   SelectionRange = TimeSliderArgs.SelectionRange.Get() / FrameResolution;
					TRange<double>   PlaybackRange = TimeSliderArgs.PlaybackRange.Get() / FrameResolution;

					// Disable selection range test if it's empty so that the playback range scrubbing gets priority.
					if (!SelectionRange.IsEmpty() && !bHitScrubber && HitTestRangeEnd(RangeToScreen, SelectionRange, MouseDownPixel))
					{
						// selection range end scrubber.
						MouseDragType = DRAG_SELECTION_END;
						TimeSliderArgs.OnSelectionRangeBeginDrag.ExecuteIfBound();
					}
					else if (!SelectionRange.IsEmpty() && !bHitScrubber && HitTestRangeStart(RangeToScreen, SelectionRange, MouseDownPixel))
					{
						// selection range start scrubber.
						MouseDragType = DRAG_SELECTION_START;
						TimeSliderArgs.OnSelectionRangeBeginDrag.ExecuteIfBound();
					}
					else if (!bLockedPlayRange && !bHitScrubber && HitTestRangeEnd(RangeToScreen, PlaybackRange, MouseDownPixel))
					{
						// playback range end scrubber.
						MouseDragType = DRAG_PLAYBACK_END;
						TimeSliderArgs.OnPlaybackRangeBeginDrag.ExecuteIfBound();
					}
					else if (!bLockedPlayRange && !bHitScrubber && HitTestRangeStart(RangeToScreen, PlaybackRange, MouseDownPixel))
					{
						// playback range start scrubber.
						MouseDragType = DRAG_PLAYBACK_START;
						TimeSliderArgs.OnPlaybackRangeBeginDrag.ExecuteIfBound();
					}
					else
					{
						MouseDragType = DRAG_SCRUBBING_TIME;
						TimeSliderArgs.OnBeginScrubberMovement.ExecuteIfBound();
					}
				}
			}
			else
			{
				const FFrameTime MouseTime = ComputeFrameTimeFromMouse(MyGeometry, MouseEvent.GetScreenSpacePosition(), RangeToScreen);

				// Set the start range time?
				if (MouseDragType == DRAG_PLAYBACK_START)
				{
					SetPlaybackRangeStart(MouseTime.FrameNumber);
				}
				// Set the end range time?
				else if (MouseDragType == DRAG_PLAYBACK_END)
				{
					SetPlaybackRangeEnd(MouseTime.FrameNumber - 1);
				}
				else if (MouseDragType == DRAG_SELECTION_START)
				{
					SetSelectionRangeStart(MouseTime.FrameNumber);
				}
				// Set the end range time?
				else if (MouseDragType == DRAG_SELECTION_END)
				{
					SetSelectionRangeEnd(MouseTime.FrameNumber);
				}
				else if (MouseDragType == DRAG_SCRUBBING_TIME)
				{
					// Delegate responsibility for clamping to the current view range to the client.
					CommitScrubPosition(MouseTime, /*bIsScrubbing=*/true);
				}
			}
		}

		if (DistanceDragged != 0.f && (bHandleLeftMouseButton || bHandleRightMouseButton))
		{
			return FReply::Handled().CaptureMouse(WidgetOwner.AsShared());
		}

		return FReply::Handled();
	}

	void FTimeSliderController::CommitScrubPosition(FFrameTime NewValue, bool bIsScrubbing)
	{
		// Manage the scrub position ourselves if it's not bound to a delegate.
		if (!TimeSliderArgs.ScrubPosition.IsBound())
		{
			TimeSliderArgs.ScrubPosition.Set(NewValue);
		}

		// TODO: Change if anim timeline needs to handle sequencer style middle mouse manipulation which changes time but doesn't evaluate.
		TimeSliderArgs.OnScrubPositionChanged.ExecuteIfBound(NewValue, bIsScrubbing, /*bEvaluate*/ true);
	}

	FReply FTimeSliderController::OnMouseWheel(SWidget& WidgetOwner, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		if (TimeSliderArgs.AllowZoom && MouseEvent.IsControlDown())
		{
			const float MouseFractionX = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition()).X / MyGeometry.GetLocalSize().X;
			const float ZoomDelta = -0.2f * MouseEvent.GetWheelDelta();
			if (ZoomByDelta(ZoomDelta, MouseFractionX))
			{
				return FReply::Handled();
			}
		}
		else if (MouseEvent.IsShiftDown())
		{
			PanByDelta(-MouseEvent.GetWheelDelta());
			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

	FCursorReply FTimeSliderController::OnCursorQuery(const SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
	{
		const FScrubRangeToScreen RangeToScreen(TimeSliderArgs.ViewRange.Get(), MyGeometry.Size);
		const FFrameRate FrameResolution = GetTickResolution();
		const TRange<double> SelectionRange = TimeSliderArgs.SelectionRange.Get() / FrameResolution;
		const TRange<double> PlaybackRange = TimeSliderArgs.PlaybackRange.Get() / FrameResolution;
		const float HitTestPixel = MyGeometry.AbsoluteToLocal(CursorEvent.GetScreenSpacePosition()).X;
		const bool bLockedPlayRange = TimeSliderArgs.IsPlaybackRangeLocked.Get();
		const bool bHitScrubber = GetHitTestScrubberPixelRange(TimeSliderArgs.ScrubPosition.Get(), RangeToScreen).HandleRange.Contains(HitTestPixel);

		if (MouseDragType == DRAG_SCRUBBING_TIME)
		{
			return FCursorReply::Unhandled();
		}

		// Use L/R resize cursor if we're dragging or hovering a playback range bound.
		if ((MouseDragType == DRAG_PLAYBACK_END) ||
			(MouseDragType == DRAG_PLAYBACK_START) ||
			(MouseDragType == DRAG_SELECTION_START) ||
			(MouseDragType == DRAG_SELECTION_END) ||
			(MouseDragType == DRAG_TIME) ||
			(!bLockedPlayRange && !bHitScrubber && HitTestRangeStart(RangeToScreen, PlaybackRange, HitTestPixel)) ||
			(!bLockedPlayRange && !bHitScrubber && HitTestRangeEnd(RangeToScreen, PlaybackRange, HitTestPixel)) ||
			(!SelectionRange.IsEmpty() && !bHitScrubber && HitTestRangeStart(RangeToScreen, SelectionRange, HitTestPixel)) ||
			(!SelectionRange.IsEmpty() && !bHitScrubber && HitTestRangeEnd(RangeToScreen, SelectionRange, HitTestPixel)))
		{
			return FCursorReply::Cursor(EMouseCursor::ResizeLeftRight);
		}

		return FCursorReply::Unhandled();
	}

	int32 FTimeSliderController::OnPaintViewArea(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, bool bEnabled, const FPaintViewAreaArgs& Args) const
	{
		const ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

		TRange<double> LocalViewRange = TimeSliderArgs.ViewRange.Get();
		FScrubRangeToScreen RangeToScreen(LocalViewRange, AllottedGeometry.Size);

		if (Args.PlaybackRangeArgs.IsSet())
		{
			FPaintPlaybackRangeArgs PaintArgs = Args.PlaybackRangeArgs.GetValue();
			LayerId = DrawPlaybackRange(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, RangeToScreen, PaintArgs);
			PaintArgs.SolidFillOpacity = 0.2f;
			LayerId = DrawSelectionRange(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, RangeToScreen, PaintArgs);
		}

		if (Args.bDisplayTickLines)
		{
			static FLinearColor TickColor(0.1f, 0.1f, 0.1f, 0.3f);

			// Draw major tick lines in the section area.
			FDrawTickArgs DrawTickArgs;
			{
				DrawTickArgs.AllottedGeometry = AllottedGeometry;
				DrawTickArgs.bMirrorLabels = false;
				DrawTickArgs.bOnlyDrawMajorTicks = true;
				DrawTickArgs.TickColor = TickColor;
				DrawTickArgs.CullingRect = MyCullingRect;
				DrawTickArgs.DrawEffects = DrawEffects;
				// Draw major ticks under sections.
				DrawTickArgs.StartLayer = LayerId - 1;
				// Draw the tick the entire height of the section area.
				DrawTickArgs.TickOffset = 0.0f;
				DrawTickArgs.MajorTickHeight = AllottedGeometry.Size.Y;
			}

			DrawTicks(OutDrawElements, LocalViewRange, RangeToScreen, DrawTickArgs);
		}

		if (Args.bDisplayScrubPosition)
		{
			// Draw a line for the scrub position.
			const float LinePos = RangeToScreen.InputToLocalX(TimeSliderArgs.ScrubPosition.Get().AsDecimal() / GetTickResolution().AsDecimal());

			TArray<FVector2D> LinePoints;
			{
				LinePoints.AddUninitialized(2);
				LinePoints[0] = FVector2D(0.0f, 0.0f);
				LinePoints[1] = FVector2D(0.0f, FMath::FloorToFloat(AllottedGeometry.Size.Y));
			}

			FSlateDrawElement::MakeLines(
				OutDrawElements,
				LayerId + 1,
				AllottedGeometry.ToPaintGeometry(FVector2f(1.0f, 1.0f), FSlateLayoutTransform(FVector2f(LinePos, 0.0f))),
				LinePoints,
				DrawEffects,
				FLinearColor(1.0f, 1.0f, 1.0f, 0.5f),
				false
			);
		}

		return LayerId;
	}

	FFrameTime FTimeSliderController::GetFrameTimeFromMouse(const FGeometry& Geometry, FVector2D ScreenSpacePosition) const
	{
		const FScrubRangeToScreen ScrubRangeToScreen(TimeSliderArgs.ViewRange.Get(), Geometry.Size);
		return ComputeFrameTimeFromMouse(Geometry, ScreenSpacePosition, ScrubRangeToScreen);
	}

	void FTimeSliderController::ClampViewRange(const double NewRangeMin, const double NewRangeMax)
	{
		bool bNeedsClampSet = false;
		double NewClampRangeMin = TimeSliderArgs.ClampRange.Get().GetLowerBoundValue();
		if (NewRangeMin < TimeSliderArgs.ClampRange.Get().GetLowerBoundValue())
		{
			NewClampRangeMin = NewRangeMin;
			bNeedsClampSet = true;
		}

		double NewClampRangeMax = TimeSliderArgs.ClampRange.Get().GetUpperBoundValue();
		if (NewRangeMax > TimeSliderArgs.ClampRange.Get().GetUpperBoundValue())
		{
			NewClampRangeMax = NewRangeMax;
			bNeedsClampSet = true;
		}

		if (bNeedsClampSet)
		{
			SetClampRange(NewClampRangeMin, NewClampRangeMax);
		}
	}

	void FTimeSliderController::SetViewRange(double NewRangeMin, double NewRangeMax, EViewRangeInterpolation Interpolation)
	{
		// Clamp to a minimum size to avoid zero-sized or negative visible ranges.
		double MinVisibleTimeRange = FFrameNumber(1) / GetTickResolution();
		TRange<double> ExistingViewRange = TimeSliderArgs.ViewRange.Get();

		if (NewRangeMax == ExistingViewRange.GetUpperBoundValue())
		{
			if (NewRangeMin > NewRangeMax - MinVisibleTimeRange)
			{
				NewRangeMin = NewRangeMax - MinVisibleTimeRange;
			}
		}
		else if (NewRangeMax < NewRangeMin + MinVisibleTimeRange)
		{
			NewRangeMax = NewRangeMin + MinVisibleTimeRange;
		}

		// Clamp to the clamp range.
		const TRange<double> NewRange = TRange<double>(NewRangeMin, NewRangeMax);
		TimeSliderArgs.OnViewRangeChanged.ExecuteIfBound(NewRange, Interpolation);

		if (!TimeSliderArgs.ViewRange.IsBound())
		{
			// The  output is not bound to a delegate so we'll manage the value ourselves (no animation).
			TimeSliderArgs.ViewRange.Set(NewRange);
		}
	}

	void FTimeSliderController::SetClampRange(double NewRangeMin, double NewRangeMax)
	{
		const TRange<double> NewRange(NewRangeMin, NewRangeMax);

		TimeSliderArgs.OnClampRangeChanged.ExecuteIfBound(NewRange);

		if (!TimeSliderArgs.ClampRange.IsBound())
		{
			// The output is not bound to a delegate so we'll manage the value ourselves (no animation).
			TimeSliderArgs.ClampRange.Set(NewRange);
		}
	}

	void FTimeSliderController::SetPlayRange(FFrameNumber RangeStart, int32 RangeDuration)
	{
		check(RangeDuration >= 0);
		const TRange<FFrameNumber> NewRange(RangeStart, RangeStart + RangeDuration);
		TimeSliderArgs.OnPlaybackRangeChanged.ExecuteIfBound(NewRange);
		if (!TimeSliderArgs.PlaybackRange.IsBound())
		{
			// The output is not bound to a delegate so we'll manage the value ourselves (no animation).
			TimeSliderArgs.PlaybackRange.Set(NewRange);
		}
	}

	bool FTimeSliderController::ZoomByDelta(float InDelta, float MousePositionFraction)
	{
		const TRange<double> LocalViewRange = TimeSliderArgs.ViewRange.Get().GetAnimationTarget();
		const double LocalViewRangeMax = LocalViewRange.GetUpperBoundValue();
		const double LocalViewRangeMin = LocalViewRange.GetLowerBoundValue();
		const double OutputViewSize = LocalViewRangeMax - LocalViewRangeMin;
		const double OutputChange = OutputViewSize * InDelta;

		double NewViewOutputMin = LocalViewRangeMin - FMath::CeilToFloat(OutputChange * MousePositionFraction);
		double NewViewOutputMax = LocalViewRangeMax + FMath::CeilToFloat(OutputChange * (1.0f - MousePositionFraction));
		if (NewViewOutputMin < NewViewOutputMax)
		{
			ClampViewRange(NewViewOutputMin, NewViewOutputMax);
			SetViewRange(NewViewOutputMin, NewViewOutputMax, EViewRangeInterpolation::Animated);
			return true;
		}

		return false;
	}

	void FTimeSliderController::PanByDelta(float InDelta)
	{
		// The fraction of the current view range to scroll per unit delta.
		const float ScrollPanFraction = 0.1f;
		const TRange<double> LocalViewRange = TimeSliderArgs.ViewRange.Get().GetAnimationTarget();
		const double CurrentMin = LocalViewRange.GetLowerBoundValue();
		const double CurrentMax = LocalViewRange.GetUpperBoundValue();

		// Adjust the delta to be a percentage of the current range.
		InDelta *= ScrollPanFraction * (CurrentMax - CurrentMin);

		double NewViewOutputMin = CurrentMin + InDelta;
		double NewViewOutputMax = CurrentMax + InDelta;

		ClampViewRange(NewViewOutputMin, NewViewOutputMax);
		SetViewRange(NewViewOutputMin, NewViewOutputMax, EViewRangeInterpolation::Animated);
	}

	bool FTimeSliderController::HitTestRangeStart(const FScrubRangeToScreen& RangeToScreen, const TRange<double>& Range, float HitPixel) const
	{
		if (Range.HasLowerBound())
		{
			static const float BrushSizeInStateUnits = 6.0f;
			static const float DragToleranceSlateUnits = 2.0f;
			static const float MouseTolerance = 2.0f;
			const float RangeStartPixel = RangeToScreen.InputToLocalX(Range.GetLowerBoundValue());

			// Hit test against the brush region to the right of the playback start position, +/- DragToleranceSlateUnits.
			return (HitPixel >= RangeStartPixel - MouseTolerance - DragToleranceSlateUnits) &&
				(HitPixel <= RangeStartPixel + MouseTolerance + BrushSizeInStateUnits + DragToleranceSlateUnits);
		}

		return false;
	}

	bool FTimeSliderController::HitTestRangeEnd(const FScrubRangeToScreen& RangeToScreen, const TRange<double>& Range, float HitPixel) const
	{
		if (Range.HasUpperBound())
		{
			static const float BrushSizeInStateUnits = 6.0f;
			static const float DragToleranceSlateUnits = 2.0f;
			static const float MouseTolerance = 2.0f;
			const float RangeEndPixel = RangeToScreen.InputToLocalX(Range.GetUpperBoundValue());

			// Hit test against the brush region to the left of the playback end position, +/- DragToleranceSlateUnits.
			return (HitPixel >= RangeEndPixel - MouseTolerance - BrushSizeInStateUnits - DragToleranceSlateUnits) &&
				(HitPixel <= RangeEndPixel + MouseTolerance + DragToleranceSlateUnits);
		}

		return false;
	}

	void FTimeSliderController::SetPlaybackRangeStart(FFrameNumber NewStart) const
	{
		const TRange<FFrameNumber> PlaybackRange = TimeSliderArgs.PlaybackRange.Get();
		if (NewStart <= UE::MovieScene::DiscreteExclusiveUpper(PlaybackRange))
		{
			TimeSliderArgs.OnPlaybackRangeChanged.ExecuteIfBound(TRange<FFrameNumber>(NewStart, PlaybackRange.GetUpperBound()));
		}
	}

	void FTimeSliderController::SetPlaybackRangeEnd(FFrameNumber NewEnd) const
	{
		const TRange<FFrameNumber> PlaybackRange = TimeSliderArgs.PlaybackRange.Get();
		if (NewEnd >= UE::MovieScene::DiscreteInclusiveLower(PlaybackRange))
		{
			TimeSliderArgs.OnPlaybackRangeChanged.ExecuteIfBound(TRange<FFrameNumber>(PlaybackRange.GetLowerBound(), NewEnd));
		}
	}

	void FTimeSliderController::SetSelectionRangeStart(FFrameNumber NewStart) const
	{
		const TRange<FFrameNumber> SelectionRange = TimeSliderArgs.SelectionRange.Get();
		if (SelectionRange.IsEmpty())
		{
			TimeSliderArgs.OnSelectionRangeChanged.ExecuteIfBound(TRange<FFrameNumber>(NewStart, NewStart + 1));
		}
		else if (NewStart <= UE::MovieScene::DiscreteExclusiveUpper(SelectionRange))
		{
			TimeSliderArgs.OnSelectionRangeChanged.ExecuteIfBound(TRange<FFrameNumber>(NewStart, SelectionRange.GetUpperBound()));
		}
	}

	void FTimeSliderController::SetSelectionRangeEnd(FFrameNumber NewEnd) const
	{
		const TRange<FFrameNumber> SelectionRange = TimeSliderArgs.SelectionRange.Get();
		if (SelectionRange.IsEmpty())
		{
			TimeSliderArgs.OnSelectionRangeChanged.ExecuteIfBound(TRange<FFrameNumber>(NewEnd - 1, NewEnd));
		}
		else if (NewEnd >= UE::MovieScene::DiscreteInclusiveLower(SelectionRange))
		{
			TimeSliderArgs.OnSelectionRangeChanged.ExecuteIfBound(TRange<FFrameNumber>(SelectionRange.GetLowerBound(), NewEnd));
		}
	}


	/**
	 * An overlay that displays global information in the track area.
	 */
	class SAnimTimelineOverlay : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SAnimTimelineOverlay)
			: _DisplayTickLines(true)
			, _DisplayScrubPosition(false)
		{}
			SLATE_ATTRIBUTE(bool, DisplayTickLines)
			SLATE_ATTRIBUTE(bool, DisplayScrubPosition)
			SLATE_ATTRIBUTE(FPaintPlaybackRangeArgs, PaintPlaybackRangeArgs)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TSharedRef<FTimeSliderController> InTimeSliderController)
		{
			bDisplayScrubPosition = InArgs._DisplayScrubPosition;
			bDisplayTickLines = InArgs._DisplayTickLines;
			PaintPlaybackRangeArgs = InArgs._PaintPlaybackRangeArgs;
			TimeSliderController = InTimeSliderController;
		}

	private:
		/** SWidget Interface. */
		virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

	private:
		/** Controller for manipulating time. */
		TSharedPtr<FTimeSliderController> TimeSliderController;
		/** Whether or not to display the scrub position. */
		TAttribute<bool> bDisplayScrubPosition;
		/** Whether or not to display tick lines. */
		TAttribute<bool> bDisplayTickLines;
		/** User-supplied options for drawing playback range. */
		TAttribute<FPaintPlaybackRangeArgs> PaintPlaybackRangeArgs;
	};

	int32 SAnimTimelineOverlay::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
	{
		FPaintViewAreaArgs PaintArgs;
		PaintArgs.bDisplayTickLines = bDisplayTickLines.Get();
		PaintArgs.bDisplayScrubPosition = bDisplayScrubPosition.Get();

		if (PaintPlaybackRangeArgs.IsSet())
		{
			PaintArgs.PlaybackRangeArgs = PaintPlaybackRangeArgs.Get();
		}

		TimeSliderController->OnPaintViewArea(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, ShouldBeEnabled(bParentEnabled), PaintArgs);
		return SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
	}

	void STimeline::Construct(const FArguments& InArgs, TWeakPtr<FTimelineModel> InModel, TWeakPtr<FTimelineTracksModel> InTracksModel)
	{
		Model = InModel;
		TracksModel = InTracksModel;

		// These settings allow us to display everything in terms of frames
		const EFrameNumberDisplayFormats DisplayFormat = EFrameNumberDisplayFormats::Frames;
		const FFrameRate TickResolution = FFrameRate(1, 1);
		const FFrameRate DisplayRate = FFrameRate(1, 1);

		// Create our numeric type interface so we can pass it to the time slider below.
		NumericTypeInterface = MakeShared<FFrameNumberInterface>(DisplayFormat, 0, TickResolution, DisplayRate);

		FTimeSliderArgs TimeSliderArgs;
		{
			TimeSliderArgs.ScrubPosition = MakeAttributeLambda([this](){ return Model.IsValid() ? Model.Pin()->GetTimelineFrameTime() : FFrameTime(0); });
			TimeSliderArgs.ViewRange = MakeAttributeLambda([this]() { return Model.IsValid() ? Model.Pin()->GetTimelineViewRangeTimes() : FAnimatedRange(0.0, 0.0); });
			TimeSliderArgs.PlaybackRange = MakeAttributeLambda([this]() { return Model.IsValid() ? Model.Pin()->GetTimelinePlaybackRangeFrames() : TRange<FFrameNumber>(0, 0); });
			TimeSliderArgs.ClampRange = MakeAttributeLambda([this]() { return Model.IsValid() ? Model.Pin()->GetTimelineWorkingRangeTimes() : FAnimatedRange(0.0, 0.0); });
			TimeSliderArgs.DisplayRate = DisplayRate;
			TimeSliderArgs.TickResolution = TickResolution;
			TimeSliderArgs.OnViewRangeChanged = FOnViewRangeChanged::CreateLambda([this](TRange<double> InRange, EViewRangeInterpolation InInterpolation) { if (Model.IsValid()) { Model.Pin()->SetTimelineViewRangeTimes(InRange); } });
			TimeSliderArgs.OnClampRangeChanged = FOnTimeRangeChanged::CreateLambda([this](TRange<double> InRange) { if (Model.IsValid()) { Model.Pin()->SetTimelineWorkingRangeTimes(InRange); } });
			TimeSliderArgs.IsPlaybackRangeLocked = true;
			TimeSliderArgs.PlaybackStatus = EMovieScenePlayerStatus::Stopped;
			TimeSliderArgs.NumericTypeInterface = NumericTypeInterface;
			TimeSliderArgs.OnScrubPositionChanged = FOnScrubPositionChanged::CreateLambda([this](FFrameTime NewScrubPosition, bool bIsScrubbing, bool bEvaluate) {
				
				if (Model.IsValid())
				{
					Model.Pin()->SetTimelineFrameTime(NewScrubPosition);
					Model.Pin()->SetTransportPlaybackMode(EPlaybackMode::Stopped);
				}
				
			});
		}

		TimeSliderController = MakeShared<FTimeSliderController>(TimeSliderArgs, Model, SharedThis(this));
	
		// Create the top slider.
		ISequencerWidgetsModule& SequencerWidgets = FModuleManager::Get().LoadModuleChecked<ISequencerWidgetsModule>("SequencerWidgets");

		const bool bMirrorLabels = false;
		TopTimeSlider = SequencerWidgets.CreateTimeSlider(TimeSliderController.ToSharedRef(), bMirrorLabels);

		// Create bottom time range slider.
		TSharedRef<ITimeSlider> BottomTimeRange = SequencerWidgets.CreateTimeRange(
			FTimeRangeArgs(
				EShowRange::ViewRange | EShowRange::WorkingRange | EShowRange::PlaybackRange,
				EShowRange::ViewRange | EShowRange::WorkingRange,
				TimeSliderController.ToSharedRef(),
				EVisibility::Visible,
				NumericTypeInterface.ToSharedRef()),
			SequencerWidgets.CreateTimeRangeSlider(TimeSliderController.ToSharedRef()));

		TSharedRef<SScrollBar> ScrollBar = SNew(SScrollBar).Thickness(FVector2D(5.0f, 5.0f));

		TSharedRef<STrackArea> TrackArea = SNew(STrackArea, TimeSliderController.ToSharedRef());

		TSharedRef<SOutliner> Outliner = SNew(SOutliner, Model, TracksModel, TrackArea)
			.ExternalScrollbar(ScrollBar)
			.Clipping(EWidgetClipping::ClipToBounds);

		TrackArea->SetOutliner(Outliner);

		TransportControls = SNew(STimelineTransportControls, Model);

		const FMargin ResizeBarPadding(4.0f, 0, 0, 0);

		TAttribute<float> OutlinerFillCoefficientGetter;
		OutlinerFillCoefficientGetter.Bind(TAttribute<float>::FGetter::CreateLambda([this]() { return OutlinerFillCoefficient; }));

		TAttribute<float> TimelineFillCoefficientGetter;
		TimelineFillCoefficientGetter.Bind(TAttribute<float>::FGetter::CreateLambda([this]() { return TimelineFillCoefficient; }));

		ChildSlot
		[
			SNew(SOverlay)
			+SOverlay::Slot()
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				[
					SNew(SOverlay)
					+SOverlay::Slot()
					[
						SNew(SGridPanel)
						.FillRow(1, 1.0f)
						.FillColumn(0, OutlinerFillCoefficientGetter)
						.FillColumn(1, TimelineFillCoefficientGetter)

						+SGridPanel::Slot(0, 0, SGridPanel::Layer(10))
						[
							SNew(SHorizontalBox)
								+ SHorizontalBox::Slot()
								.FillWidth(1.0f)
								.HAlign(HAlign_Center)
								.VAlign(VAlign_Center)
								+SHorizontalBox::Slot()
								.VAlign(VAlign_Center)
								.HAlign(HAlign_Center)
								.AutoWidth()
								.Padding(2.0f, 0.0f, 2.0f, 0.0f)
								[
									SNew(SBox)
										.MinDesiredWidth(30.0f)
										.VAlign(VAlign_Center)
										.HAlign(HAlign_Center)
										[
											// Current play time.
											SNew(SSpinBox<double>)
												.Style(&FAppStyle::GetWidgetStyle<FSpinBoxStyle>("Sequencer.PlayTimeSpinBox"))
												.Value_Lambda([this]() { return Model.IsValid() ? Model.Pin()->GetTimelineFrameTime().AsDecimal() : 0.0; })
												.OnValueChanged_Lambda([this](double InFrameTime) { if (Model.IsValid()) { Model.Pin()->SetTimelineFrameTime(FFrameTime::FromDecimal(InFrameTime)); } })
												.OnValueCommitted_Lambda([this](double InFrame, ETextCommit::Type) { if (Model.IsValid()) { Model.Pin()->SetTimelineFrameTime(FFrameTime::FromDecimal(InFrame)); } })
												.MinValue(TOptional<double>())
												.MaxValue(TOptional<double>())
												.TypeInterface(NumericTypeInterface)
												.Delta_Lambda([this]() { return 1.0; })
												.LinearDeltaSensitivity(25)
										]
								]
						]

						// Main Timeline Area
						+ SGridPanel::Slot(0, 1, SGridPanel::Layer(10))
						.ColumnSpan(2)
						[
							SNew(SHorizontalBox)
								+ SHorizontalBox::Slot()
								[
									SNew(SOverlay)
										+ SOverlay::Slot()
										[
											SNew(SVerticalBox)
												+ SVerticalBox::Slot()
												.FillHeight(1.f)
												[
													SNew(SScrollBorder, Outliner)
														[
															SNew(SHorizontalBox)
						
																+ SHorizontalBox::Slot()
																.FillWidth(OutlinerFillCoefficientGetter)
																[
																	SNew(SBox)
																		[
																			Outliner
																		]
																]
						
																+ SHorizontalBox::Slot()
																.FillWidth(TimelineFillCoefficientGetter)
																[
																	SNew(SBox)
																		.Padding(ResizeBarPadding)
																		.Clipping(EWidgetClipping::ClipToBounds)
																		[
																			TrackArea
																		]
																]
														]
												]
										]
						
										+ SOverlay::Slot()
										.HAlign(HAlign_Right)
										[
											ScrollBar
										]
								]
						]

						// Transport controls.
						+SGridPanel::Slot(0, 3, SGridPanel::Layer(10))
						[
							TransportControls.ToSharedRef()
						]

						// Second column.
						+SGridPanel::Slot(1, 0)
						.Padding(ResizeBarPadding)
						.RowSpan(2)
						[
							SNew(SBorder)
							.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
							[
								SNew(SSpacer)
							]
						]

						+SGridPanel::Slot(1, 0, SGridPanel::Layer(10))
						.Padding(ResizeBarPadding)
						[
							SNew(SBorder)
							.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
							.BorderBackgroundColor(FLinearColor(0.5f, 0.5f, 0.5f, 1.0f))
							.Padding(0)
							.Clipping(EWidgetClipping::ClipToBounds)
							[
								TopTimeSlider.ToSharedRef()
							]
						]

						// Overlay that draws the tick lines.
						+SGridPanel::Slot(1, 1, SGridPanel::Layer(10))
						.Padding(ResizeBarPadding)
						[
							SNew(SAnimTimelineOverlay, TimeSliderController.ToSharedRef())
							.Visibility( EVisibility::HitTestInvisible )
							.DisplayScrubPosition(false)
							.DisplayTickLines(true)
							.Clipping(EWidgetClipping::ClipToBounds)
							.PaintPlaybackRangeArgs(FPaintPlaybackRangeArgs(FAppStyle::GetBrush("Sequencer.Timeline.PlayRange_L"), FAppStyle::GetBrush("Sequencer.Timeline.PlayRange_R"), 6.0f))
						]

						// Overlay that draws the scrub position.
						+SGridPanel::Slot(1, 1, SGridPanel::Layer(20))
						.Padding(ResizeBarPadding)
						[
							SNew(SAnimTimelineOverlay, TimeSliderController.ToSharedRef())
							.Visibility(EVisibility::HitTestInvisible)
							.DisplayScrubPosition(true)
							.DisplayTickLines(false)
							.Clipping(EWidgetClipping::ClipToBounds)
						]

						// Play range slider.
						+SGridPanel::Slot(1, 3, SGridPanel::Layer(10))
						.Padding(ResizeBarPadding)
						[
							SNew(SBorder)
							.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
							.BorderBackgroundColor(FLinearColor(0.5f, 0.5f, 0.5f, 1.0f))
							.Clipping(EWidgetClipping::ClipToBounds)
							.Padding(0)
							[
								BottomTimeRange
							]
						]
					]
					+ SOverlay::Slot()
					[
						// track area virtual splitter overlay
						SNew(STimelineSplitterOverlay)
							.Style(FAppStyle::Get(), "AnimTimeline.Outliner.Splitter")
							.Visibility(EVisibility::SelfHitTestInvisible)

							+ SSplitter::Slot()
							.Value(OutlinerFillCoefficientGetter)
							.OnSlotResized(SSplitter::FOnSlotResized::CreateLambda([this](float Value) { OutlinerFillCoefficient = Value; }))
							[
								SNew(SSpacer)
							]

							+ SSplitter::Slot()
							.Value(TimelineFillCoefficientGetter)
							.OnSlotResized(SSplitter::FOnSlotResized::CreateLambda([this](float Value) { TimelineFillCoefficient = Value; }))
							[
								SNew(SSpacer)
							]
					]
				]
			]
		];
	}

	FReply STimeline::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
		{
			const FWidgetPath WidgetPath = (MouseEvent.GetEventPath() != nullptr) ? *MouseEvent.GetEventPath() : FWidgetPath();

			const bool bCloseAfterSelection = true;
			FMenuBuilder MenuBuilder(bCloseAfterSelection, nullptr);

			MenuBuilder.BeginSection("SnappingOptions", LOCTEXT("SnappingOptions", "Snapping"));
			{
				MenuBuilder.AddMenuEntry(FUIAction(),
					SNew(SCheckBox)
						.OnCheckStateChanged_Lambda([this](ECheckBoxState State) { if (Model.IsValid()) { Model.Pin()->SetTimelineSnapOnFrames(State == ECheckBoxState::Checked); } })
						.IsChecked_Lambda([this]() { return Model.IsValid() ? (Model.Pin()->GetTimelineSnapOnFrames() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked) : ECheckBoxState::Undetermined; })
						[
							SNew(STextBlock).Text(LOCTEXT("SnapOnFramesCheckbox", "Frames"))
						]
				);
			}

			MenuBuilder.EndSection();

			FSlateApplication::Get().PushMenu(SharedThis(this), WidgetPath, MenuBuilder.MakeWidget(), FSlateApplication::Get().GetCursorPos(), FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

	// FFrameRate::ComputeGridSpacing doesn't deal well with prime numbers, so we have a custom implementation here
	static bool ComputeGridSpacing(const FFrameRate& InFrameRate, float PixelsPerSecond, double& OutMajorInterval, int32& OutMinorDivisions, float MinTickPx, float DesiredMajorTickPx)
	{
		// First try built-in spacing.
		const bool bResult = InFrameRate.ComputeGridSpacing(PixelsPerSecond, OutMajorInterval, OutMinorDivisions, MinTickPx, DesiredMajorTickPx);
		if (!bResult || OutMajorInterval <= 1.0)
		{
			if (PixelsPerSecond <= 0.0f)
			{
				return false;
			}

			const int32 RoundedFPS = FMath::RoundToInt(InFrameRate.AsDecimal());

			if (RoundedFPS > 0)
			{
				// Showing frames.
				TArray<int32, TInlineAllocator<10>> CommonBases;

				// Divide the rounded frame rate by 2s, 3s or 5s recursively.
				{
					const int32 Denominators[] = { 2, 3, 5 };

					int32 LowestBase = RoundedFPS;
					for (;;)
					{
						CommonBases.Add(LowestBase);
	
						if (LowestBase % 2 == 0)      { LowestBase = LowestBase / 2; }
						else if (LowestBase % 3 == 0) { LowestBase = LowestBase / 3; }
						else if (LowestBase % 5 == 0) { LowestBase = LowestBase / 5; }
						else
						{ 
							int32 LowestResult = LowestBase;
							for (int32 Denominator : Denominators)
							{
								const int32 Result = LowestBase / Denominator;
								if(Result > 0 && Result < LowestResult)
								{
									LowestResult = Result;
								}
							}

							if (LowestResult < LowestBase)
							{
								LowestBase = LowestResult;
							}
							else
							{
								break;
							}
						}
					}
				}

				Algo::Reverse(CommonBases);

				const int32 Scale = FMath::CeilToInt(DesiredMajorTickPx / PixelsPerSecond * InFrameRate.AsDecimal());
				const int32 BaseIndex = FMath::Min(Algo::LowerBound(CommonBases, Scale), CommonBases.Num() - 1);
				const int32 Base = CommonBases[BaseIndex];

				const int32 MajorIntervalFrames = FMath::CeilToInt(Scale / (float)Base) * Base;
				OutMajorInterval = FMath::Max(MajorIntervalFrames * InFrameRate.AsInterval(), 1.0f);

				// Find the lowest number of divisions we can show that's larger than the minimum tick size.
				OutMinorDivisions = 0;
				for (int32 DivIndex = 0; DivIndex < BaseIndex; ++DivIndex)
				{
					if (Base % CommonBases[DivIndex] == 0)
					{
						const int32 MinorDivisions = MajorIntervalFrames/CommonBases[DivIndex];
						if (OutMajorInterval / MinorDivisions * PixelsPerSecond >= MinTickPx)
						{
							OutMinorDivisions = MinorDivisions;
							break;
						}
					}
				}
			}
		}

		return OutMajorInterval != 0;
	}

	bool STimeline::GetGridMetrics(float PhysicalWidth, double& OutMajorInterval, int32& OutMinorDivisions) const
	{
		const FSlateFontInfo SmallLayoutFont = FCoreStyle::GetDefaultFontStyle("Regular", 8);
		const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
		const FFrameRate FrameRate = Model.IsValid() ? Model.Pin()->GetTimelineFrameRate() : FFrameRate(60, 1);
		const double BiggestTime = Model.IsValid() ? Model.Pin()->GetTimelineViewRangeTimes().GetUpperBoundValue() : 0.0;
		const FString TickString = NumericTypeInterface->ToString((BiggestTime * FrameRate).FrameNumber.Value);
		const FVector2D MaxTextSize = FontMeasureService->Measure(TickString, SmallLayoutFont);
		static const float MajorTickMultiplier = 2.0f;
		const float MinTickPx = MaxTextSize.X + 5.0f;
		const float DesiredMajorTickPx = MaxTextSize.X * MajorTickMultiplier;

		if (PhysicalWidth > 0 && FrameRate.AsDecimal() > 0)
		{
			return ComputeGridSpacing(
				FrameRate,
				PhysicalWidth / Model.Pin()->GetTimelineViewRangeTimes().Size<double>(),
				OutMajorInterval,
				OutMinorDivisions,
				MinTickPx,
				DesiredMajorTickPx);
		}

		return false;
	}

	TSharedPtr<ITimeSliderController> STimeline::GetTimeSliderController() const
	{ 
		return TimeSliderController; 
	}

	void STimeline::SetModel(const TWeakPtr<FTimelineModel>& InModel, const TWeakPtr<FTimelineTracksModel>& InTracksModel)
	{
		Model = InModel; 
		TracksModel = InTracksModel;
		TimeSliderController->SetModel(Model);
		TransportControls->SetModel(Model);
	}

}

#undef LOCTEXT_NAMESPACE
