// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAnimDatabaseEditorViewport.h"

#include "AnimDatabaseEditorViewportClient.h"
#include "Framework/Application/SlateApplication.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Viewports.h"
#include "PreviewProfileController.h"
#include "AnimDatabaseEditorToolkit.h"
#include "AnimDatabaseEditorPreviewScene.h"

#define LOCTEXT_NAMESPACE "AnimDatabaseEditorViewport"

namespace UE::AnimDatabase::Editor
{
	void SViewport::Construct(
		const FArguments& InArgs, 
		const TSharedRef<FAssetEditorToolkit>& InAssetEditorToolkit, 
		const TSharedRef<FPreviewScene>& InPreviewScene, 
		const FEditorModeID InModeID)
	{
		PreviewScenePtr = InPreviewScene;
		AssetEditorToolkitPtr = InAssetEditorToolkit;
		ModeID = InModeID;

		SEditorViewport::Construct(
			SEditorViewport::FArguments()
			.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
			.AddMetaData<FTagMetaData>(TEXT("AnimationTools.Viewport"))
		);
	}

	TSharedRef<FEditorViewportClient> SViewport::MakeEditorViewportClient()
	{
		ViewportClient = MakeShared<FViewportClient>(PreviewScenePtr, SharedThis(this), AssetEditorToolkitPtr, ModeID);
		ViewportClient->ViewportType = LVT_Perspective;
		ViewportClient->bSetListenerPosition = false;
		ViewportClient->SetViewLocation(EditorViewportDefs::DefaultPerspectiveViewLocation);
		ViewportClient->SetViewRotation(EditorViewportDefs::DefaultPerspectiveViewRotation);

		return ViewportClient.ToSharedRef();
	}
}

#undef LOCTEXT_NAMESPACE