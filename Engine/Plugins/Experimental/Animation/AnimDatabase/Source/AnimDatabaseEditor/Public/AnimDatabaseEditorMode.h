// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimDatabasePose.h"
#include "AnimDatabaseFrameRanges.h"
#include "AnimDatabaseFrameAttribute.h"

#include "EdMode.h"

namespace UE::AnimDatabase::Editor
{
	class FDatabaseToolkit;

	/** Simple Editor mode used for ticking the viewport of the Animation Database Editor */
	class FDatabaseMode : public FEdMode
	{
	public:

		const static FEditorModeID EditorModeId;

		virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override;
		virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;
		virtual bool RequiresLegacyViewportInteractions() const override { return false; }

	private:

		/** Maximum number of characters to draw in the scene */
		const int32 MaxCharacterNum = 128;

		/** Weak pointer to the database toolkit used for rendering */
		TWeakPtr<FDatabaseToolkit> WeakToolkit;

		/** Used to track when the selected ranges have changed and so when the timeline needs updating */
		TArray<int32> ActiveRanges;

		/** Temporary bone parents array */
		TArray<int32> BoneParents;

		/** Character Data */
		TArray<int32> CharacterSequenceIndices;
		TArray<float> CharacterRangeTimes;
		TArray<int32> CharacterRangeStarts;
		TArray<int32> CharacterRangeLengths;

		/** Temporary pose state object used to store the sampled pose state for all characters */
		FAnimDatabasePoseState PoseState;
		TLearningArray<1, FVector> AccumulatedRootLocations;
		TLearningArray<1, FQuat4f> AccumulatedRootRotations;

		/** Trajectories for all currently queried ranges */
		TArray<TLearningArray<1, FVector>> RangeTrajectoryLocations;
		TArray<TLearningArray<1, FQuat4f>> RangeTrajectoryRotations;

		/** Cached Frames Objects */
		TArray<FAnimDatabaseFrames> FramesObjects;
		TArray<FName> FramesNames;
		
		/** Cached Frame Ranges Objects */
		TArray<FAnimDatabaseFrameRanges> FrameRangesObjects;
		TArray<FName> FrameRangeNames;

		/** Cached Frame Attributes Objects */
		TArray<FAnimDatabaseFrameAttribute> FrameAttributeObjects;
		TArray<EAnimDatabaseAttributeType> FrameAttributeTypes;
		TArray<FName> FrameAttributeNames;
	};

}

