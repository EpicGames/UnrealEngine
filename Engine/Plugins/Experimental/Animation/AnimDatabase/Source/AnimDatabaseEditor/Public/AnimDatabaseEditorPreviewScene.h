// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AdvancedPreviewScene.h"

#define UE_API ANIMDATABASEEDITOR_API

class ACharacter;
class UPoseableMeshComponent;
class UCapsuleComponent;

namespace UE::AnimDatabase::Editor
{
	/** Simple re-usable preview scene that contains multiple characters each with a PoseableMeshComponent */
	class FPreviewScene : public FAdvancedPreviewScene
	{
	public:

		UE_API FPreviewScene(ConstructionValues CVS);
		UE_API virtual ~FPreviewScene() override;

		/** Updates the preview scene */
		UE_API virtual void Tick(float InDeltaTime) override;

		/** Gets the number of characters in the scene */
		UE_API int32 GetCharacterNum() const;

		/** Adds a new character to the scene. Returns the added character index */
		UE_API int32 AddCharacter();

		/** Removes a character from the scene */
		UE_API int32 RemoveCharacter();

		/** Adjusts the visibility for the given character */
		UE_API void SetCharacterVisibility(const int32 CharacterIdx, bool bVisibility);

		/** Gets the current visibility of the given character */
		UE_API bool GetCharacterVisibility(const int32 CharacterIdx) const;

		/** Gets the actor associated with a given character */
		UE_API ACharacter* GetCharacterActor(const int32 CharacterIdx) const;

		/** Gets the poseable mesh component associated with a given character */
		UE_API UPoseableMeshComponent* GetPoseableMeshComponent(const int32 CharacterIdx) const;

	private:

		/** Array of Character Actors */
		TArray<ACharacter*> CharacterActors;

		/** Array of Capsule Components which get added by default to each new character */
		TArray<UCapsuleComponent*> CapsuleComponents;

		/** Array of poseable mesh components */
		TArray<UPoseableMeshComponent*> PoseableMeshComponents;
	};
}

#undef UE_API