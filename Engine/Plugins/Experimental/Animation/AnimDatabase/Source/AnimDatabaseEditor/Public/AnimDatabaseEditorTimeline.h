// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/FrameRate.h"
#include "Math/Color.h"
#include "AnimatedRange.h"
#include "ITransportControl.h"

#include "AnimDatabaseFrameRanges.h"
#include "AnimDatabaseFrameAttribute.h"

#define UE_API ANIMDATABASEEDITOR_API

class UAnimSequence;

namespace UE::AnimDatabase::Editor
{
	/**
	 * Extremely basic data-model for the timeline widget. Stores information about various ranges, as well as playback state etc. Contains mainly
	 * just some basic getters and setters, as well as some functions that can be used within callbacks.
	 * 
	 * For the UAnimDatabaseDatabase we usually want to work in frames (given we have a fixed-framerate for all animations) so this model stores everything
	 * in terms of frames where possible (rather than in terms of time).
	 */
	struct FTimelineModel
	{
		UE_API FFrameRate GetTimelineFrameRate() const;
		UE_API FFrameTime GetTimelineFrameTime() const;
		UE_API FAnimatedRange GetTimelineViewRangeTimes() const;
		UE_API FAnimatedRange GetTimelineWorkingRangeTimes() const;
		UE_API TRange<FFrameNumber> GetTimelineViewRange() const;
		UE_API TRange<FFrameNumber> GetTimelineWorkingRange() const;
		UE_API TRange<FFrameNumber> GetTimelinePlaybackRangeFrames() const;
		UE_API float GetTimelinePlaybackRate() const;
		UE_API float GetTimelineCustomPlaybackRate() const;
		UE_API bool GetTimelineUseCustomPlaybackRate() const;
		UE_API bool GetTimelineSnapOnFrames() const;
		UE_API EPlaybackMode::Type GetTransportPlaybackMode() const;
		UE_API bool GetLooping() const;

		UE_API void SetTimelineFrameRate(const FFrameRate InFrameRate);
		UE_API void SetTimelineFrameTime(const FFrameTime InFrameTime);
		UE_API void SetTimelineViewRangeTimes(const FAnimatedRange InViewRange);
		UE_API void SetTimelineWorkingRangeTimes(const FAnimatedRange InWorkingRange);
		UE_API void SetTimelinePlaybackRate(const float InPlaybackRate);
		UE_API void SetTimelineCustomPlaybackRate(const float InCustomPlaybackRate);
		UE_API void SetTimelineViewRange(const TRange<FFrameNumber> InViewRange);
		UE_API void SetTimelineWorkingRange(const TRange<FFrameNumber> InWorkingRange);
		UE_API void SetTimelinePlaybackRange(const TRange<FFrameNumber> InPlaybackRange);
		UE_API void SetTimelineUseCustomPlaybackRate(const bool bInUseCustomPlaybackRate);
		UE_API void SetTimelineSnapOnFrames(const bool bInSnapOnFrames);
		UE_API void SetTransportPlaybackMode(const EPlaybackMode::Type InPlaybackMode);
		UE_API void SetLooping(const bool bLooping);

		UE_API void UpdateTimelineFrameTime(const float DeltaTime);

		UE_API void OnTransportStepForward();
		UE_API void OnTransportStepBackward();
		UE_API void OnTransportToStartFrame();
		UE_API void OnTransportToEndFrame();
		UE_API void OnTransportPlayPressed();

	private:

		/** Current Active View Range */
		TRange<FFrameNumber> ViewRange = TRange<FFrameNumber>(0, 0);

		/** Current Active Working Range */
		TRange<FFrameNumber> WorkingRange = TRange<FFrameNumber>(0, 0);

		/** Current Active Playback Range */
		TRange<FFrameNumber> PlaybackRange = TRange<FFrameNumber>(0, 0);

		/** Current FrameTime for the scrubber */
		FFrameTime FrameTime = FFrameTime();

		/** Current FrameRate - should match that of the database */
		FFrameRate FrameRate = FFrameRate(60, 1);

		/** Playback State */
		EPlaybackMode::Type PlaybackMode = (EPlaybackMode::Type)0; // Stopped

		/** Playback Rate */
		float PlaybackRate = 1.0f;

		/** Custom Playback Rate when used */
		float CustomPlaybackRate = 1.0f;

		/** If to use CustomPlaybackRate or the standard PlaybackRate */
		bool bUseCustomPlaybackRate = false;

		/** If to snap the scrubbing to discrete frames */
		bool bSnapOnFrames = false;

		/** If looping is enabled */
		bool bLooping = false;
	};

	/**
	 * This is a struct used for each row in the timeline tracks. Since this is a tree-view each row can correspond to something different so this
	 * has several members for the different types of row objects to display.
	 */
	struct FTimelineTrack
	{
		/** Height of each row in the timeline tracks */
		static constexpr float Height = 22.0f;

		/** Child objects */
		TArray<TSharedRef<FTimelineTrack>> Children;

		/** Color identifier */
		FLinearColor Color = FLinearColor::Black;

		/** Display name to use */
		FText DisplayName;

		/** Track Vertical Offset */
		float Offset = 0.0f;

		/** Whether this track is a range */
		bool bIsRange : 1;

		/** Whether this track is expanded */
		bool bIsExpanded : 1;

		/** Timeline View Offset in frames */
		int32 ViewOffset = 0;

		/** Sequence Name */
		FString SequenceName;

		/** Sequence Path */
		FSoftObjectPath SequencePath;

		/** Frames Object for this track */
		FAnimDatabaseFrames FramesObject;

		/** Frame Ranges Object for this track */
		FAnimDatabaseFrameRanges FrameRangesObject;

		/** Frame Attribute Object for this track */
		FAnimDatabaseFrameAttribute FrameAttributeObject;
	};

	/**
	 * This is the model object for timeline tracks. In this case the only kinds of tracks allowed are FAnimDatabaseFrames and FAnimDatabaseFrameRanges so this
	 * is what this model is designed to display and handle.
	 */
	struct FTimelineTracksModel
	{
		/** Resets and removes all the existing tracks */
		UE_API void ResetTracks();

		/** Adds a new set of tracks for a single sequence for various frames, frame ranges, and frame attributes */
		UE_API void AddTrack(
			const UAnimSequence* Sequence,
			const FLinearColor& RangeColor,
			const TArrayView<const FName> FrameNames,
			const TArrayView<const FName> FrameRangeNames,
			const TArrayView<const FName> FrameAttributeNames,
			const TArrayView<const FAnimDatabaseFrames> Frames,
			const TArrayView<const FAnimDatabaseFrameRanges> FrameRanges,
			const TArrayView<const FAnimDatabaseFrameAttribute> FrameAttributes,
			const int32 ViewOffset);

		/** Gets a reference to the root track objects */
		UE_API TArray<TSharedRef<FTimelineTrack>>& GetRootTracks();

	private:

		/** Root track objects */
		TArray<TSharedRef<FTimelineTrack>> RootTracks;
	};

}

#undef UE_API