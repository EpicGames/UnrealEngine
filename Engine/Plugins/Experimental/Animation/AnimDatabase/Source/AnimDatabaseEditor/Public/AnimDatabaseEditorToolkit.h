// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Toolkits/AssetEditorToolkit.h"
#include "AssetDefinitionDefault.h"
#include "IDetailCustomization.h"
#include "Widgets/Views/SListView.h"
#include "Factories/Factory.h"

#include "AnimDatabaseEditorToolkit.generated.h"

class IDetailsView;
class UAnimDatabase;
class ITableRow;
class STableViewBase;
class UAnimDatabaseQuery;
namespace ESelectInfo { enum Type : int; }

/** Asset definition for the Animation Database class */
UCLASS()
class UAnimDatabaseEditorAssetDefinition : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:

	virtual FText GetAssetDisplayName() const override final;
	virtual FLinearColor GetAssetColor() const override final;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override final;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override final;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override final;
};

/** Asset factory for the Animation Database class */
UCLASS(MinimalAPI, hidecategories = Object)
class UAnimDatabaseEditorFactory : public UFactory
{
	GENERATED_BODY()

public:
	UAnimDatabaseEditorFactory();

	// UFactory overrides.
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual FText GetDisplayName() const override;
	virtual uint32 GetMenuCategories() const override;
	virtual FText GetToolTip() const override;
	virtual FString GetDefaultNewAssetName() const override;
	virtual bool ConfigureProperties() override;
	virtual bool ShouldShowInNewMenu() const override;
	// ~END UFactory overrides.
};

namespace UE::AnimDatabase::Editor
{
	class FPreviewScene;
	struct FTimelineModel;
	struct FTimelineTracksModel;
	struct FQueryEntry;

	/** Details customization used to render the appropriate UI for the UAnimDatabaseQuery object */
	class FQueryDetails : public IDetailCustomization
	{
	public:

		static TSharedRef<IDetailCustomization> MakeInstance();

		virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

		/** Callback to generate a single row of the list of ranges */
		TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FQueryEntry> Item, const TSharedRef<STableViewBase>& OwnerTable);

		/** Callback for when the selection changes */
		void OnSelectionChanged(TSharedPtr<FQueryEntry> SelectedItem, ESelectInfo::Type SelectInfo);

		/** Callback for when keys are pressed */
		FReply OnKeyDown(UAnimDatabaseQuery* Query, const FKeyEvent& InKeyEvent);

	private:

		/** Keep track of the selected items so we can see when they change */
		TArray<TSharedPtr<FQueryEntry>> SelectedItems;

		/** ListView for all the FQueryEntry items in the query */
		TSharedPtr<SListView<TSharedPtr<FQueryEntry>>> ListView;
	};

	/** Toolkit used for animation database assets */
	class FDatabaseToolkit : public FAssetEditorToolkit
	{
	public:

		void InitAssetEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UAnimDatabase* InDatabase);

		virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;
		virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;
		virtual FName GetToolkitFName() const override;
		virtual FText GetBaseToolkitName() const override;
		virtual FText GetToolkitName() const override;
		virtual FLinearColor GetWorldCentricTabColorScale() const override;
		virtual FString GetWorldCentricTabPrefix() const override;

		/** Reconstruct the timeline widget. This can be used for when new ranges are selected. */
		void ReconstructTimelineWidget();

		/** We keep a pointer to the database asset. We assume this will out-live the editor toolkit. */
		UAnimDatabase* Database = nullptr;

		/** Model for the timeline Widget */
		TSharedPtr<FTimelineModel> TimelineModel;

		/** Model for the tracks Widget */
		TSharedPtr<FTimelineTracksModel> TracksModel;

		/** Preview Scene for the viewport Widget */
		TSharedPtr<FPreviewScene> PreviewScene;

		/** Viewport Tab Widget */
		TSharedPtr<SWidget> ViewportWidget;

		/** Asset Details Panel Tab Widget */
		TSharedPtr<IDetailsView> EditingAssetWidget;
		
		/** Preview Settings Tab Widget */
		TSharedPtr<SWidget> PreviewSettingsWidget;

		/** Viewport Settings Tab Widget */
		TSharedPtr<IDetailsView> ViewportSettingsWidget;

		/** Timeline Tab Widget */
		TSharedPtr<SWidget> TimelineWidget;

		/** Query Tab Widget */
		TSharedPtr<IDetailsView> QueryWidget;

		/** Asset Browser Tab Widget */
		TSharedPtr<SWidget> AssetBrowserWidget;
	};
}
