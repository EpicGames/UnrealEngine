// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorViewportClient.h"

#define UE_API ANIMDATABASEEDITOR_API

class FAssetEditorToolkit;
class SEditorViewport;
struct FDebugDrawerCanvasBuffer;

namespace UE::AnimDatabase::Editor
{
	class FPreviewScene;

	/** Simple Viewport Client that can be used in the editor mode to get the preview scene and the editor toolkit */
	class FViewportClient : public FEditorViewportClient
	{
	public:

		UE_API FViewportClient(const TWeakPtr<FPreviewScene> InPreviewScene, const TSharedRef<SEditorViewport>& InViewport, const TWeakPtr<FAssetEditorToolkit> InAssetEditorToolkit, const FEditorModeID ModeID);
		UE_API virtual ~FViewportClient() override;

		// ~FEditorViewportClient interface
		UE_API virtual void TrackingStarted(const struct FInputEventState& InInputState, bool bIsDragging, bool bNudge) override;
		UE_API virtual void TrackingStopped() override;
		// ~End of FEditorViewportClient interface

		/** Get the preview scene we are viewing */
		UE_API TWeakPtr<FPreviewScene> GetPreviewScene() const;

		/** Gets the Asset Editor Toolkit we are using */
		UE_API TWeakPtr<FAssetEditorToolkit> GetAssetEditorToolkit() const;

		/** Custom Draw Function */
		UE_API virtual void Draw(FViewport* Viewport, FCanvas* Canvas) override;

		/** Clears the current warning message */
		UE_API void ClearWarningMessage();

		/** Sets the current warning message */
		UE_API void SetWarningMessage(const FText& InWarningMessage);

		/** Gets or makes a new DebugDrawCanvasBuffer for this viewport and returns a shared pointer to it */
		UE_API const TSharedPtr<FDebugDrawerCanvasBuffer>& GetOrMakeDebugDrawBuffer();

	private:

		/** Preview scene we are viewing */
		TWeakPtr<FPreviewScene> PreviewScenePtr;

		/** Asset editor toolkit we are embedded in */
		TWeakPtr<FAssetEditorToolkit> AssetEditorToolkitPtr;

		/** Optional warning message to display in viewport */
		FText WarningMessage;

		/** Buffer for debug draw commands */
		TSharedPtr<FDebugDrawerCanvasBuffer> DebugDrawBuffer;
	};
}

#undef UE_API