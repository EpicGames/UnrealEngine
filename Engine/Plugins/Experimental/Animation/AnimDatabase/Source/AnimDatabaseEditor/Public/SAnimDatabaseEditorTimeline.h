// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/NumericTypeInterface.h"

#define UE_API ANIMDATABASEEDITOR_API

class ITimeSliderController;
class ITimeSlider;

namespace UE::AnimDatabase::Editor
{
	struct FTimelineModel;
	struct FTimelineTracksModel;
	class STimelineTransportControls;
	class FTimeSliderController;

	/**
	 * This is a timeline widget used by the Animation Database and other editors. Lots of this code is copy-pasted from the MLDeformer timeline, 
	 * which itself is based off the sequencer timeline.
	 * 
	 * The main changes here are that the data model(s) (i.e. FTimelineModel and FTimelineTracksModel) are changed to be a bit more generic and not 
	 * include anything application specific, and that the timeline is now working by default in terms of frames rather than time. This is more 
	 * appropriate for the Animation Database and related editors since we use a fixed frame-rate and all frames and frame-ranges are using this same 
	 * fixed frame-rate.
	 */
	class STimeline : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(STimeline) {}
		SLATE_END_ARGS()

	public:

		/**
		 * Construct this widget.
		 * @param InArgs The declaration data for this widget.
		 * @param InModel The model for the anim timeline.
		 * @param InTracksModel The model for the anim timeline tracks.
		 */
		UE_API void Construct(const FArguments& InArgs, TWeakPtr<FTimelineModel> InModel, TWeakPtr<FTimelineTracksModel> InTracksModel);

		/** Sets the model(s) to use with this timeline widget */
		UE_API void SetModel(const TWeakPtr<FTimelineModel>& InModel, const TWeakPtr<FTimelineTracksModel>& InTracksModel);

		/** Compute a major grid interval and number of minor divisions to display. */
		UE_API bool GetGridMetrics(float PhysicalWidth, double& OutMajorInterval, int32& OutMinorDivisions) const;

		/** Get the time slider controller. */
		UE_API TSharedPtr<ITimeSliderController> GetTimeSliderController() const;

		/** Callback for mouse button up */
		UE_API virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	private:

		/** Timeline model. */
		TWeakPtr<FTimelineModel> Model;

		/** Tracks model. */
		TWeakPtr<FTimelineTracksModel> TracksModel;

		/** The anim timeline transport controls, which contains the forward, backward and step actions. */
		TSharedPtr<STimelineTransportControls> TransportControls;

		/** The controller used for the time slider */
		TSharedPtr<FTimeSliderController> TimeSliderController;

		/** The top time slider widget. */
		TSharedPtr<ITimeSlider> TopTimeSlider;

		/** Numeric Type interface for converting between frame numbers and display formats. */
		TSharedPtr<INumericTypeInterface<double>> NumericTypeInterface;

		/** Width ratio of the outliner */
		float OutlinerFillCoefficient = 0.25f;

		/** Width ratio of the timeline */
		float TimelineFillCoefficient = 0.75f;
	};

}

#undef UE_API