// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SEditorViewport.h"

#define UE_API ANIMDATABASEEDITOR_API

class FAssetEditorToolkit;

namespace UE::AnimDatabase::Editor
{
	class FPreviewScene;
	class FViewportClient;

	/** Basic Viewport Widget containing the Viewport Client as well as a weak pointer to the Preview Scene */
	class SViewport : public SEditorViewport
	{
	public:

		SLATE_BEGIN_ARGS(SViewport) {}
		SLATE_END_ARGS();

		UE_API void Construct(
			const FArguments& InArgs, 
			const TSharedRef<FAssetEditorToolkit>& InAssetEditorToolkit, 
			const TSharedRef<FPreviewScene>& InPreviewScene, 
			const FEditorModeID InModeID);

	protected:

		// ~SEditorViewport interface
		UE_API virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;
		// ~End of SEditorViewport interface

		/** Viewport client */
		TSharedPtr<FViewportClient> ViewportClient;

		/** The preview scene that we are viewing */
		TWeakPtr<FPreviewScene> PreviewScenePtr;

		/** Asset editor toolkit we are embedded in */
		TWeakPtr<FAssetEditorToolkit> AssetEditorToolkitPtr;

		/** Editor Mode */
		FEditorModeID ModeID;
	};
}

#undef UE_API